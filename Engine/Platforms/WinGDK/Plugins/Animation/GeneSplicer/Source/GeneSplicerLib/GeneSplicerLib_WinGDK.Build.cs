// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GeneSplicerLib_WinGDK : GeneSplicerLib
	{
		public GeneSplicerLib_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDefinitions.Add("TERSE_ENABLE_SSE=1");
			PrivateDefinitions.Add("GS_BUILD_WITH_SSE=1");
		}
	}
}
