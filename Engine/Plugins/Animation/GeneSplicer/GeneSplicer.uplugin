{
	"FileVersion": 3,
	"Version": 10.1,
	"VersionName": "10.1.3",
	"FriendlyName": "GeneSplicer Plugin",
	"Description": "GeneSplicer plugin for facial animation",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "GeneSplicerLib",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "GeneSplicerLibTest",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "GeneSplicerModule",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "RigLogic",
			"Enabled": true
		},
		{		
			"Name": "ControlRig",
			"Enabled": true
		}

	]
}