// Copyright Epic Games, Inc. All Rights Reserved.


#include "GenePoolAssetActions.h"
#include "GenePoolAsset.h"

UClass* FGenePoolAssetTypeActions::GetSupportedClass() const
{
	return UGenePoolAsset::StaticClass();
}

FText FGenePoolAssetTypeActions::GetName() const
{
	return INVTEXT("Gene Pool");
}

FColor FGenePoolAssetTypeActions::GetTypeColor() const
{
	return FColor::Cyan;
}

uint32 FGenePoolAssetTypeActions::GetCategories()
{
	return EAssetTypeCategories::Misc;
}
