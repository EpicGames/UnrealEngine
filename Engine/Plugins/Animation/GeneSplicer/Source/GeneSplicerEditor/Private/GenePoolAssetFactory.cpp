// Copyright Epic Games, Inc. All Rights Reserved.


#include "GenePoolAssetFactory.h"
#include "GenePoolAsset.h"

UGenePoolAssetFactory::UGenePoolAssetFactory()
{
	SupportedClass = UGenePoolAsset::StaticClass();
	bCreateNew = true;

}

UObject* UGenePoolAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UGenePoolAsset>(InParent, Class, Name, Flags, Context);
}
