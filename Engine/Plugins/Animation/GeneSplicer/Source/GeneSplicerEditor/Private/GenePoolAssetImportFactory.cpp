// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenePoolAssetImportFactory.h"

#include "Editor.h"
#include "GenePoolAsset.h"

UGenePoolAssetImportFactory::UGenePoolAssetImportFactory()
{
	bCreateNew = false;
	SupportedClass = UGenePoolAsset::StaticClass();

	bEditorImport = true;
	Formats.Add(TEXT("gp;GenePool file"));
}


UObject* UGenePoolAssetImportFactory::FactoryCreateFile(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	if (bOutOperationCanceled)
	{
		return nullptr;
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, Class, InParent, Name, TEXT("gp"));
	UGenePoolAsset* GPAsset = NewObject<UGenePoolAsset>(InParent, Class, Name, Flags);
	auto GenePoolPtr = MakeShared<FGenePool>(Filename);
	GPAsset->SetGenePoolPtr(GenePoolPtr);
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, GPAsset);
	return GPAsset;
}

bool UGenePoolAssetImportFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename);

	if (Extension == TEXT("gp"))
	{
		return true;
	}

	return false;
}



