// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeneSplicerEditor.h"

#define LOCTEXT_NAMESPACE "FGeneSplicerEditorModule"

void FGeneSplicerEditorModule::StartupModule()
{
	GenePoolAssetTypeActions = MakeShared<FGenePoolAssetTypeActions>();
	FAssetToolsModule::GetModule().Get().RegisterAssetTypeActions(GenePoolAssetTypeActions.ToSharedRef());
}

void FGeneSplicerEditorModule::ShutdownModule()
{
	if (!FModuleManager::Get().IsModuleLoaded("AssetTools")) return;
	FAssetToolsModule::GetModule().Get().UnregisterAssetTypeActions(GenePoolAssetTypeActions.ToSharedRef());
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FGeneSplicerEditorModule, GeneSplicerEditor)