// Copyright Epic Games, Inc. All Rights Reserved.


#include "RegionAffiliationAssetActions.h"
#include "RegionAffiliationAsset.h"

UClass* FRegionAffiliationAssetTypeActions::GetSupportedClass() const
{
	return URegionAffiliationAsset::StaticClass();
}

FText FRegionAffiliationAssetTypeActions::GetName() const
{
	return INVTEXT("Region Affiliation");
}

FColor FRegionAffiliationAssetTypeActions::GetTypeColor() const
{
	return FColor::Cyan;
}

uint32 FRegionAffiliationAssetTypeActions::GetCategories()
{
	return EAssetTypeCategories::Misc;
}
