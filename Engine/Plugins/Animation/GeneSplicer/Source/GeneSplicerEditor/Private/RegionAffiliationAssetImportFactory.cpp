// Copyright Epic Games, Inc. All Rights Reserved.

#include "RegionAffiliationAssetImportFactory.h"

#include "Editor.h"
#include "RegionAffiliationAsset.h"

URegionAffiliationAssetImportFactory::URegionAffiliationAssetImportFactory()
{
	bCreateNew = false;
	SupportedClass = URegionAffiliationAsset::StaticClass();

	bEditorImport = true;
	Formats.Add(TEXT("raf;RegionAffiliation file"));
}

UObject* URegionAffiliationAssetImportFactory::FactoryCreateFile(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	if (bOutOperationCanceled)
	{
		return nullptr;
	}
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, Class, InParent, Name, TEXT("gp"));
	URegionAffiliationAsset* RAFAsset = NewObject<URegionAffiliationAsset>(InParent, Class, Name, Flags);
	auto RegionAffiliationReaderPtr = MakeShared<FRegionAffiliationReader>(Filename);
	RAFAsset->SetRegionAffiliationPtr(RegionAffiliationReaderPtr);
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, RAFAsset);
	return RAFAsset;	
}

bool URegionAffiliationAssetImportFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename);

	if (Extension == TEXT("raf"))
	{
		return true;
	}

	return false;
}



