// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "GenePoolAssetFactory.generated.h"

/**
 * 
 */
UCLASS()
class GENESPLICEREDITOR_API UGenePoolAssetFactory : public UFactory
{
	GENERATED_BODY()
public:
	UGenePoolAssetFactory();
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};
