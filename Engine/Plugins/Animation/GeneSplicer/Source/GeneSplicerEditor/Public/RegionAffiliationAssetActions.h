// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeActions_Base.h"


class FRegionAffiliationAssetTypeActions : public FAssetTypeActions_Base
{
public:
	UClass* GetSupportedClass() const override;
	FText GetName() const override;
	FColor GetTypeColor() const override;
	uint32 GetCategories() override;
};
