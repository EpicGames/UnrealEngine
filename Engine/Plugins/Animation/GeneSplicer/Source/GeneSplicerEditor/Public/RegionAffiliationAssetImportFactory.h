// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Factories/Factory.h"
#include "RegionAffiliationReader.h"
#include "RegionAffiliationAssetImportFactory.generated.h"


UCLASS()
class GENESPLICEREDITOR_API URegionAffiliationAssetImportFactory: public UFactory
{ 
	GENERATED_BODY()

public:
	URegionAffiliationAssetImportFactory();
	/** UFactory interface */
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual void CleanUp() override {};
};
