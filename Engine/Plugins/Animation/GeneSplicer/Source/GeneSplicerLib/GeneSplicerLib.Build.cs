// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GeneSplicerLib : ModuleRules
	{
		public GeneSplicerLib(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
						"Core",
						"CoreUObject",
						"Engine",
						"RigLogicLib"
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PublicDependencyModuleNames.Add("UnrealEd");
			}

			Type = ModuleType.CPlusPlus;

			if (Target.LinkType != TargetLinkType.Monolithic)
			{
				PrivateDefinitions.Add("GS_BUILD_SHARED=1");
				PublicDefinitions.Add("GS_SHARED=1");
			}

			PrivateDefinitions.Add("GS_AUTODETECT_SSE=1");
		}
	}
}
