// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef GENESPLICER_MODULE_DISCARD

#include "GeneSplicerLib.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FGeneSplicerLib"

DEFINE_LOG_CATEGORY(LogGeneSplicerLib);

void FGeneSplicerLib::StartupModule()
{
}

void FGeneSplicerLib::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FGeneSplicerLib, GeneSplicerLib)

#endif  // GENESPLICER_MODULE_DISCARD
