// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/GeneSplicer.h"

#include "genesplicer/CalculationType.h"
#include "genesplicer/GeneSplicerImpl.h"
#include "genesplicer/splicedata/GenePool.h"

namespace gs4 {

GeneSplicer::GeneSplicer(CalculationType calculationType, MemoryResource* memRes) {
    switch (calculationType) {
    case CalculationType::SSE:
        pImpl = ScopedPtr<Impl, FactoryDestroy<Impl>>(Impl::create<CalculationType::SSE>(memRes));
        break;
    case CalculationType::AVX:
        pImpl = ScopedPtr<Impl, FactoryDestroy<Impl>>(Impl::create<CalculationType::AVX>(memRes));
        break;
    case CalculationType::Scalar:
    default:
        pImpl = ScopedPtr<Impl, FactoryDestroy<Impl>>(Impl::create<CalculationType::Scalar>(memRes));
        break;
    }
}

GeneSplicer::~GeneSplicer() = default;

GeneSplicer::GeneSplicer(GeneSplicer&& rhs) = default;
GeneSplicer& GeneSplicer::operator=(GeneSplicer&& rhs) = default;

void GeneSplicer::splice(const SpliceData* spliceData, GeneSplicerDNAReader* output) {
    pImpl->splice(*spliceData, output);
}

void GeneSplicer::spliceNeutralMeshes(const SpliceData* spliceData, GeneSplicerDNAReader* output) {
    pImpl->spliceNeutralMeshes(*spliceData, output);
}

void GeneSplicer::spliceBlendShapes(const SpliceData* spliceData, GeneSplicerDNAReader* output) {
    pImpl->spliceBlendShapes(*spliceData, output);
}

void GeneSplicer::spliceNeutralJoints(const SpliceData* spliceData, GeneSplicerDNAReader* output) {
    pImpl->spliceNeutralJoints(*spliceData, output);
}

void GeneSplicer::spliceJointBehavior(const SpliceData* spliceData, GeneSplicerDNAReader* output) {
    pImpl->spliceJointBehavior(*spliceData, output);
}

void GeneSplicer::spliceSkinWeights(const SpliceData* spliceData, GeneSplicerDNAReader* output) {
    pImpl->spliceSkinWeights(*spliceData, output);
}

}  // namespace gs4
