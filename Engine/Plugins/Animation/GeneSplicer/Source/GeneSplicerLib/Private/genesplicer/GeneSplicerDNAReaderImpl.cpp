// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/GeneSplicerDNAReaderImpl.h"

#include "genesplicer/dna/LODMapping.h"
#include "genesplicer/types/Aliases.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cstddef>
#include <cstring>
#include <limits>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

namespace {

using namespace dna;

}  // namespace

GeneSplicerDNAReader::~GeneSplicerDNAReader() = default;

GeneSplicerDNAReader* GeneSplicerDNAReader::create(const Reader* reader, MemoryResource* memRes) {
    PolyAllocator<GeneSplicerDNAReaderImpl> alloc{memRes};
    auto instance = alloc.newObject(memRes);
    instance->setFrom(reader, DataLayer::All, UnknownLayerPolicy::Preserve, memRes);
    return instance;
}

void GeneSplicerDNAReader::destroy(GeneSplicerDNAReader* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto reader = static_cast<GeneSplicerDNAReaderImpl*>(instance);
    PolyAllocator<GeneSplicerDNAReaderImpl> alloc{reader->getMemoryResource()};
    alloc.deleteObject(reader);
}

GeneSplicerDNAReaderImpl::GeneSplicerDNAReaderImpl(MemoryResource* memRes_) :
    BaseImpl{memRes_},
    ReaderImpl{memRes_},
    WriterImpl{memRes_} {
}

void GeneSplicerDNAReaderImpl::setJointGroups(Vector<RawJointGroup>&& jointGroups) {
    dna.behavior.joints.jointGroups = std::move(jointGroups);
}

void GeneSplicerDNAReaderImpl::setVertexPositions(std::uint16_t meshIndex, RawVector3Vector&& positions) {
    ensureHasSize(dna.geometry.meshes, meshIndex + 1ul);
    dna.geometry.meshes[meshIndex].positions = std::move(positions);
}

void GeneSplicerDNAReaderImpl::setVertexNormals(std::uint16_t meshIndex, RawVector3Vector&& normals) {
    ensureHasSize(dna.geometry.meshes, meshIndex + 1ul);
    dna.geometry.meshes[meshIndex].normals = std::move(normals);
}

void GeneSplicerDNAReaderImpl::setNeutralJointTranslations(RawVector3Vector&& translations) {
    dna.definition.neutralJointTranslations = std::move(translations);
}

void GeneSplicerDNAReaderImpl::setNeutralJointRotations(RawVector3Vector&& rotations) {
    dna.definition.neutralJointRotations = std::move(rotations);
}

void GeneSplicerDNAReaderImpl::setSkinWeights(std::uint16_t meshIndex, Vector<RawVertexSkinWeights>&& rawSkinWeights) {
    ensureHasSize(dna.geometry.meshes, meshIndex + 1ul);
    dna.geometry.meshes[meshIndex].skinWeights = std::move(rawSkinWeights);
}

void GeneSplicerDNAReaderImpl::setBlendShapeTargets(std::uint16_t meshIndex, Vector<RawBlendShapeTarget>&& blendShapeTargets) {
    ensureHasSize(dna.geometry.meshes, meshIndex + 1ul);
    dna.geometry.meshes[meshIndex].blendShapeTargets = std::move(blendShapeTargets);
}

void GeneSplicerDNAReaderImpl::unload(DataLayer layer) {
    if ((layer == DataLayer::All) || (layer == DataLayer::Descriptor)) {
        dna = DNA{dna.layers.unknownPolicy, dna.layers.upgradePolicy, memRes};
    } else if (layer == DataLayer::TwistSwingBehavior) {
        dna.unloadTwistSwingBehavior();
    } else if (layer == DataLayer::RBFBehavior) {
        dna.unloadRBFBehavior();
    } else if (layer == DataLayer::JointBehaviorMetadata) {
        dna.unloadJointBehaviorMetadata();
    } else if (layer == DataLayer::MachineLearnedBehavior) {
        dna.unloadMachineLearnedBehavior();
    } else if ((layer == DataLayer::Geometry) || (layer == DataLayer::GeometryWithoutBlendShapes)) {
        dna.unloadGeometry();
    } else if (layer == DataLayer::Behavior) {
        dna.unloadRBFBehavior();
        dna.unloadBehavior();
    } else if (layer == DataLayer::Definition) {
        dna.unloadJointBehaviorMetadata();
        dna.unloadTwistSwingBehavior();
        dna.unloadRBFBehavior();
        dna.unloadMachineLearnedBehavior();
        dna.unloadGeometry();
        dna.unloadBehavior();
        dna.unloadDefinition();
    }
}

}  // namespace gs4
