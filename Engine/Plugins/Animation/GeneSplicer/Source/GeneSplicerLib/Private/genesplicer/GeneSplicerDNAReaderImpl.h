// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/GeneSplicerDNAReader.h"
#include "genesplicer/dna/Aliases.h"
#include "genesplicer/dna/ReaderImpl.h"
#include "genesplicer/dna/WriterImpl.h"

#include <dna/Reader.h>
#include <dna/Writer.h>

#include <cstdint>

namespace gs4 {

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4250)
#endif

class GeneSplicerDNAReaderImpl : public ReaderImpl<GeneSplicerDNAReader>, public WriterImpl<dna::Writer> {
public:
    explicit GeneSplicerDNAReaderImpl(MemoryResource* memRes_);

    // Reader methods
    void unload(DataLayer layer) override;

    // GSWriter methods
    void setJointGroups(Vector<RawJointGroup>&& jointGroups);

    using WriterImpl<dna::Writer>::setVertexPositions;
    void setVertexPositions(std::uint16_t meshIndex, RawVector3Vector&& positions);

    using WriterImpl<dna::Writer>::setVertexNormals;
    void setVertexNormals(std::uint16_t meshIndex, RawVector3Vector&& normals);

    using WriterImpl<dna::Writer>::setNeutralJointTranslations;
    void setNeutralJointTranslations(RawVector3Vector&& translations);

    using WriterImpl<dna::Writer>::setNeutralJointRotations;
    void setNeutralJointRotations(RawVector3Vector&& rotations);

    void setSkinWeights(std::uint16_t meshIndex, Vector<RawVertexSkinWeights>&& rawSkinWeights);
    void setBlendShapeTargets(std::uint16_t meshIndex, Vector<RawBlendShapeTarget>&& blendShapeTargets);
};

#ifdef _MSC_VER
    #pragma warning(pop)
#endif

}  // namespace gs4
