// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/GeneSplicerImpl.h"

#include "genesplicer/Macros.h"
#include "genesplicer/Splicer.h"
#include "genesplicer/splicedata/SpliceData.h"
#include "genesplicer/types/PImplExtractor.h"

namespace gs4 {

GeneSplicer::Impl::Impl(SplicerPtr neutralJointSplicer_,
                        SplicerPtr jointBehaviorSplicer_,
                        SplicerPtr blendShapeSplicer_,
                        SplicerPtr neutralMeshSplicer_,
                        SplicerPtr skinWeightSplicer_,
                        MemoryResource* memRes_) :
    memRes{memRes_},
    neutralJointSplicer{std::move(neutralJointSplicer_)},
    jointBehaviorSplicer{std::move(jointBehaviorSplicer_)},
    blendShapeSplicer{std::move(blendShapeSplicer_)},
    neutralMeshSplicer{std::move(neutralMeshSplicer_)},
    skinWeightSplicer{std::move(skinWeightSplicer_)} {
}

void GeneSplicer::Impl::destroy(GeneSplicer::Impl* instance) {
    PolyAllocator<GeneSplicer::Impl> alloc{instance->memRes};
    alloc.deleteObject(instance);
}

void GeneSplicer::Impl::splice(const SpliceData& spliceData, GeneSplicerDNAReader* output) {
    spliceNeutralMeshes(spliceData, output);
    spliceBlendShapes(spliceData, output);
    spliceNeutralJoints(spliceData, output);
    spliceJointBehavior(spliceData, output);
    spliceSkinWeights(spliceData, output);
}

void GeneSplicer::Impl::spliceNeutralMeshes(const SpliceData& spliceData, GeneSplicerDNAReader* output) {
    neutralMeshSplicer.get()->splice(PImplExtractor<SpliceData>::get(spliceData), output);
}

void GeneSplicer::Impl::spliceBlendShapes(const SpliceData& spliceData, GeneSplicerDNAReader* output) {
    blendShapeSplicer.get()->splice(PImplExtractor<SpliceData>::get(spliceData), output);
}

void GeneSplicer::Impl::spliceNeutralJoints(const SpliceData& spliceData, GeneSplicerDNAReader* output) {
    neutralJointSplicer.get()->splice(PImplExtractor<SpliceData>::get(spliceData), output);
}

void GeneSplicer::Impl::spliceJointBehavior(const SpliceData& spliceData, GeneSplicerDNAReader* output) {
    jointBehaviorSplicer.get()->splice(PImplExtractor<SpliceData>::get(spliceData), output);
}

void GeneSplicer::Impl::spliceSkinWeights(const SpliceData& spliceData, GeneSplicerDNAReader* output) {
    skinWeightSplicer.get()->splice(PImplExtractor<SpliceData>::get(spliceData), output);
}

}  // namespace gs4
