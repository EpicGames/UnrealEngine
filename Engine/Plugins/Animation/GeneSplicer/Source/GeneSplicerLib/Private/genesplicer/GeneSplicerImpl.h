// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/GeneSplicer.h"
#include "genesplicer/Splicer.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/blendshapesplicer/BlendShapeSplicer.h"
#include "genesplicer/geometrysplicer/NeutralMeshSplicer.h"
#include "genesplicer/jointbehaviorsplicer/JointBehaviorSplicer.h"
#include "genesplicer/neutraljointsplicer/NeutralJointSplicer.h"
#include "genesplicer/skinweightsplicer/SkinWeightSplicer.h"

namespace gs4 {

class GeneSplicerDNAReader;
class SpliceData;

class GeneSplicer::Impl {
private:
    using SplicerPtr = pma::UniqueInstance<Splicer>::PointerType;

public:
    Impl(SplicerPtr neutralJointSplicer_,
         SplicerPtr jointBehaviorSplicer_,
         SplicerPtr blendShapeSplicer_,
         SplicerPtr neutralMeshSplicer_,
         SplicerPtr skinWeightSplicer_,
         MemoryResource* memRes);

    template<CalculationType CT>
    static Impl* create(MemoryResource* memRes) {
        PolyAllocator<GeneSplicer::Impl> alloc{memRes};
        SplicerPtr neutralJointSplicer = pma::UniqueInstance<NeutralJointSplicer<CT>, Splicer>::with(memRes).create(memRes);
        SplicerPtr jointBehaviorSplicer = pma::UniqueInstance<JointBehaviorSplicer<CT>, Splicer>::with(memRes).create(memRes);
        SplicerPtr blendShapeSplicer = pma::UniqueInstance<BlendShapeSplicer<CT>, Splicer>::with(memRes).create(memRes);
        SplicerPtr neutralMeshSplicer = pma::UniqueInstance<NeutralMeshSplicer<CT>, Splicer>::with(memRes).create(memRes);
        SplicerPtr skinWeightSplicer = pma::UniqueInstance<SkinWeightSplicer<CT>, Splicer>::with(memRes).create(memRes);
        return alloc.newObject(std::move(neutralJointSplicer),
                               std::move(jointBehaviorSplicer),
                               std::move(blendShapeSplicer),
                               std::move(neutralMeshSplicer),
                               std::move(skinWeightSplicer),
                               memRes);
    }

    static void destroy(Impl* instance);
    void splice(const SpliceData& spliceData, GeneSplicerDNAReader* output);
    void spliceNeutralJoints(const SpliceData& spliceData, GeneSplicerDNAReader* output);
    void spliceNeutralMeshes(const SpliceData& spliceData, GeneSplicerDNAReader* output);
    void spliceBlendShapes(const SpliceData& spliceData, GeneSplicerDNAReader* output);
    void spliceJointBehavior(const SpliceData& spliceData, GeneSplicerDNAReader* output);
    void spliceSkinWeights(const SpliceData& spliceData, GeneSplicerDNAReader* output);

private:
    MemoryResource* memRes;
    SplicerPtr neutralJointSplicer;
    SplicerPtr jointBehaviorSplicer;
    SplicerPtr blendShapeSplicer;
    SplicerPtr neutralMeshSplicer;
    SplicerPtr skinWeightSplicer;
};

}  // namespace gs4
