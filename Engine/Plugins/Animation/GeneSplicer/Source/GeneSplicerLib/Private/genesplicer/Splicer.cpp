// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/Splicer.h"

namespace gs4 {

Splicer::~Splicer() = default;

Splicer::Splicer(MemoryResource* memRes_) :
    memRes{memRes_} {
}

}  // namespace gs4
