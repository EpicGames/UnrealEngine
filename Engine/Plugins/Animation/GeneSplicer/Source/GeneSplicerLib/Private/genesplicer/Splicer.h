// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/CalculationType.h"
#include "genesplicer/splicedata/SpliceDataImpl.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"

#include <cstddef>

namespace gs4 {

class GeneSplicerDNAReader;
class SpliceData;

class Splicer {
public:
    explicit Splicer(MemoryResource* memRes);
    virtual void splice(const SpliceDataInterface* spliceData, GeneSplicerDNAReader* output) = 0;
    virtual ~Splicer();

protected:
    MemoryResource* memRes;
};

}  // namespace gs4
