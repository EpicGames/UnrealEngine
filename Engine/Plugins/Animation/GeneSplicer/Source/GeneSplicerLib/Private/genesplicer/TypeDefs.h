// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/system/SIMD.h"
#include "genesplicer/types/Aliases.h"

#include <pma/PolyAllocator.h>
#include <pma/TypeDefs.h>
#include <pma/resources/AlignedMemoryResource.h>
#include <pma/resources/DefaultMemoryResource.h>
#include <pma/utils/ManagedInstance.h>
#include <status/Provider.h>
#include <tdm/TDM.h>
#include <terse/types/ArchiveOffset.h>
#include <terse/types/Blob.h>
#include <terse/types/DynArray.h>

namespace gs4 {

using namespace pma;
using namespace tdm;
using namespace terse;

template<typename T>
using AlignedAllocator = PolyAllocator<T, trimd::F256::alignment(), AlignedMemoryResource>;

template<typename T>
using AlignedVector = Vector<T, AlignedAllocator<T>>;

template<typename T>
using Blob = terse::Blob<T, PolyAllocator<T>>;

template<typename T>
using DynArray = terse::DynArray<T, PolyAllocator<T>>;

template<typename T>
using AlignedDynArray = terse::DynArray<T, AlignedAllocator<T>>;

template<typename T>
using AlignedMatrix = Vector<AlignedVector<T>>;

static constexpr auto cFalse = static_cast<char>(0);
static constexpr auto cTrue = static_cast<char>(1);

}  // namespace gs4
