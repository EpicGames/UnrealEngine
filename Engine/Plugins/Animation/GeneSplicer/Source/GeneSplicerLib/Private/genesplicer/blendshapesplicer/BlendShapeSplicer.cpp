// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/blendshapesplicer/BlendShapeSplicer.h"

#include "genesplicer/GeneSplicerDNAReaderImpl.h"
#include "genesplicer/Macros.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Aliases.h"
#include "genesplicer/splicedata/PoolSpliceParamsFilter.h"
#include "genesplicer/splicedata/PoolSpliceParamsImpl.h"
#include "genesplicer/splicedata/SpliceDataImpl.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/Block.h"
#include "genesplicer/types/PImplExtractor.h"
#include "genesplicer/types/Vec3.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4242 4244 4365 4987)
#endif
#include <algorithm>
#include <cstddef>
#include <cstdint>
#include <iterator>
#include <limits>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

namespace {

Vector<RawBlendShapeTarget> constructBlendShapeTargetsWithPadding(ConstArrayView<RawBlendShapeTarget> sourceBlendShapeTargets,
                                                                  MemoryResource* memRes) {

    auto bsCount = static_cast<std::uint16_t>(sourceBlendShapeTargets.size());
    Vector<RawBlendShapeTarget> resultingBlendShapeTargets{memRes};
    resultingBlendShapeTargets.reserve(bsCount);

    for (const auto& blendShapeTarget : sourceBlendShapeTargets) {
        RawBlendShapeTarget paddedBlendShape{memRes};
        paddedBlendShape.blendShapeChannelIndex = blendShapeTarget.blendShapeChannelIndex;
        paddedBlendShape.deltas = constructWithPadding(blendShapeTarget.deltas, memRes, 4);
        paddedBlendShape.vertexIndices.assign(blendShapeTarget.vertexIndices.begin(), blendShapeTarget.vertexIndices.end());
        resultingBlendShapeTargets.push_back(std::move(paddedBlendShape));
    }
    return resultingBlendShapeTargets;
}

class FilteredIterator {
public:
    FilteredIterator(ConstArrayView<std::uint16_t> blockDnaIndices_, ConstArrayView<std::uint16_t> dnaFilter_) :
        blockDnaIndices{blockDnaIndices_},
        dnaFilter{dnaFilter_} {
        advance();
    }

    void advance() {
        while (i < blockDnaIndices.size() && j < dnaFilter.size()) {
            if (blockDnaIndices[i] < dnaFilter[j]) {
                i++;
            } else if (dnaFilter[j] < blockDnaIndices[i]) {
                j++;
            } else {
                break;
            }
        }
    }

    std::uint16_t getIndex() {
        return i;
    }

    std::uint16_t consumeIndex() {
        return i++;
    }

private:
    ConstArrayView<std::uint16_t> blockDnaIndices;
    ConstArrayView<std::uint16_t> dnaFilter;
    std::uint16_t i = 0u;
    std::uint16_t j = 0u;
};

}  // namespace

template<CalculationType CT>
void BlendShapeSplicer<CT>::splice(const SpliceDataInterface* spliceData, GeneSplicerDNAReader* output_) {
    auto output = static_cast<GeneSplicerDNAReaderImpl*>(output_);
    auto outputMemRes = output->getMemoryResource();

    const auto& baseArch = spliceData->getBaseArchetype();
    auto baseArchBlendShapes = baseArch.getBlendShapeTargets();

    auto blendShapePredicate = [](const RawGenes& baseArchetype, const PoolSpliceParamsImpl* pool, std::uint16_t meshIndex) {
        return pool->getGenePool()->getBlendShapeTargetCount(meshIndex) == baseArchetype.getBlendShapeTargets()[meshIndex].size();
    };
    auto poolsToSplicePerMesh = filterPoolSpliceParamsPerMesh(spliceData, blendShapePredicate, outputMemRes);

    for (std::uint16_t meshIdx = 0u; meshIdx < poolsToSplicePerMesh.rowCount(); meshIdx++) {
        Vector<RawBlendShapeTarget> resultingBlendShapeTargets =
            constructBlendShapeTargetsWithPadding(baseArchBlendShapes[meshIdx], outputMemRes);

        auto poolParams = poolsToSplicePerMesh[meshIdx];

        for (std::uint16_t bsIdx = 0u; bsIdx < resultingBlendShapeTargets.size(); bsIdx++) {
            auto& resultDeltas = resultingBlendShapeTargets[bsIdx].deltas;
            for (const auto& pool : poolParams) {
                const auto& deltas = pool->getGenePool()->getBlendShapeTargetDeltas();  // -V758

                const auto& vertexWeights = pool->getVertexWeightsData()[meshIdx];  // -V758
                auto dnaFilter = pool->getDNAIndices();                             // -V758
                float scale = pool->getScale();                                     // -V758

                using TF128 = typename GetTF128<CT>::type;
                const TF128 scale128(scale);

                // We get our first bucket for this BlendShape and we want to iterate to first bucket of next blend shape
                const std::size_t* bucketOffsetPtr = &deltas.bucketOffsets[meshIdx][bsIdx];
                std::size_t bucketIndex = *bucketOffsetPtr;
                const std::size_t endBucketIndex = *std::next(bucketOffsetPtr);

                for (; bucketIndex < endBucketIndex; bucketIndex++) {
                    const std::uint32_t vertexIndex = deltas.bucketVertexIndices[bucketIndex];

                    const auto blocksOffset = deltas.bucketDNABlockOffsets[bucketIndex];
                    const auto blockCount = deltas.bucketDNABlockOffsets[bucketIndex + 1u] - blocksOffset;
                    ConstArrayView<std::uint16_t> dnaIndices{&deltas.dnaIndices[blocksOffset], blockCount};
                    ConstArrayView<XYZBlock<4>> dnaBlocks{&deltas.dnaBlocks[blocksOffset], blockCount};

                    // Since our indices are divisible by 4(BlockSize) by design, we will hit our alignment
                    assert(vertexIndex % 4u == 0u);
                    float* destX = resultDeltas.xs.data() + vertexIndex;
                    float* destY = resultDeltas.ys.data() + vertexIndex;
                    float* destZ = resultDeltas.zs.data() + vertexIndex;

                    auto sumX = TF128::fromAlignedSource(destX);
                    auto sumY = TF128::fromAlignedSource(destY);
                    auto sumZ = TF128::fromAlignedSource(destZ);
                    // Weights are packed into blocks of 16, so we have to calculate offsets
                    auto weightBlock = vertexWeights[vertexIndex / 16];
                    // Same thing here vertexIndex % 16 must be divisible by 4(BlockSize) so we will hit our alignment
                    const auto weightOffset = static_cast<std::uint16_t>(vertexIndex % 16);
                    for (FilteredIterator iter{dnaIndices, dnaFilter}; iter.getIndex() < dnaIndices.size(); iter.advance()) {
                        const std::uint16_t i = iter.consumeIndex();
                        const std::uint16_t dnaIdx = dnaIndices[i];
                        const auto& dna = dnaBlocks[i];
                        const auto weight = scale128 * TF128::fromAlignedSource(weightBlock[dnaIdx].v + weightOffset);
                        sumX += TF128::fromAlignedSource(dna.Xs) * weight;
                        sumY += TF128::fromAlignedSource(dna.Ys) * weight;
                        sumZ += TF128::fromAlignedSource(dna.Zs) * weight;
                    }

                    auto weightSum = TF128{};
                    for (const auto dnaIndex : dnaFilter) {
                        weightSum += scale128 * TF128::fromAlignedSource(weightBlock[dnaIndex].v + weightOffset);
                    }

                    const auto& arch = deltas.archBlocks[bucketIndex];
                    sumX -= TF128::fromAlignedSource(arch.Xs) * weightSum;
                    sumY -= TF128::fromAlignedSource(arch.Ys) * weightSum;
                    sumZ -= TF128::fromAlignedSource(arch.Zs) * weightSum;
                    sumX.alignedStore(destX);
                    sumY.alignedStore(destY);
                    sumZ.alignedStore(destZ);
                }
            }
            const auto& resultIndices = resultingBlendShapeTargets[bsIdx].vertexIndices;
            for (std::uint32_t i = 0u; i < resultIndices.size(); i++) {
                const auto index = resultIndices[i];
                resultDeltas.xs[i] = resultDeltas.xs[index];
                resultDeltas.ys[i] = resultDeltas.ys[index];
                resultDeltas.zs[i] = resultDeltas.zs[index];
            }
            resultDeltas.resize(resultIndices.size());
        }
        output->setBlendShapeTargets(meshIdx, std::move(resultingBlendShapeTargets));
    }
}

template class BlendShapeSplicer<CalculationType::Scalar>;
template class BlendShapeSplicer<CalculationType::SSE>;
template class BlendShapeSplicer<CalculationType::AVX>;

}  // namespace gs4
