// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/Splicer.h"
#include "genesplicer/splicedata/PoolSpliceParams.h"
#include "genesplicer/splicedata/SpliceDataImpl.h"
#include "genesplicer/types/VariableWidthMatrix.h"

namespace gs4 {

class GeneSplicerDNAReader;
class SpliceData;

template<CalculationType CT>
class BlendShapeSplicer : public Splicer {
public:
    using Splicer::Splicer;

    void splice(const SpliceDataInterface* spliceData, GeneSplicerDNAReader* output) override;
};

}  // namespace gs4
