// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/dna/DNA.h"
#include "genesplicer/types/Aliases.h"

namespace gs4 {

class BaseImpl {
protected:
    explicit BaseImpl(MemoryResource* memRes_) :
        memRes{memRes_},
        dna{UnknownLayerPolicy::Preserve, UpgradeFormatPolicy::Allowed, memRes} {
    }

    BaseImpl(UnknownLayerPolicy unknownPolicy, UpgradeFormatPolicy upgradePolicy, MemoryResource* memRes_) :
        memRes{memRes_},
        dna{unknownPolicy, upgradePolicy, memRes} {
    }

    ~BaseImpl() = default;

    BaseImpl(const BaseImpl&) = delete;
    BaseImpl& operator=(const BaseImpl&) = delete;

    BaseImpl(BaseImpl&& rhs) = delete;
    BaseImpl& operator=(BaseImpl&&) = delete;

public:
    MemoryResource* getMemoryResource() {
        return memRes;
    }

protected:
    MemoryResource* memRes;
    mutable DNA dna;
};

}  // namespace gs4
