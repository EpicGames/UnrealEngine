// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/Macros.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Extd.h"
#include "genesplicer/dna/LODMapping.h"
#include "genesplicer/dna/SurjectiveMapping.h"
#include "genesplicer/types/ExpectedValue.h"
#include "genesplicer/types/Signature.h"

#include <dna/Configuration.h>
#include <terse/types/Anchor.h>
#include <terse/types/ArchiveOffset.h>
#include <terse/types/ArchiveSize.h>
#include <terse/types/Transparent.h>
#include <terse/types/Versioned.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <array>
#include <cassert>
#include <cstddef>
#include <cstdint>
#include <iterator>
#include <memory>
#include <tuple>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace tdm {

#ifndef TDM_ROT_SEQ_SERIALIZER_DEFINED
    #define TDM_ROT_SEQ_SERIALIZER_DEFINED

template<typename Archive>
void load(Archive& archive, tdm::rot_seq& value) {
    archive.label("value");
    std::uint32_t tmp = {};
    archive(tmp);
    value = static_cast<tdm::rot_seq>(tmp);
}

template<typename Archive>
void save(Archive& archive, tdm::rot_seq& value) {
    archive.label("value");
    archive(static_cast<std::uint32_t>(value));
}

#endif  // TDM_ROT_SEQ_SERIALIZER_DEFINED

#ifndef TDM_ROT_SIGN_SERIALIZER_DEFINED
    #define TDM_ROT_SIGN_SERIALIZER_DEFINED

template<typename Archive>
void load(Archive& archive, tdm::rot_sign& value) {
    std::uint32_t tmp = {};

    archive.label("x");
    archive(tmp);
    value.x = static_cast<tdm::rot_dir>(tmp);

    archive.label("y");
    archive(tmp);
    value.y = static_cast<tdm::rot_dir>(tmp);

    archive.label("z");
    archive(tmp);
    value.z = static_cast<tdm::rot_dir>(tmp);
}

template<typename Archive>
void save(Archive& archive, tdm::rot_sign& value) {
    archive.label("x");
    archive(static_cast<std::uint32_t>(value.x));

    archive.label("y");
    archive(static_cast<std::uint32_t>(value.y));

    archive.label("z");
    archive(static_cast<std::uint32_t>(value.z));
}

#endif  // TDM_ROT_SIGN_SERIALIZER_DEFINED

}  // namespace tdm

namespace gs4 {

using dna::CoordinateSystem;
using dna::RotationDirection;
using dna::RotationSequence;
using dna::RotationSign;
using dna::FaceWindingOrder;

template<typename Archive>
void load(Archive& archive, FaceWindingOrder& value) {
    archive.label("value");
    std::uint8_t tmp = {};
    archive(tmp);
    value = static_cast<FaceWindingOrder>(tmp);
}

template<typename Archive>
void save(Archive& archive, FaceWindingOrder& value) {
    archive.label("value");
    archive(static_cast<std::uint8_t>(value));
}

template<typename TVersion, TVersion... Args>
struct Dispatcher;

template<typename TVersion, TVersion Fallback>
struct Dispatcher<TVersion, Fallback> {

    template<class Archive, typename TSerializable>
    void operator()(Archive& archive, TSerializable& object, TVersion /*unused*/, bool transparent) {
        // Dispatching occurs only after DNA version has already been read, identified and confirmed
        // to be supported (the unknown enum won't be passed here ever). Likewise, if a newer
        // version is encountered than the latest version listed here, it will be dispatched to the
        // latest available serializer.
        auto versionedObject = terse::versioned(object, terse::Version<Fallback>{});
        if (transparent) {
            archive(terse::transparent(versionedObject));
        } else {
            archive(versionedObject);
        }
    }
};

template<typename TVersion, TVersion Expected, TVersion... Rest>
struct Dispatcher<TVersion, Expected, Rest...> : Dispatcher<TVersion, Rest...> {
    using Super = Dispatcher<TVersion, Rest...>;

    template<class Archive, typename TSerializable>
    void operator()(Archive& archive, TSerializable& object, TVersion version, bool transparent) {
        if (version == Expected) {
            auto versionedObject = terse::versioned(object, terse::Version<Expected>{});
            if (transparent) {
                archive(terse::transparent(versionedObject));
            } else {
                archive(versionedObject);
            }
        } else {
            Super::operator()(archive, object, version, transparent);
        }
    }
};

static constexpr std::uint32_t sid4(const char* name) {
    return ((static_cast<std::uint32_t>(name[0]) << 24u) + (static_cast<std::uint32_t>(name[1]) << 16u) +
            (static_cast<std::uint32_t>(name[2]) << 8u) + (static_cast<std::uint32_t>(name[3]) << 0u));
}

static constexpr std::uint32_t rev(std::uint16_t generation, std::uint16_t version) {
    return (static_cast<std::uint32_t>(generation) << 16u) + static_cast<std::uint32_t>(version);
}

static constexpr std::uint16_t high16(std::uint32_t packed) {
    return static_cast<std::uint16_t>(packed >> 16u);
}

static constexpr std::uint16_t low16(std::uint32_t packed) {
    return static_cast<std::uint16_t>(packed & 0x0000FFFFu);
}

enum FileVersion : std::uint64_t {
    unknown = 0u,
    v21 = rev(2, 1),
    v22 = rev(2, 2),
    v23 = rev(2, 3),
    v24 = rev(2, 4),
    v25 = rev(2, 5),
    v26 = rev(2, 6),
    v27 = rev(2, 7),
    v28 = rev(2, 8),
    latest = v28
};

using Dispatch = Dispatcher<FileVersion,
                            FileVersion::v21,
                            FileVersion::v22,
                            FileVersion::v23,
                            FileVersion::v24,
                            FileVersion::v25,
                            FileVersion::v26,
                            FileVersion::v27,
                            FileVersion::v28>;

template<typename A, FileVersion B>
using IfHigher = typename std::enable_if<(A::value() > B), void>::type;

template<typename A, FileVersion B>
using IfLower = typename std::enable_if<(A::value() < B), void>::type;

enum class UpgradeFormatPolicy {
    Disallowed,
    Allowed
};

struct SerializationContext {
    FileVersion version;
    void* data;
};

struct Version {
    std::uint16_t generation;
    std::uint16_t version;

    Version(FileVersion fileVersion) :
        generation{high16(static_cast<std::uint32_t>(fileVersion))},
        version{low16(static_cast<std::uint32_t>(fileVersion))} {
    }

    operator FileVersion() const {
        return static_cast<FileVersion>(rev(generation, version));
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("generation");
        archive(generation);
        archive.label("version");
        archive(version);
    }

    bool matches(FileVersion fileVersion) const {
        const Version expected{fileVersion};
        return (generation == expected.generation) && (version == expected.version);
    }

    bool supported() const {
        return (generation == Version{FileVersion::latest}.generation);
    }
};

struct SectionLookupTable {
    terse::ArchiveOffset<std::uint32_t> descriptor;
    terse::ArchiveOffset<std::uint32_t> definition;
    terse::ArchiveOffset<std::uint32_t> behavior;
    terse::ArchiveOffset<std::uint32_t> controls;
    terse::ArchiveOffset<std::uint32_t> joints;
    terse::ArchiveOffset<std::uint32_t> blendShapeChannels;
    terse::ArchiveOffset<std::uint32_t> animatedMaps;
    terse::ArchiveOffset<std::uint32_t> geometry;

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("descriptor");
        archive(descriptor);
        archive.label("definition");
        archive(definition);
        archive.label("behavior");
        archive(behavior);
        archive.label("controls");
        archive(controls);
        archive.label("joints");
        archive(joints);
        archive.label("blendShapeChannels");
        archive(blendShapeChannels);
        archive.label("animatedMaps");
        archive(animatedMaps);
        archive.label("geometry");
        archive(geometry);
    }
};

struct Index {
    std::uint32_t id;
    std::uint32_t version;
    terse::ArchiveOffset<std::uint32_t> offset;
    terse::ArchiveSize<std::uint32_t, std::uint32_t> size;

    template<std::size_t Length = 16ul>
    std::array<char, Length> label() {
        std::array<char, Length> output;
        std::memcpy(output.data(), &id, sizeof(id));
        std::reverse(output.begin(), extd::advanced(output.begin(), sizeof(id)));
        const std::uint32_t major = version >> 16u;
        const std::uint32_t minor = version & 0x0000FFFFu;
        std::snprintf(output.data() + sizeof(id), output.size() - sizeof(id), "%u.%u", major, minor);
        return output;
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("id");
        archive(id);
        archive.label("version");
        archive(version);
        archive.label("offset");
        archive(offset);
        archive.label("size");
        archive(size);
    }
};

struct IndexTable {
    Vector<Index> entries;

    explicit IndexTable(MemoryResource* memRes) :
        entries{memRes} {
    }

    Index* get(std::uint32_t id, std::uint32_t version) {
        for (auto& index : entries) {
            if ((index.id == id) && (index.version == version)) {
                return &index;
            }
        }
        return nullptr;
    }

    const Index* get(std::uint32_t id, std::uint32_t version) const {
        return const_cast<IndexTable*>(this)->get(id, version);
    }

    Index* create(std::uint32_t id, std::uint32_t version) {
        assert(get(id, version) == nullptr);
        entries.push_back({id, version, {}, {}});
        return &entries.back();
    }

    void destroy(std::uint32_t id, std::uint32_t version) {
        entries.erase(
            std::remove_if(entries.begin(),
                           entries.end(),
                           [id, version](const Index& index) { return (index.id == id) && (index.version == version); }),
            entries.end());
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("entries");
        archive(entries);
    }
};

struct UnknownLayer {
    std::uint32_t id;
    std::uint32_t version;
    Blob<char> data;

    UnknownLayer(std::uint32_t id_, std::uint32_t version_, MemoryResource* memRes) :
        id{id_},
        version{version_},
        data{memRes} {
    }

    std::uint32_t layerId() {
        return id;
    }

    std::uint32_t layerVersion() {
        return version;
    }

    bool layerSupported(FileVersion /*unused*/) {
        return true;
    }

    template<class Archive>
    void serialize(Archive& archive) {
        // Serialization of the layer type itself is transparent
        // and should not generate an additional, new structure level in JSON
        auto context = static_cast<SerializationContext*>(archive.getUserData());
        auto index = static_cast<Index*>(context->data);

        const auto layerName = index->label();
        archive.label(layerName.data());

        archive(terse::proxy(index->offset));
        terse::Anchor<std::uint32_t> base;
        archive(base);
        // Clamp against stream size on input archives to reject malformed/maliciously
        // constructed indices before allocating; output archives are identity.
        data.setSize(archive.boundSize(static_cast<std::size_t>(index->size.value)));
        archive(data);
        archive(terse::proxy(index->size, base));
    }
};

template<std::uint32_t Id, std::uint32_t Version, typename TData, FileVersion AvailableFromFileVersion>
struct Layer : TData {

    template<typename... Args>
    explicit Layer(Args&&... args) :
        TData{std::forward<Args>(args)...} {
    }

    std::uint32_t layerId() {
        return Id;
    }

    std::uint32_t layerVersion() {
        return Version;
    }

    bool layerSupported(FileVersion inVersion) {
        return (inVersion >= AvailableFromFileVersion);
    }

    template<class Archive>
    void serialize(Archive& archive) {
        // Serialization of the layer type itself is transparent
        // and should not generate an additional, new structure level in JSON
        auto context = static_cast<SerializationContext*>(archive.getUserData());
        auto index = static_cast<Index*>(context->data);

        const auto layerName = index->label();
        archive.label(layerName.data());

        archive(terse::proxy(index->offset));
        terse::Anchor<std::uint32_t> base;
        archive(base);
        Dispatch()(archive, static_cast<TData&>(*this), context->version, false);
        archive(terse::proxy(index->size, base));
    }
};

template<typename... TLayer>
struct LayerContainer;

template<>
struct LayerContainer<> {
    UnknownLayerPolicy unknownPolicy;
    UpgradeFormatPolicy upgradePolicy;
    Vector<UnknownLayer> unknownLayers;

    LayerContainer(UnknownLayerPolicy unknownPolicy_, UpgradeFormatPolicy upgradePolicy_, MemoryResource* memRes) :
        unknownPolicy{unknownPolicy_},
        upgradePolicy{upgradePolicy_},
        unknownLayers{memRes} {
    }

    template<typename TArchive>
    void serialize(TArchive& archive) {
        auto context = static_cast<SerializationContext*>(archive.getUserData());
        auto index = static_cast<Index*>(context->data);

        for (auto& layer : unknownLayers) {
            if ((index->id == layer.layerId()) && (index->version == layer.layerVersion())) {
                archive(terse::transparent(layer));
                return;
            }
        }

        // This code path should be executed only by input archives, when loading unknown layers.
        // After initially adding all unknown layers, there is no way for new unknown layers to be
        // encountered, so the above loop will take care of all during saving.
        if (unknownPolicy == UnknownLayerPolicy::Preserve) {
            auto memRes = unknownLayers.get_allocator().getMemoryResource();
            unknownLayers.push_back(UnknownLayer{index->id, index->version, memRes});
            archive(terse::transparent(unknownLayers.back()));
        } else {
            // Mark index for deletion
            context->data = nullptr;
        }
    }

    void updateIndexTable(IndexTable& /*unused*/, FileVersion /*unused*/) {
        // Unknown layers are already in the index (that's the only way they could have been encountered)
    }
};

template<typename TLayer, typename... TRest>
struct LayerContainer<TLayer, TRest...> : LayerContainer<TRest...> {
    using Super = LayerContainer<TRest...>;

    TLayer* layer;

    LayerContainer(UnknownLayerPolicy unknownPolicy_,
                   UpgradeFormatPolicy upgradePolicy_,
                   MemoryResource* memRes,
                   TLayer* layer_,
                   TRest*... rest) :
        Super{unknownPolicy_, upgradePolicy_, memRes, std::forward<TRest*>(rest)...},
        layer{layer_} {
    }

    template<typename TArchive>
    void serialize(TArchive& archive) {
        auto context = static_cast<SerializationContext*>(archive.getUserData());
        auto index = static_cast<Index*>(context->data);
        if ((index->id == layer->layerId()) && (index->version == layer->layerVersion())) {
            archive(terse::transparent(*layer));
        } else {
            archive(terse::transparent(static_cast<Super&>(*this)));
        }
    }

    void updateIndexTable(IndexTable& indexTable, FileVersion fileVersion) {
        const bool exists = (indexTable.get(layer->layerId(), layer->layerVersion()) != nullptr);
        if (layer->layerSupported(fileVersion)) {
            // Do not merge ifs, as the below else branch depends solely on the condition of the
            // layer not being supported in the file version
            if (!exists && (this->upgradePolicy == UpgradeFormatPolicy::Allowed)) {
                indexTable.create(layer->layerId(), layer->layerVersion());
            }
        } else {
            if (exists && (this->unknownPolicy == UnknownLayerPolicy::Ignore)) {
                indexTable.destroy(layer->layerId(), layer->layerVersion());
            }
        }
        Super::updateIndexTable(indexTable, fileVersion);
    }
};

template<typename TFrom, typename TTo = TFrom>
struct RawSurjectiveMapping : public SurjectiveMapping<TFrom, TTo> {
    using SurjectiveMapping<TFrom, TTo>::SurjectiveMapping;

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("from");
        archive(this->from);
        archive.label("to");
        archive(this->to);
    }
};

struct RawCoordinateSystem {
    std::uint16_t xAxis;
    std::uint16_t yAxis;
    std::uint16_t zAxis;

    bool operator==(const CoordinateSystem& coordinateSystem) const {
        return (static_cast<std::uint16_t>(coordinateSystem.x) == xAxis &&
                static_cast<std::uint16_t>(coordinateSystem.y) == yAxis &&
                static_cast<std::uint16_t>(coordinateSystem.z) == zAxis);
    }

    bool operator!=(const CoordinateSystem& coordinateSystem) const {
        return !(*this == coordinateSystem);
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("xAxis");
        archive(xAxis);
        archive.label("yAxis");
        archive(yAxis);
        archive.label("zAxis");
        archive(zAxis);
    }
};

struct RawLODMapping : public LODMapping {
    using LODMapping::LODMapping;

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("lods");
        archive(lods);
        archive.label("indices");
        archive(indices);
    }
};

struct RawDescriptor {
    using StringPair = std::tuple<String<char>, String<char>>;

    String<char> name;
    std::uint16_t archetype;
    std::uint16_t gender;
    std::uint16_t age;
    Vector<StringPair> metadata;
    std::uint16_t translationUnit;
    std::uint16_t rotationUnit;
    RawCoordinateSystem coordinateSystem;
    std::uint16_t lodCount;
    std::uint16_t maxLOD;
    String<char> complexity;
    String<char> dbName;

    explicit RawDescriptor(MemoryResource* memRes) :
        name{memRes},
        archetype{},
        gender{},
        age{},
        metadata{memRes},
        translationUnit{},
        rotationUnit{},
        coordinateSystem{},
        lodCount{},
        maxLOD{},
        complexity{memRes},
        dbName{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("name");
        archive(name);
        archive.label("archetype");
        archive(archetype);
        archive.label("gender");
        archive(gender);
        archive.label("age");
        archive(age);
        archive.label("metadata");
        archive(metadata);
        archive.label("translationUnit");
        archive(translationUnit);
        archive.label("rotationUnit");
        archive(rotationUnit);
        archive.label("coordinateSystem");
        archive(coordinateSystem);
        archive.label("lodCount");
        archive(lodCount);
        archive.label("maxLOD");
        archive(maxLOD);
        archive.label("complexity");
        archive(complexity);
        archive.label("dbName");
        archive(dbName);
    }
};

struct RawDescriptorExt {
    RotationSequence rotationSequence;
    RotationSign rotationSign;
    FaceWindingOrder faceWindingOrder;

    explicit RawDescriptorExt(MemoryResource* memRes) :
        rotationSequence{RotationSequence::xyz},
        rotationSign{RotationDirection::positive, RotationDirection::positive, RotationDirection::positive},
        faceWindingOrder{FaceWindingOrder::ccw} {
        UNUSED(memRes);
    }

    template<class Archive>
    void serialize(Archive& archive, terse::Version<FileVersion::v21> /*unused*/) {
        archive.label("rotationSequence");
        archive(rotationSequence);
        archive.label("rotationSign");
        archive(rotationSign);
    }

    template<class Archive>
    void serialize(Archive& archive, terse::Version<FileVersion::v22> /*unused*/) {
        serialize(archive, terse::Version<FileVersion::v21>{});
    }

    template<class Archive>
    void serialize(Archive& archive, terse::Version<FileVersion::v23> /*unused*/) {
        serialize(archive, terse::Version<FileVersion::v21>{});
    }

    template<class Archive>
    void serialize(Archive& archive, terse::Version<FileVersion::v24> /*unused*/) {
        serialize(archive, terse::Version<FileVersion::v21>{});
    }

    template<class Archive>
    void serialize(Archive& archive, terse::Version<FileVersion::v25> /*unused*/) {
        serialize(archive, terse::Version<FileVersion::v21>{});
    }

    template<class Archive>
    void serialize(Archive& archive, terse::Version<FileVersion::v26> /*unused*/) {
        serialize(archive, terse::Version<FileVersion::v21>{});
    }

    template<class Archive>
    void serialize(Archive& archive, terse::Version<FileVersion::v27> /*unused*/) {
        serialize(archive, terse::Version<FileVersion::v21>{});
    }

    template<class Archive, typename AnyVersion>
    void serialize(Archive& archive, AnyVersion /*unused*/) {
        archive.label("rotationSequence");
        archive(rotationSequence);
        archive.label("rotationSign");
        archive(rotationSign);
        archive.label("faceWindingOrder");
        archive(faceWindingOrder);
    }
};

struct RawVector3Vector {
    AlignedDynArray<float> xs;
    AlignedDynArray<float> ys;
    AlignedDynArray<float> zs;

    explicit RawVector3Vector(MemoryResource* memRes) :
        xs{memRes},
        ys{memRes},
        zs{memRes} {
    }

    RawVector3Vector(std::size_t size_, float initial, MemoryResource* memRes) :
        xs{size_, initial, memRes},
        ys{size_, initial, memRes},
        zs{size_, initial, memRes} {
    }

    RawVector3Vector(ConstArrayView<float> xs_, ConstArrayView<float> ys_, ConstArrayView<float> zs_, MemoryResource* memRes) :
        xs{xs_.begin(), xs_.end(), memRes},
        ys{ys_.begin(), ys_.end(), memRes},
        zs{zs_.begin(), zs_.end(), memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("xs");
        archive(xs);
        archive.label("ys");
        archive(ys);
        archive.label("zs");
        archive(zs);
    }

    std::size_t size() const {
        assert(xs.size() == ys.size() && ys.size() == zs.size());
        return xs.size();
    }

    void reserve(std::size_t count) {
        xs.resize_uninitialized(count);
        ys.resize_uninitialized(count);
        zs.resize_uninitialized(count);
    }

    void resize(std::size_t count) {
        xs.resize(count);
        ys.resize(count);
        zs.resize(count);
    }

    void resize(std::size_t count, float value) {
        xs.resize(count, value);
        ys.resize(count, value);
        zs.resize(count, value);
    }

    void clear() {
        xs.clear();
        ys.clear();
        zs.clear();
    }

    template<typename Iterator>
    void assign(Iterator start, Iterator end) {
        reserve(static_cast<std::size_t>(std::distance(start, end)));
        std::size_t i{};
        for (auto it = start; it != end; ++it, ++i) {
            xs[i] = it->x;
            ys[i] = it->y;
            zs[i] = it->z;
        }
    }
};

struct RawDefinition {
    RawLODMapping lodJointMapping;
    RawLODMapping lodBlendShapeMapping;
    RawLODMapping lodAnimatedMapMapping;
    RawLODMapping lodMeshMapping;
    Vector<String<char>> guiControlNames;
    Vector<String<char>> rawControlNames;
    Vector<String<char>> jointNames;
    Vector<String<char>> blendShapeChannelNames;
    Vector<String<char>> animatedMapNames;
    Vector<String<char>> meshNames;
    RawSurjectiveMapping<std::uint16_t> meshBlendShapeChannelMapping;
    DynArray<std::uint16_t> jointHierarchy;
    RawVector3Vector neutralJointTranslations;
    RawVector3Vector neutralJointRotations;

    explicit RawDefinition(MemoryResource* memRes) :
        lodJointMapping{memRes},
        lodBlendShapeMapping{memRes},
        lodAnimatedMapMapping{memRes},
        lodMeshMapping{memRes},
        guiControlNames{memRes},
        rawControlNames{memRes},
        jointNames{memRes},
        blendShapeChannelNames{memRes},
        animatedMapNames{memRes},
        meshNames{memRes},
        meshBlendShapeChannelMapping{memRes},
        jointHierarchy{memRes},
        neutralJointTranslations{memRes},
        neutralJointRotations{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("lodJointMapping");
        archive(lodJointMapping);
        archive.label("lodBlendShapeMapping");
        archive(lodBlendShapeMapping);
        archive.label("lodAnimatedMapMapping");
        archive(lodAnimatedMapMapping);
        archive.label("lodMeshMapping");
        archive(lodMeshMapping);
        archive.label("guiControlNames");
        archive(guiControlNames);
        archive.label("rawControlNames");
        archive(rawControlNames);
        archive.label("jointNames");
        archive(jointNames);
        archive.label("blendShapeChannelNames");
        archive(blendShapeChannelNames);
        archive.label("animatedMapNames");
        archive(animatedMapNames);
        archive.label("meshNames");
        archive(meshNames);
        archive.label("meshBlendShapeChannelMapping");
        archive(meshBlendShapeChannelMapping);
        archive.label("jointHierarchy");
        archive(jointHierarchy);
        archive.label("neutralJointTranslations");
        archive(neutralJointTranslations);
        archive.label("neutralJointRotations");
        archive(neutralJointRotations);
    }
};

struct RawConditionalTable {
    DynArray<std::uint16_t> inputIndices;
    DynArray<std::uint16_t> outputIndices;
    DynArray<float> fromValues;
    DynArray<float> toValues;
    DynArray<float> slopeValues;
    DynArray<float> cutValues;

    explicit RawConditionalTable(MemoryResource* memRes) :
        inputIndices{memRes},
        outputIndices{memRes},
        fromValues{memRes},
        toValues{memRes},
        slopeValues{memRes},
        cutValues{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("inputIndices");
        archive(inputIndices);
        archive.label("outputIndices");
        archive(outputIndices);
        archive.label("fromValues");
        archive(fromValues);
        archive.label("toValues");
        archive(toValues);
        archive.label("slopeValues");
        archive(slopeValues);
        archive.label("cutValues");
        archive(cutValues);
    }
};

struct RawPSDMatrix {
    DynArray<std::uint16_t> rows;
    DynArray<std::uint16_t> columns;
    DynArray<float> values;

    explicit RawPSDMatrix(MemoryResource* memRes) :
        rows{memRes},
        columns{memRes},
        values{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("rows");
        archive(rows);
        archive.label("columns");
        archive(columns);
        archive.label("values");
        archive(values);
    }
};

struct RawControls {
    std::uint16_t psdCount;
    RawConditionalTable conditionals;
    RawPSDMatrix psds;

    explicit RawControls(MemoryResource* memRes) :
        psdCount{},
        conditionals{memRes},
        psds{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("psdCount");
        archive(psdCount);
        archive.label("conditionals");
        archive(conditionals);
        archive.label("psds");
        archive(psds);
    }
};

struct RawJointGroup {
    // Row count of each LOD
    // 12, 9, 3,
    // |  |  + LOD-2 contains first 3 rows
    // |  + LOD-1 contains first 9 rows
    // + LOD-0 contains first 12 rows
    DynArray<std::uint16_t> lods;
    // Sub-matrix col -> input vector
    DynArray<std::uint16_t> inputIndices;
    // Sub-matrix row -> output vector
    DynArray<std::uint16_t> outputIndices;
    // Non-zero values of all sub-matrices
    AlignedDynArray<float> values;

    DynArray<std::uint16_t> jointIndices;

    explicit RawJointGroup(MemoryResource* memRes) :
        lods{memRes},
        inputIndices{memRes},
        outputIndices{memRes},
        values{memRes},
        jointIndices{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("lods");
        archive(lods);
        archive.label("inputIndices");
        archive(inputIndices);
        archive.label("outputIndices");
        archive(outputIndices);
        archive.label("values");
        archive(values);
        archive.label("jointIndices");
        archive(jointIndices);
    }
};

struct RawJoints {
    std::uint16_t rowCount;
    std::uint16_t colCount;
    Vector<RawJointGroup> jointGroups;

    explicit RawJoints(MemoryResource* memRes) :
        rowCount{},
        colCount{},
        jointGroups{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("rowCount");
        archive(rowCount);
        archive.label("colCount");
        archive(colCount);
        archive.label("jointGroups");
        archive(jointGroups);
    }
};

struct RawBlendShapeChannels {
    DynArray<std::uint16_t> lods;
    DynArray<std::uint16_t> inputIndices;
    DynArray<std::uint16_t> outputIndices;

    explicit RawBlendShapeChannels(MemoryResource* memRes) :
        lods{memRes},
        inputIndices{memRes},
        outputIndices{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("lods");
        archive(lods);
        archive.label("inputIndices");
        archive(inputIndices);
        archive.label("outputIndices");
        archive(outputIndices);
    }
};

struct RawAnimatedMaps {
    DynArray<std::uint16_t> lods;
    RawConditionalTable conditionals;

    explicit RawAnimatedMaps(MemoryResource* memRes) :
        lods{memRes},
        conditionals{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("lods");
        archive(lods);
        archive.label("conditionals");
        archive(conditionals);
    }
};

struct RawBehavior {
    RawControls controls;
    RawJoints joints;
    RawBlendShapeChannels blendShapeChannels;
    RawAnimatedMaps animatedMaps;

    explicit RawBehavior(MemoryResource* memRes) :
        controls{memRes},
        joints{memRes},
        blendShapeChannels{memRes},
        animatedMaps{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive, terse::Version<FileVersion::v21> /*unused*/) {
        auto context = static_cast<SerializationContext*>(archive.getUserData());
        auto sections = static_cast<SectionLookupTable*>(context->data);

        archive(terse::proxy(sections->controls));
        archive.label("controls");
        archive(controls);

        archive(terse::proxy(sections->joints));
        archive.label("joints");
        archive(joints);

        archive(terse::proxy(sections->blendShapeChannels));
        archive.label("blendShapeChannels");
        archive(blendShapeChannels);

        archive(terse::proxy(sections->animatedMaps));
        archive.label("animatedMaps");
        archive(animatedMaps);
    }

    template<class Archive, typename AnyVersion>
    void serialize(Archive& archive, AnyVersion) {
        archive.label("controls");
        archive(controls);
        archive.label("joints");
        archive(joints);
        archive.label("blendShapeChannels");
        archive(blendShapeChannels);
        archive.label("animatedMaps");
        archive(animatedMaps);
    }
};

struct RawTextureCoordinateVector {
    DynArray<float> us;
    DynArray<float> vs;

    explicit RawTextureCoordinateVector(MemoryResource* memRes) :
        us{memRes},
        vs{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("us");
        archive(us);
        archive.label("vs");
        archive(vs);
    }

    std::size_t size() const {
        assert(us.size() == vs.size());
        return us.size();
    }

    void clear() {
        us.clear();
        vs.clear();
    }
};

struct RawVertexLayoutVector {
    DynArray<std::uint32_t> positions;
    DynArray<std::uint32_t> textureCoordinates;
    DynArray<std::uint32_t> normals;

    explicit RawVertexLayoutVector(MemoryResource* memRes) :
        positions{memRes},
        textureCoordinates{memRes},
        normals{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("positions");
        archive(positions);
        archive.label("textureCoordinates");
        archive(textureCoordinates);
        archive.label("normals");
        archive(normals);
    }

    std::size_t size() const {
        assert(positions.size() == textureCoordinates.size() && textureCoordinates.size() == normals.size());
        return positions.size();
    }

    void clear() {
        positions.clear();
        textureCoordinates.clear();
        normals.clear();
    }
};

struct RawFace {
    DynArray<std::uint32_t> layoutIndices;

    explicit RawFace(MemoryResource* memRes) :
        layoutIndices{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("layoutIndices");
        archive(layoutIndices);
    }
};

struct RawVertexSkinWeights {
    AlignedDynArray<float> weights;
    DynArray<std::uint16_t> jointIndices;

    explicit RawVertexSkinWeights(MemoryResource* memRes) :
        weights{memRes},
        jointIndices{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("weights");
        archive(weights);
        archive.label("jointIndices");
        archive(jointIndices);
    }
};

struct RawBlendShapeTarget {
    RawVector3Vector deltas;
    DynArray<std::uint32_t> vertexIndices;
    std::uint16_t blendShapeChannelIndex;

    explicit RawBlendShapeTarget(MemoryResource* memRes) :
        deltas{memRes},
        vertexIndices{memRes},
        blendShapeChannelIndex{} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("deltas");
        archive(deltas);
        archive.label("vertexIndices");
        archive(vertexIndices);
        archive.label("blendShapeChannelIndex");
        archive(blendShapeChannelIndex);
    }
};

struct RawMesh {
    terse::ArchiveOffset<std::uint32_t> offset;
    terse::ArchiveSize<std::uint32_t, std::uint32_t> size;
    terse::Anchor<std::uint32_t> baseMarker;
    RawVector3Vector positions;
    RawTextureCoordinateVector textureCoordinates;
    RawVector3Vector normals;
    RawVertexLayoutVector layouts;
    Vector<RawFace> faces;
    std::uint16_t maximumInfluencePerVertex;
    Vector<RawVertexSkinWeights> skinWeights;
    Vector<RawBlendShapeTarget> blendShapeTargets;
    terse::ArchiveOffset<std::uint32_t>::Proxy offsetMarker;
    terse::ArchiveSize<std::uint32_t, std::uint32_t>::Proxy sizeMarker;

    explicit RawMesh(MemoryResource* memRes) :
        offset{},
        size{},
        baseMarker{},
        positions{memRes},
        textureCoordinates{memRes},
        normals{memRes},
        layouts{memRes},
        faces{memRes},
        maximumInfluencePerVertex{},
        skinWeights{memRes},
        blendShapeTargets{memRes},
        offsetMarker{offset},
        sizeMarker{size, baseMarker} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        auto context = static_cast<SerializationContext*>(archive.getUserData());
        Dispatch()(archive, *this, context->version, true);
    }

    template<class Archive>
    void serialize(Archive& archive, terse::Version<FileVersion::v21> /*unused*/) {
        // Old format uses a hacky absolute file offset pointing to the next Mesh start
        archive.label("offset");
        archive(offset);

        archive.label("positions");
        archive(positions);
        archive.label("textureCoordinates");
        archive(textureCoordinates);
        archive.label("normals");
        archive(normals);
        archive.label("layouts");
        archive(layouts);
        archive.label("faces");
        archive(faces);
        archive.label("maximumInfluencePerVertex");
        archive(maximumInfluencePerVertex);
        archive.label("skinWeights");
        archive(skinWeights);
        archive.label("blendShapeTargets");
        archive(blendShapeTargets);

        archive(offsetMarker);
    }

    template<class Archive, typename AnyVersion>
    void serialize(Archive& archive, AnyVersion) {
        // The new format uses a size, denoting the size of the current mesh, thus allowing
        // the same skip mechanism, but in a position independent way
        archive.label("size");
        archive(size);
        archive(baseMarker);

        archive.label("positions");
        archive(positions);
        archive.label("textureCoordinates");
        archive(textureCoordinates);
        archive.label("normals");
        archive(normals);
        archive.label("layouts");
        archive(layouts);
        archive.label("faces");
        archive(faces);
        archive.label("maximumInfluencePerVertex");
        archive(maximumInfluencePerVertex);
        archive.label("skinWeights");
        archive(skinWeights);
        archive.label("blendShapeTargets");
        archive(blendShapeTargets);

        // Offset is not written into file
        archive(sizeMarker);
    }
};

struct RawGeometry {
    Vector<RawMesh> meshes;

    explicit RawGeometry(MemoryResource* memRes) :
        meshes{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("meshes");
        archive(meshes);
    }
};

struct RawActivationFunction {
    std::uint16_t functionId;
    DynArray<float> parameters;

    explicit RawActivationFunction(MemoryResource* memRes) :
        functionId{},
        parameters{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("functionId");
        archive(functionId);
        archive.label("parameters");
        archive(parameters);
    }
};

struct RawNeuralNetworkLayer {
    DynArray<float> biases;
    DynArray<float> weights;  // 2D Matrix
    RawActivationFunction activationFunction;

    explicit RawNeuralNetworkLayer(MemoryResource* memRes) :
        biases{memRes},
        weights{memRes},
        activationFunction{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("biases");
        archive(biases);
        archive.label("weights");
        archive(weights);
        archive.label("activationFunction");
        archive(activationFunction);
    }
};

struct RawNeuralNetwork {
    terse::ArchiveSize<std::uint32_t, std::uint32_t> size;
    terse::Anchor<std::uint32_t> baseMarker;
    DynArray<std::uint16_t> outputIndices;
    DynArray<std::uint16_t> inputIndices;
    Vector<RawNeuralNetworkLayer> layers;
    terse::ArchiveSize<std::uint32_t, std::uint32_t>::Proxy sizeMarker;

    explicit RawNeuralNetwork(MemoryResource* memRes) :
        size{},
        baseMarker{},
        outputIndices{memRes},
        inputIndices{memRes},
        layers{memRes},
        sizeMarker{size, baseMarker} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("size");
        archive(size);
        archive(baseMarker);
        archive.label("outputIndices");
        archive(outputIndices);
        archive.label("inputIndices");
        archive(inputIndices);
        archive.label("layers");
        archive(layers);
        archive(sizeMarker);
    }
};

struct RawMeshRegionMembership {
    Matrix<String<char>> regionNames;       // [meshIndex][regionIndex]
    Vector<Matrix<std::uint16_t>> indices;  // [meshIndex][regionIndex][neuralNetworkIndices...]

    explicit RawMeshRegionMembership(MemoryResource* memRes) :
        regionNames{memRes},
        indices{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("regionNames");
        archive(regionNames);
        archive.label("indices");
        archive(indices);
    }
};

struct RawMachineLearnedBehavior {
    Vector<String<char>> mlControlNames;
    RawLODMapping lodNeuralNetworkMapping;
    RawMeshRegionMembership neuralNetworkToMeshRegion;
    Vector<RawNeuralNetwork> neuralNetworks;

    explicit RawMachineLearnedBehavior(MemoryResource* memRes) :
        mlControlNames{memRes},
        lodNeuralNetworkMapping{memRes},
        neuralNetworkToMeshRegion{memRes},
        neuralNetworks{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("mlControlNames");
        archive(mlControlNames);
        archive.label("lodNeuralNetworkMapping");
        archive(lodNeuralNetworkMapping);
        archive.label("neuralNetworkToMeshRegion");
        archive(neuralNetworkToMeshRegion);
        archive.label("neuralNetworks");
        archive(neuralNetworks);
    }
};

struct RawRBFPose {
    String<char> name;
    float scale;

    explicit RawRBFPose(MemoryResource* memRes) :
        name{memRes},
        scale{1.0f} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("name");
        archive(name);
        archive.label("scale");
        archive(scale);
    }
};

struct RawRBFPoseExt {
    DynArray<std::uint16_t> inputControlIndices;
    DynArray<std::uint16_t> outputControlIndices;
    DynArray<float> outputControlWeights;

    explicit RawRBFPoseExt(MemoryResource* memRes) :
        inputControlIndices{memRes},
        outputControlIndices{memRes},
        outputControlWeights{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("inputControlIndices");
        archive(inputControlIndices);
        archive.label("outputControlIndices");
        archive(outputControlIndices);
        archive.label("outputControlWeights");
        archive(outputControlWeights);
    }
};

struct RawRBFSolver {
    terse::ArchiveSize<std::uint32_t, std::uint32_t> size;
    terse::Anchor<std::uint32_t> baseMarker;
    String<char> name;
    DynArray<std::uint16_t> rawControlIndices;
    DynArray<std::uint16_t> poseIndices;
    DynArray<float> rawControlValues;
    float radius;
    float weightThreshold;
    std::uint16_t solverType;
    std::uint16_t automaticRadius;
    std::uint16_t distanceMethod;
    std::uint16_t normalizeMethod;
    std::uint16_t functionType;
    std::uint16_t twistAxis;
    terse::ArchiveSize<std::uint32_t, std::uint32_t>::Proxy sizeMarker;

    explicit RawRBFSolver(MemoryResource* memRes) :
        size{},
        baseMarker{},
        name{memRes},
        rawControlIndices{memRes},
        poseIndices{memRes},
        rawControlValues{memRes},
        radius{},
        weightThreshold{},
        solverType{},
        automaticRadius{},
        distanceMethod{},
        normalizeMethod{},
        functionType{},
        twistAxis{},
        sizeMarker{size, baseMarker} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("size");
        archive(size);
        archive(baseMarker);
        archive.label("name");
        archive(name);
        archive.label("rawControlIndices");
        archive(rawControlIndices);
        archive.label("poseIndices");
        archive(poseIndices);
        archive.label("rawControlValues");
        archive(rawControlValues);
        archive.label("radius");
        archive(radius);
        archive.label("weightThreshold");
        archive(weightThreshold);
        archive.label("solverType");
        archive(solverType);
        archive.label("automaticRadius");
        archive(automaticRadius);
        archive.label("distanceMethod");
        archive(distanceMethod);
        archive.label("normalizeMethod");
        archive(normalizeMethod);
        archive.label("functionType");
        archive(functionType);
        archive.label("twistAxis");
        archive(twistAxis);
        archive(sizeMarker);
    }
};

struct RawRBFBehavior {
    RawLODMapping lodSolverMapping;
    Vector<RawRBFSolver> solvers;
    Vector<RawRBFPose> poses;

    explicit RawRBFBehavior(MemoryResource* memRes) :
        lodSolverMapping{memRes},
        solvers{memRes},
        poses{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("lodSolverMapping");
        archive(lodSolverMapping);
        archive.label("solvers");
        archive(solvers);
        archive.label("poses");
        archive(poses);
    }
};

struct RawRBFBehaviorExt {
    Vector<String<char>> poseControlNames;
    Vector<RawRBFPoseExt> poses;

    explicit RawRBFBehaviorExt(MemoryResource* memRes) :
        poseControlNames{memRes},
        poses{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("poseControlNames");
        archive(poseControlNames);
        archive.label("poses");
        archive(poses);
    }
};

struct RawJointRepresentation {
    std::uint16_t translation;
    std::uint16_t rotation;
    std::uint16_t scale;

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("translation");
        archive(translation);
        archive.label("rotation");
        archive(rotation);
        archive.label("scale");
        archive(scale);
    }
};

struct RawJointBehaviorMetadata {
    Vector<RawJointRepresentation> jointRepresentations;

    explicit RawJointBehaviorMetadata(MemoryResource* memRes) :
        jointRepresentations{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("jointRepresentations");
        archive(jointRepresentations);
    }
};

struct RawTwist {
    DynArray<float> twistBlendWeights;
    DynArray<std::uint16_t> twistOutputJointIndices;
    DynArray<std::uint16_t> twistInputControlIndices;
    std::uint16_t twistAxis;

    explicit RawTwist(MemoryResource* memRes) :
        twistBlendWeights{memRes},
        twistOutputJointIndices{memRes},
        twistInputControlIndices{memRes},
        twistAxis{} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("twistBlendWeights");
        archive(twistBlendWeights);
        archive.label("twistOutputJointIndices");
        archive(twistOutputJointIndices);
        archive.label("twistInputControlIndices");
        archive(twistInputControlIndices);
        archive.label("twistAxis");
        archive(twistAxis);
    }
};

struct RawSwing {
    DynArray<float> swingBlendWeights;
    DynArray<std::uint16_t> swingOutputJointIndices;
    DynArray<std::uint16_t> swingInputControlIndices;
    std::uint16_t twistAxis;

    explicit RawSwing(MemoryResource* memRes) :
        swingBlendWeights{memRes},
        swingOutputJointIndices{memRes},
        swingInputControlIndices{memRes},
        twistAxis{} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("swingBlendWeights");
        archive(swingBlendWeights);
        archive.label("swingOutputJointIndices");
        archive(swingOutputJointIndices);
        archive.label("swingInputControlIndices");
        archive(swingInputControlIndices);
        archive.label("twistAxis");
        archive(twistAxis);
    }
};

struct RawTwistSwingBehavior {
    Vector<RawTwist> twists;
    Vector<RawSwing> swings;

    explicit RawTwistSwingBehavior(MemoryResource* memRes) :
        twists{memRes},
        swings{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("twists");
        archive(twists);
        archive.label("swings");
        archive(swings);
    }
};

struct RawMLOperation {
    DynArray<std::uint32_t> parameters;
    DynArray<std::uint16_t> mlOperationSetDependencyIndices;
    DynArray<std::uint16_t> mlOperationDependencyIndices;
    std::uint16_t operationType;

    explicit RawMLOperation(MemoryResource* memRes) :
        parameters{memRes},
        mlOperationSetDependencyIndices{memRes},
        mlOperationDependencyIndices{memRes},
        operationType{} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("parameters");
        archive(parameters);
        archive.label("mlOperationSetDependencyIndices");
        archive(mlOperationSetDependencyIndices);
        archive.label("mlOperationDependencyIndices");
        archive(mlOperationDependencyIndices);
        archive.label("operationType");
        archive(operationType);
    }
};

struct RawMachineLearnedBehaviorTypeData {
    Vector<RawLODMapping> lodMLOperationMappings;
    Matrix<RawMLOperation> operations;

    explicit RawMachineLearnedBehaviorTypeData(MemoryResource* memRes) :
        lodMLOperationMappings{memRes},
        operations{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("lodMLOperationMappings");
        archive(lodMLOperationMappings);
        archive.label("operations");
        archive(operations);
    }
};

struct RawMachineLearnedBehaviorJoints {
    DynArray<std::uint16_t> parameterKeys;
    DynArray<std::uint16_t> parameterValues;
    DynArray<std::uint16_t> inputIndices;
    DynArray<std::uint16_t> outputIndices;

    explicit RawMachineLearnedBehaviorJoints(MemoryResource* memRes) :
        parameterKeys{memRes},
        parameterValues{memRes},
        inputIndices{memRes},
        outputIndices{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("parameterKeys");
        archive(parameterKeys);
        archive.label("parameterValues");
        archive(parameterValues);
        archive.label("inputIndices");
        archive(inputIndices);
        archive.label("outputIndices");
        archive(outputIndices);
    }
};

struct RawMachineLearnedBehaviorExt {
    Vector<RawMachineLearnedBehaviorTypeData> mlbTypeData;
    RawMachineLearnedBehaviorJoints mlbJoints;

    explicit RawMachineLearnedBehaviorExt(MemoryResource* memRes) :
        mlbTypeData{memRes},
        mlbJoints{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("mlbTypeData");
        archive(mlbTypeData);
        archive.label("mlbJoints");
        archive(mlbJoints);
    }
};

struct DNA {
    MemoryResource* memRes;
    Signature<3> signature{{'D', 'N', 'A'}};
    Version version{FileVersion::latest};
    IndexTable indexTable;
    Layer<sid4("desc"), rev(1, 1), RawDescriptor, FileVersion::v22> descriptor;
    Layer<sid4("defn"), rev(1, 1), RawDefinition, FileVersion::v22> definition;
    Layer<sid4("bhvr"), rev(1, 1), RawBehavior, FileVersion::v22> behavior;
    Layer<sid4("geom"), rev(1, 1), RawGeometry, FileVersion::v22> geometry;
    Layer<sid4("mlbh"), rev(1, 0), RawMachineLearnedBehavior, FileVersion::v23> machineLearnedBehavior;
    Layer<sid4("rbfb"), rev(1, 0), RawRBFBehavior, FileVersion::v24> rbfBehavior;
    Layer<sid4("rbfe"), rev(1, 0), RawRBFBehaviorExt, FileVersion::v25> rbfBehaviorExt;
    Layer<sid4("jbmd"), rev(1, 0), RawJointBehaviorMetadata, FileVersion::v24> jointBehaviorMetadata;
    Layer<sid4("twsw"), rev(1, 0), RawTwistSwingBehavior, FileVersion::v24> twistSwingBehavior;
    Layer<sid4("mlbe"), rev(1, 0), RawMachineLearnedBehaviorExt, FileVersion::v26> machineLearnedBehaviorExt;
    Layer<sid4("dsce"), rev(1, 0), RawDescriptorExt, FileVersion::v27> descriptorExt;

    using Layers = LayerContainer<decltype(descriptor),
                                  decltype(definition),
                                  decltype(behavior),
                                  decltype(geometry),
                                  decltype(machineLearnedBehavior),
                                  decltype(rbfBehavior),
                                  decltype(rbfBehaviorExt),
                                  decltype(jointBehaviorMetadata),
                                  decltype(twistSwingBehavior),
                                  decltype(machineLearnedBehaviorExt),
                                  decltype(descriptorExt)>;
    Layers layers;

    DNA(UnknownLayerPolicy unknownPolicy, UpgradeFormatPolicy upgradePolicy, MemoryResource* memRes_) :
        memRes{memRes_},
        indexTable{memRes_},
        descriptor{memRes},
        definition{memRes},
        behavior{memRes},
        geometry{memRes},
        machineLearnedBehavior{memRes},
        rbfBehavior{memRes},
        rbfBehaviorExt{memRes},
        jointBehaviorMetadata{memRes},
        twistSwingBehavior{memRes},
        machineLearnedBehaviorExt{memRes},
        descriptorExt{memRes},
        layers{unknownPolicy,
               upgradePolicy,
               memRes,
               &descriptor,
               &definition,
               &behavior,
               &geometry,
               &machineLearnedBehavior,
               &rbfBehavior,
               &rbfBehaviorExt,
               &jointBehaviorMetadata,
               &twistSwingBehavior,
               &machineLearnedBehaviorExt,
               &descriptorExt} {
    }

    explicit DNA(MemoryResource* memRes_) :
        DNA{UnknownLayerPolicy::Preserve, UpgradeFormatPolicy::Allowed, memRes_} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("signature");
        archive(signature);
        archive.label("version");
        archive(version);

        if (!signature.matches()) {
            return;
        }

        SerializationContext context{version, nullptr};
        auto oldUserData = archive.getUserData();
        archive.setUserData(&context);
        Dispatch()(archive, *this, version, true);
        archive.setUserData(oldUserData);
    }

    template<class Archive>
    void serialize(Archive& archive, terse::Version<FileVersion::v21> v21) {
        auto context = static_cast<SerializationContext*>(archive.getUserData());

        SectionLookupTable sections;
        context->data = &sections;

        archive.label("sections");
        archive(sections);

        archive.label("descriptor");
        archive(terse::proxy(sections.descriptor));
        archive(terse::versioned(static_cast<RawDescriptor&>(descriptor), v21));
        archive.label("definition");
        archive(terse::proxy(sections.definition));
        archive(terse::versioned(static_cast<RawDefinition&>(definition), v21));
        archive.label("behavior");
        archive(terse::proxy(sections.behavior));
        archive(terse::versioned(static_cast<RawBehavior&>(behavior), v21));
        archive.label("geometry");
        archive(terse::proxy(sections.geometry));
        archive(terse::versioned(static_cast<RawGeometry&>(geometry), v21));

        Signature<3> eof{{'A', 'N', 'D'}};
        archive(eof);

        context->data = nullptr;
    }

    template<class Archive, typename AnyVersion>
    IfHigher<AnyVersion, FileVersion::v21> load(Archive& archive, AnyVersion) {
        auto context = static_cast<SerializationContext*>(archive.getUserData());

        archive.label("index");
        archive(indexTable);

        std::size_t indexCount = indexTable.entries.size();
        for (std::size_t i = {}; i < indexCount;) {
            auto& index = indexTable.entries[i];
            context->data = &index;
            archive(terse::transparent(layers));
            // Check if index was marked for deletion, and if so, get rid of it
            if (context->data == nullptr) {
                indexTable.destroy(index.id, index.version);
                --indexCount;
            } else {
                ++i;
            }
        }

        context->data = nullptr;

        postLoadActions();
    }

    template<class Archive, typename AnyVersion>
    IfHigher<AnyVersion, FileVersion::v21> save(Archive& archive, AnyVersion) {
        auto context = static_cast<SerializationContext*>(archive.getUserData());

        // Add layers to index that were not already there
        layers.updateIndexTable(indexTable, context->version);

        archive.label("index");
        archive(indexTable);

        for (auto& index : indexTable.entries) {
            context->data = &index;
            archive(terse::transparent(layers));
        }
    }

    void postLoadActions() {
        if (rbfBehaviorExt.poses.empty() && !rbfBehavior.poses.empty()) {
            rbfBehaviorExt.poses.resize(rbfBehavior.poses.size(), RawRBFPoseExt{memRes});
            rbfBehaviorExt.poseControlNames.resize(rbfBehavior.poses.size());
            const auto controlOffset =
                (definition.rawControlNames.size() + behavior.controls.psdCount + machineLearnedBehavior.mlControlNames.size());
            for (std::size_t pi = {}; pi < rbfBehaviorExt.poses.size(); ++pi) {
                rbfBehaviorExt.poseControlNames[pi] = rbfBehavior.poses[pi].name;
                rbfBehaviorExt.poses[pi].outputControlIndices.resize(1ul);
                rbfBehaviorExt.poses[pi].outputControlWeights.resize(1ul);
                rbfBehaviorExt.poses[pi].outputControlIndices[0] = static_cast<std::uint16_t>(controlOffset + pi);
                rbfBehaviorExt.poses[pi].outputControlWeights[0] = 1.0f;
            }
        }
    }

    void unloadDefinition() {
        static_cast<RawDefinition&>(definition) = RawDefinition{memRes};
    }

    void unloadBehavior() {
        static_cast<RawBehavior&>(behavior) = RawBehavior{memRes};
    }

    void unloadGeometry() {
        static_cast<RawGeometry&>(geometry) = RawGeometry{memRes};
    }

    void unloadMachineLearnedBehavior() {
        static_cast<RawMachineLearnedBehavior&>(machineLearnedBehavior) = RawMachineLearnedBehavior{memRes};
        static_cast<RawMachineLearnedBehaviorExt&>(machineLearnedBehaviorExt) = RawMachineLearnedBehaviorExt{memRes};
    }

    void unloadRBFBehavior() {
        static_cast<RawRBFBehavior&>(rbfBehavior) = RawRBFBehavior{memRes};
        static_cast<RawRBFBehaviorExt&>(rbfBehaviorExt) = RawRBFBehaviorExt{memRes};
    }

    void unloadJointBehaviorMetadata() {
        static_cast<RawJointBehaviorMetadata&>(jointBehaviorMetadata) = RawJointBehaviorMetadata{memRes};
    }

    void unloadTwistSwingBehavior() {
        static_cast<RawTwistSwingBehavior&>(twistSwingBehavior) = RawTwistSwingBehavior{memRes};
    }
};

}  // namespace gs4
