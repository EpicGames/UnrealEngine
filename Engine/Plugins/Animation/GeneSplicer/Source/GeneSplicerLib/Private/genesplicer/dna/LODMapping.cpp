// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/dna/LODMapping.h"

#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Extd.h"
#include "genesplicer/dna/LODConstraint.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cassert>
#include <cstddef>
#include <cstdint>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

LODMapping::LODMapping(MemoryResource* memRes_) :
    lods{memRes_},
    indices{memRes_} {
}

std::uint16_t LODMapping::getLODCount() const {
    return static_cast<std::uint16_t>(lods.size());
}

void LODMapping::resetIndices() {
    indices.clear();
}

void LODMapping::resetLODs() {
    lods.clear();
}

void LODMapping::reset() {
    lods.clear();
    indices.clear();
}

void LODMapping::setLODCount(std::uint16_t lodCount) {
    reset();
    lods.resize(lodCount);
    indices.resize(lodCount);
}

void LODMapping::discardLODs(const LODConstraint& lodConstraint) {
    lodConstraint.applyTo(lods);
    cleanupIndices();
}

void LODMapping::cleanupIndices() {
    for (std::size_t i = indices.size(); i > 0ul; --i) {
        const auto idx = (i - 1ul);
        if (std::find(lods.begin(), lods.end(), idx) == lods.end()) {
            indices.erase(extd::advanced(indices.begin(), idx));
            for (auto& l2i : lods) {
                if (l2i > idx) {
                    --l2i;
                }
            }
        }
    }
}

ConstArrayView<std::uint16_t> LODMapping::getIndices(std::uint16_t lod) const {
    if (lod >= lods.size()) {
        return {};
    }
    assert(lods[lod] < indices.size());
    const auto it = extd::advanced(indices.cbegin(), lods[lod]);
    return (it == indices.cend() ? ConstArrayView<std::uint16_t>{} : ConstArrayView<std::uint16_t>{it->data(), it->size()});
}

std::uint16_t LODMapping::getIndexListCount() const {
    return static_cast<std::uint16_t>(indices.size());
}

void LODMapping::clearIndices(std::uint16_t index) {
    if (index < indices.size()) {
        indices[index].clear();
    } else {
        indices.resize(index + 1ul);
    }
}

void LODMapping::addIndices(std::uint16_t index, const std::uint16_t* source, std::uint16_t count) {
    if (index >= indices.size()) {
        indices.resize(index + 1ul);
    }
    indices[index].reserve(indices[index].size() + count);
    indices[index].insert(indices[index].end(), source, source + count);
}

void LODMapping::mapIndices(std::function<std::uint16_t(std::uint16_t)> mapper) {
    for (auto& row : indices) {
        for (auto& value : row) {
            value = mapper(value);
        }
    }
}

void LODMapping::filterIndices(std::function<bool(std::uint16_t)> filterer) {
    for (auto& row : indices) {
        for (auto it = row.begin(); it != row.end();) {
            if (filterer(*it)) {
                ++it;
            } else {
                it = row.erase(it);
            }
        }
    }
}

void LODMapping::sortIndices() {
    for (auto& row : indices) {
        std::sort(row.begin(), row.end());
    }
}

void LODMapping::associateLODWithIndices(std::uint16_t lod, std::uint16_t index) {
    if (lod >= lods.size()) {
        lods.resize(lod + 1ul);
    }
    if (index >= indices.size()) {
        indices.resize(index + 1ul);
    }
    lods[lod] = index;
}

UnorderedSet<std::uint16_t> LODMapping::getCombinedDistinctIndices(MemoryResource* memRes) const {
    UnorderedSet<std::uint16_t> distinctIndices{memRes};
    for (const auto& row : indices) {
        distinctIndices.insert(row.begin(), row.end());
    }
    return distinctIndices;
}

}  // namespace gs4
