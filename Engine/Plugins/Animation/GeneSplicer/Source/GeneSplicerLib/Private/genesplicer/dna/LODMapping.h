// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <functional>
#include <iterator>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

class LODConstraint;

class LODMapping {
public:
    explicit LODMapping(MemoryResource* memRes_);

    std::uint16_t getLODCount() const;
    void resetIndices();
    void resetLODs();
    void reset();
    void setLODCount(std::uint16_t lodCount);
    void discardLODs(const LODConstraint& lodConstraint);
    ConstArrayView<std::uint16_t> getIndices(std::uint16_t lod) const;
    std::uint16_t getIndexListCount() const;
    void clearIndices(std::uint16_t index);
    void addIndices(std::uint16_t index, const std::uint16_t* source, std::uint16_t count);

    template<class TIterator>
    void addIndices(std::uint16_t index, TIterator begin, TIterator end) {
        if (index >= indices.size()) {
            indices.resize(index + 1ul);
        }
        indices[index].reserve(indices[index].size() + static_cast<std::size_t>(std::distance(begin, end)));
        indices[index].insert(indices[index].end(), begin, end);
    }

    void associateLODWithIndices(std::uint16_t lod, std::uint16_t index);
    void mapIndices(std::function<std::uint16_t(std::uint16_t)> mapper);
    void filterIndices(std::function<bool(std::uint16_t)> filterer);
    void sortIndices();
    UnorderedSet<std::uint16_t> getCombinedDistinctIndices(MemoryResource* memRes) const;

private:
    void cleanupIndices();

protected:
    // Map indices to rows of the below defined matrix, e.g.:
    // lods: [0, 0, 1, 1]
    // indices: {[10, 15, 12], [9, 7, 43, 67]}
    // expands into:
    // 0: [10, 15, 12]
    // 1: [10, 15, 12]
    // 2: [9, 7, 43, 67]
    // 3: [9, 7, 43, 67]
    Vector<std::uint16_t> lods;
    Matrix<std::uint16_t> indices;
};

}  // namespace gs4
