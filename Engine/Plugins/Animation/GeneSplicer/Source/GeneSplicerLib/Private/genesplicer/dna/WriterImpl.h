// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Aliases.h"
#include "genesplicer/dna/BaseImpl.h"
#include "genesplicer/dna/Extd.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cstddef>
#include <cstring>
#include <tuple>
#include <type_traits>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

template<class TContainer>
typename std::enable_if<std::is_constructible<typename TContainer::value_type, MemoryResource*>::value>::type
ensureHasSize(TContainer& target, std::size_t size) {
    target.reserve(size);
    while (target.size() < size) {
        target.push_back(typename TContainer::value_type(target.get_allocator().getMemoryResource()));
    }
}

template<class TContainer>
typename std::enable_if<!std::is_constructible<typename TContainer::value_type, MemoryResource*>::value>::type
ensureHasSize(TContainer& target, std::size_t size) {
    if (target.size() < size) {
        target.resize(size);
    }
}

template<class TContainer, typename U>
typename std::enable_if<std::is_integral<U>::value, typename TContainer::value_type&>::type getAt(TContainer& target, U index) {
    ensureHasSize(target, index + 1ul);
    return target[index];
}

template<class TContainer, typename TSize, typename TValue>
typename std::enable_if<std::is_integral<TSize>::value>::type setAt(TContainer& target, TSize index, const TValue& value) {
    getAt(target, index) = value;
}

template<class TWriterBase>
class WriterImpl : public virtual TWriterBase, public virtual BaseImpl {
public:
    explicit WriterImpl(MemoryResource* memRes_);

    // HeaderWriter methods
    void setFileFormatGeneration(std::uint16_t generation) override;
    void setFileFormatVersion(std::uint16_t version) override;

    // DescriptorWriter methods
    void setName(const char* name) override;
    void setArchetype(Archetype archetype) override;
    void setGender(Gender gender) override;
    void setAge(std::uint16_t age) override;
    void clearMetaData() override;
    void setMetaData(const char* key, const char* value) override;
    void setTranslationUnit(TranslationUnit unit) override;
    void setRotationUnit(RotationUnit unit) override;
    void setCoordinateSystem(CoordinateSystem system) override;
    void setRotationSequence(RotationSequence rotationSequence) override;
    void setRotationSign(const RotationSign& rotationSign) override;
    void setFaceWindingOrder(FaceWindingOrder faceWindingOrder) override;
    void setLODCount(std::uint16_t lodCount) override;
    void setDBMaxLOD(std::uint16_t lod) override;
    void setDBComplexity(const char* name) override;
    void setDBName(const char* name) override;

    // DefinitionWriter methods
    void clearGUIControlNames() override;
    void setGUIControlName(std::uint16_t index, const char* name) override;
    void clearRawControlNames() override;
    void setRawControlName(std::uint16_t index, const char* name) override;
    void clearJointNames() override;
    void setJointName(std::uint16_t index, const char* name) override;
    void clearJointIndices() override;
    void setJointIndices(std::uint16_t index, const std::uint16_t* jointIndices, std::uint16_t count) override;
    void clearLODJointMappings() override;
    void setLODJointMapping(std::uint16_t lod, std::uint16_t index) override;
    void clearBlendShapeChannelNames() override;
    void setJointHierarchy(const std::uint16_t* jointIndices, std::uint16_t count) override;
    void setBlendShapeChannelName(std::uint16_t index, const char* name) override;
    void clearBlendShapeChannelIndices() override;
    void setBlendShapeChannelIndices(std::uint16_t index,
                                     const std::uint16_t* blendShapeChannelIndices,
                                     std::uint16_t count) override;
    void clearLODBlendShapeChannelMappings() override;
    void setLODBlendShapeChannelMapping(std::uint16_t lod, std::uint16_t index) override;
    void clearAnimatedMapNames() override;
    void setAnimatedMapName(std::uint16_t index, const char* name) override;
    void clearAnimatedMapIndices() override;
    void setAnimatedMapIndices(std::uint16_t index, const std::uint16_t* animatedMapIndices, std::uint16_t count) override;
    void clearLODAnimatedMapMappings() override;
    void setLODAnimatedMapMapping(std::uint16_t lod, std::uint16_t index) override;
    void clearMeshNames() override;
    void setMeshName(std::uint16_t index, const char* name) override;
    void clearMeshIndices() override;
    void setMeshIndices(std::uint16_t index, const std::uint16_t* meshIndices, std::uint16_t count) override;
    void clearLODMeshMappings() override;
    void setLODMeshMapping(std::uint16_t lod, std::uint16_t index) override;
    void clearMeshBlendShapeChannelMappings() override;
    void setMeshBlendShapeChannelMapping(std::uint32_t index,
                                         std::uint16_t meshIndex,
                                         std::uint16_t blendShapeChannelIndex) override;
    void setNeutralJointTranslations(const Vector3* translations, std::uint16_t count) override;
    void setNeutralJointRotations(const Vector3* rotations, std::uint16_t count) override;

    // BehaviorWriter methods
    void setGUIToRawInputIndices(const std::uint16_t* inputIndices, std::uint16_t count) override;
    void setGUIToRawOutputIndices(const std::uint16_t* outputIndices, std::uint16_t count) override;
    void setGUIToRawFromValues(const float* fromValues, std::uint16_t count) override;
    void setGUIToRawToValues(const float* toValues, std::uint16_t count) override;
    void setGUIToRawSlopeValues(const float* slopeValues, std::uint16_t count) override;
    void setGUIToRawCutValues(const float* cutValues, std::uint16_t count) override;
    void setPSDCount(std::uint16_t count) override;
    void setPSDRowIndices(const std::uint16_t* rowIndices, std::uint16_t count) override;
    void setPSDColumnIndices(const std::uint16_t* columnIndices, std::uint16_t count) override;
    void setPSDValues(const float* weights, std::uint16_t count) override;
    void setJointRowCount(std::uint16_t rowCount) override;
    void setJointColumnCount(std::uint16_t columnCount) override;
    void clearJointGroups() override;
    void deleteJointGroup(std::uint16_t jointGroupIndex) override;
    void setJointGroupLODs(std::uint16_t jointGroupIndex, const std::uint16_t* lods, std::uint16_t count) override;
    void setJointGroupInputIndices(std::uint16_t jointGroupIndex,
                                   const std::uint16_t* inputIndices,
                                   std::uint16_t count) override;
    void setJointGroupOutputIndices(std::uint16_t jointGroupIndex,
                                    const std::uint16_t* outputIndices,
                                    std::uint16_t count) override;
    void setJointGroupValues(std::uint16_t jointGroupIndex, const float* values, std::uint32_t count) override;
    void setJointGroupJointIndices(std::uint16_t jointGroupIndex,
                                   const std::uint16_t* jointIndices,
                                   std::uint16_t count) override;
    void setBlendShapeChannelLODs(const std::uint16_t* lods, std::uint16_t count) override;
    void setBlendShapeChannelInputIndices(const std::uint16_t* inputIndices, std::uint16_t count) override;
    void setBlendShapeChannelOutputIndices(const std::uint16_t* outputIndices, std::uint16_t count) override;
    void setAnimatedMapLODs(const std::uint16_t* lods, std::uint16_t count) override;
    void setAnimatedMapInputIndices(const std::uint16_t* inputIndices, std::uint16_t count) override;
    void setAnimatedMapOutputIndices(const std::uint16_t* outputIndices, std::uint16_t count) override;
    void setAnimatedMapFromValues(const float* fromValues, std::uint16_t count) override;
    void setAnimatedMapToValues(const float* toValues, std::uint16_t count) override;
    void setAnimatedMapSlopeValues(const float* slopeValues, std::uint16_t count) override;
    void setAnimatedMapCutValues(const float* cutValues, std::uint16_t count) override;

    // GeometryWriter methods
    void clearMeshes() override;
    void deleteMesh(std::uint16_t meshIndex) override;
    void setVertexPositions(std::uint16_t meshIndex, const Position* positions, std::uint32_t count) override;
    void setVertexTextureCoordinates(std::uint16_t meshIndex,
                                     const TextureCoordinate* textureCoordinates,
                                     std::uint32_t count) override;
    void setVertexNormals(std::uint16_t meshIndex, const Normal* normals, std::uint32_t count) override;
    void setVertexLayouts(std::uint16_t meshIndex, const VertexLayout* layouts, std::uint32_t count) override;
    void clearFaceVertexLayoutIndices(std::uint16_t meshIndex) override;
    void setFaceVertexLayoutIndices(std::uint16_t meshIndex,
                                    std::uint32_t faceIndex,
                                    const std::uint32_t* layoutIndices,
                                    std::uint32_t count) override;
    void setMaximumInfluencePerVertex(std::uint16_t meshIndex, std::uint16_t maxInfluenceCount) override;
    void clearSkinWeights(std::uint16_t meshIndex) override;
    void setSkinWeightsValues(std::uint16_t meshIndex,
                              std::uint32_t vertexIndex,
                              const float* weights,
                              std::uint16_t count) override;
    void setSkinWeightsJointIndices(std::uint16_t meshIndex,
                                    std::uint32_t vertexIndex,
                                    const std::uint16_t* jointIndices,
                                    std::uint16_t count) override;
    void clearBlendShapeTargets(std::uint16_t meshIndex) override;
    void setBlendShapeChannelIndex(std::uint16_t meshIndex,
                                   std::uint16_t blendShapeTargetIndex,
                                   std::uint16_t blendShapeChannelIndex) override;
    void setBlendShapeTargetDeltas(std::uint16_t meshIndex,
                                   std::uint16_t blendShapeTargetIndex,
                                   const Delta* deltas,
                                   std::uint32_t count) override;
    void setBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                          std::uint16_t blendShapeTargetIndex,
                                          const std::uint32_t* vertexIndices,
                                          std::uint32_t count) override;

    // MachineLearnedBehaviorWriter methods
    void clearMLControlNames() override;
    void setMLControlName(std::uint16_t index, const char* name) override;
    void clearNeuralNetworks() override;
    void clearNeuralNetworkIndices() override;
    void setNeuralNetworkIndices(std::uint16_t index, const std::uint16_t* netIndices, std::uint16_t count) override;
    void clearMeshRegionNames() override;
    void clearMeshRegionNames(std::uint16_t meshIndex) override;
    void setMeshRegionName(std::uint16_t meshIndex, std::uint16_t regionIndex, const char* name) override;
    void clearLODNeuralNetworkMappings() override;
    void setLODNeuralNetworkMapping(std::uint16_t lod, std::uint16_t index) override;
    void clearNeuralNetworkIndicesPerMeshRegion() override;
    void setNeuralNetworkIndicesForMeshRegion(std::uint16_t meshIndex,
                                              std::uint16_t regionIndex,
                                              const std::uint16_t* netIndices,
                                              std::uint16_t count) override;
    void deleteNeuralNetwork(std::uint16_t netIndex) override;
    void setNeuralNetworkInputIndices(std::uint16_t netIndex, const std::uint16_t* inputIndices, std::uint16_t count) override;
    void setNeuralNetworkOutputIndices(std::uint16_t netIndex, const std::uint16_t* outputIndices, std::uint16_t count) override;
    void clearNeuralNetworkLayers(std::uint16_t netIndex) override;
    void setNeuralNetworkLayerActivationFunction(std::uint16_t netIndex,
                                                 std::uint16_t layerIndex,
                                                 ActivationFunction function) override;
    void setNeuralNetworkLayerActivationFunctionParameters(std::uint16_t netIndex,
                                                           std::uint16_t layerIndex,
                                                           const float* activationFunctionParameters,
                                                           std::uint16_t count) override;
    void setNeuralNetworkLayerBiases(std::uint16_t netIndex,
                                     std::uint16_t layerIndex,
                                     const float* biases,
                                     std::uint32_t count) override;
    void setNeuralNetworkLayerWeights(std::uint16_t netIndex,
                                      std::uint16_t layerIndex,
                                      const float* weights,
                                      std::uint32_t count) override;

    // RBFBehaviorWriter methods
    void clearRBFPoses() override;
    void setRBFPoseName(std::uint16_t poseIndex, const char* name) override;
    void setRBFPoseScale(std::uint16_t poseIndex, float scale) override;
    void clearRBFPoseControlNames() override;
    void setRBFPoseControlName(std::uint16_t poseControlIndex, const char* name) override;
    void setRBFPoseInputControlIndices(std::uint16_t poseIndex,
                                       const std::uint16_t* controlIndices,
                                       std::uint16_t controlIndexCount) override;
    void setRBFPoseOutputControlIndices(std::uint16_t poseIndex,
                                        const std::uint16_t* controlIndices,
                                        std::uint16_t controlIndexCount) override;
    void setRBFPoseOutputControlWeights(std::uint16_t poseIndex,
                                        const float* controlWeights,
                                        std::uint16_t controlWeightCount) override;
    void clearRBFSolvers() override;
    void clearRBFSolverIndices() override;
    void setRBFSolverIndices(std::uint16_t index, const std::uint16_t* solverIndices, std::uint16_t count) override;
    void clearLODRBFSolverMappings() override;
    void setLODRBFSolverMapping(std::uint16_t lod, std::uint16_t index) override;
    void setRBFSolverName(std::uint16_t solverIndex, const char* name) override;
    void setRBFSolverRawControlIndices(std::uint16_t solverIndex,
                                       const std::uint16_t* inputIndices,
                                       std::uint16_t count) override;
    void setRBFSolverPoseIndices(std::uint16_t solverIndex, const std::uint16_t* poseIndices, std::uint16_t count) override;
    void setRBFSolverRawControlValues(std::uint16_t solverIndex, const float* values, std::uint16_t count) override;
    void setRBFSolverType(std::uint16_t solverIndex, RBFSolverType type) override;
    void setRBFSolverRadius(std::uint16_t solverIndex, float radius) override;
    void setRBFSolverAutomaticRadius(std::uint16_t solverIndex, AutomaticRadius automaticRadius) override;
    void setRBFSolverWeightThreshold(std::uint16_t solverIndex, float weightThreshold) override;
    void setRBFSolverDistanceMethod(std::uint16_t solverIndex, RBFDistanceMethod distanceMethod) override;
    void setRBFSolverNormalizeMethod(std::uint16_t solverIndex, RBFNormalizeMethod normalizeMethod) override;
    void setRBFSolverFunctionType(std::uint16_t solverIndex, RBFFunctionType functionType) override;
    void setRBFSolverTwistAxis(std::uint16_t solverIndex, TwistAxis twistAxis) override;

    // JointBehaviorMetadataWriter
    void clearJointRepresentations() override;
    void setJointTranslationRepresentation(std::uint16_t jointIndex, TranslationRepresentation representation) override;
    void setJointRotationRepresentation(std::uint16_t jointIndex, RotationRepresentation representation) override;
    void setJointScaleRepresentation(std::uint16_t jointIndex, ScaleRepresentation representation) override;

    // TwistSwingBehaviorWriter
    void clearTwists() override;
    void deleteTwist(std::uint16_t twistIndex) override;
    void setTwistSetupTwistAxis(std::uint16_t twistIndex, TwistAxis twistAxis) override;
    void setTwistInputControlIndices(std::uint16_t twistIndex,
                                     const std::uint16_t* controlIndices,
                                     std::uint16_t controlIndexCount) override;
    void setTwistOutputJointIndices(std::uint16_t twistIndex,
                                    const std::uint16_t* jointIndices,
                                    std::uint16_t jointIndexCount) override;
    void setTwistBlendWeights(std::uint16_t twistIndex, const float* blendWeights, std::uint16_t blendWeightCount) override;
    void clearSwings() override;
    void deleteSwing(std::uint16_t swingIndex) override;
    void setSwingSetupTwistAxis(std::uint16_t swingIndex, TwistAxis twistAxis) override;
    void setSwingInputControlIndices(std::uint16_t swingIndex,
                                     const std::uint16_t* controlIndices,
                                     std::uint16_t controlIndexCount) override;
    void setSwingOutputJointIndices(std::uint16_t swingIndex,
                                    const std::uint16_t* jointIndices,
                                    std::uint16_t jointIndexCount) override;
    void setSwingBlendWeights(std::uint16_t swingIndex, const float* blendWeights, std::uint16_t blendWeightCount) override;

    // MachineLearnedBehaviorExtWriter methods
    void clearMLExtData() override;
    void clearMLOperationSets(std::uint16_t mlTypeIndex) override;
    void clearMLOperations(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) override;
    void setMLOperationType(std::uint16_t mlTypeIndex,
                            std::uint16_t mlOperationSetIndex,
                            std::uint16_t mlOperationIndex,
                            MachineLearnedBehaviorOperationType operationType) override;
    void setMLOperationParameters(std::uint16_t mlTypeIndex,
                                  std::uint16_t mlOperationSetIndex,
                                  std::uint16_t mlOperationIndex,
                                  const std::uint32_t* parameters,
                                  std::uint16_t count) override;
    void setMLOperationDependencyOperationSetIndices(std::uint16_t mlTypeIndex,
                                                     std::uint16_t mlOperationSetIndex,
                                                     std::uint16_t mlOperationIndex,
                                                     const std::uint16_t* indices,
                                                     std::uint16_t count) override;
    void setMLOperationDependencyOperationIndices(std::uint16_t mlTypeIndex,
                                                  std::uint16_t mlOperationSetIndex,
                                                  std::uint16_t mlOperationIndex,
                                                  const std::uint16_t* indices,
                                                  std::uint16_t count) override;
    void clearMLOperationIndicesAndLODMappings() override;
    void clearMLOperationIndices(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) override;
    void clearLODMLOperationMappings(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) override;
    void setMLOperationIndices(std::uint16_t mlTypeIndex,
                               std::uint16_t mlOperationSetIndex,
                               std::uint16_t index,
                               const std::uint16_t* mlOperationIndices,
                               std::uint16_t count) override;
    void setLODMLOperationMapping(std::uint16_t mlTypeIndex,
                                  std::uint16_t mlOperationSetIndex,
                                  std::uint16_t lod,
                                  std::uint16_t index) override;
    void setMLJointsParameterKeys(const std::uint16_t* parameterKeys, std::uint16_t count) override;
    void setMLJointsParameterValues(const std::uint16_t* parameterValues, std::uint16_t count) override;
    void setMLJointsInputIndices(const std::uint16_t* inputIndices, std::uint16_t count) override;
    void setMLJointsOutputIndices(const std::uint16_t* outputIndices, std::uint16_t count) override;
};

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4589)
#endif
template<class TWriterBase>
WriterImpl<TWriterBase>::WriterImpl(MemoryResource* memRes_) :
    BaseImpl{memRes_} {
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4505)
#endif
template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setFileFormatGeneration(std::uint16_t generation) {
    dna.version.generation = generation;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setFileFormatVersion(std::uint16_t version) {
    dna.version.version = version;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setName(const char* name) {
    dna.descriptor.name = name;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setArchetype(Archetype archetype) {
    dna.descriptor.archetype = static_cast<std::uint16_t>(archetype);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setGender(Gender gender) {
    dna.descriptor.gender = static_cast<std::uint16_t>(gender);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setAge(std::uint16_t age) {
    dna.descriptor.age = age;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMetaData() {
    dna.descriptor.metadata.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMetaData(const char* key, const char* value) {
    using CharStringPair = std::tuple<String<char>, String<char>>;
    auto it = std::find_if(dna.descriptor.metadata.begin(), dna.descriptor.metadata.end(), [&key](const CharStringPair& kv) {
        auto& k = std::get<0>(kv);
        return (std::strlen(key) == k.size() && std::strncmp(k.data(), key, k.size()) == 0);
    });
    if (it == dna.descriptor.metadata.end()) {
        if (value != nullptr) {
            dna.descriptor.metadata.emplace_back(String<char>{key, memRes}, String<char>{value, memRes});
        }
    } else {
        if (value == nullptr) {
            dna.descriptor.metadata.erase(it);
        } else {
            std::get<1>(*it) = value;
        }
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setTranslationUnit(TranslationUnit unit) {
    dna.descriptor.translationUnit = static_cast<std::uint16_t>(unit);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRotationUnit(RotationUnit unit) {
    dna.descriptor.rotationUnit = static_cast<std::uint16_t>(unit);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setCoordinateSystem(CoordinateSystem system) {
    dna.descriptor.coordinateSystem.xAxis = static_cast<std::uint16_t>(system.x);
    dna.descriptor.coordinateSystem.yAxis = static_cast<std::uint16_t>(system.y);
    dna.descriptor.coordinateSystem.zAxis = static_cast<std::uint16_t>(system.z);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRotationSequence(RotationSequence rotationSequence) {
    dna.descriptorExt.rotationSequence = rotationSequence;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRotationSign(const RotationSign& rotationSign) {
    dna.descriptorExt.rotationSign = rotationSign;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setFaceWindingOrder(FaceWindingOrder faceWindingOrder) {
    dna.descriptorExt.faceWindingOrder = faceWindingOrder;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setLODCount(std::uint16_t lodCount) {
    dna.descriptor.lodCount = lodCount;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setDBMaxLOD(std::uint16_t lod) {
    dna.descriptor.maxLOD = lod;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setDBComplexity(const char* name) {
    dna.descriptor.complexity = name;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setDBName(const char* name) {
    dna.descriptor.dbName = name;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearGUIControlNames() {
    dna.definition.guiControlNames.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setGUIControlName(std::uint16_t index, const char* name) {
    setAt(dna.definition.guiControlNames, index, name);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearRawControlNames() {
    dna.definition.rawControlNames.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRawControlName(std::uint16_t index, const char* name) {
    setAt(dna.definition.rawControlNames, index, name);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearJointNames() {
    dna.definition.jointNames.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointName(std::uint16_t index, const char* name) {
    setAt(dna.definition.jointNames, index, name);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearJointIndices() {
    dna.definition.lodJointMapping.resetIndices();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointIndices(std::uint16_t index,
                                                     const std::uint16_t* jointIndices,
                                                     std::uint16_t count) {
    dna.definition.lodJointMapping.clearIndices(index);
    dna.definition.lodJointMapping.addIndices(index, jointIndices, count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearLODJointMappings() {
    dna.definition.lodJointMapping.resetLODs();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setLODJointMapping(std::uint16_t lod, std::uint16_t index) {
    dna.definition.lodJointMapping.associateLODWithIndices(lod, index);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointHierarchy(const std::uint16_t* jointIndices, std::uint16_t count) {
    dna.definition.jointHierarchy.assign(jointIndices, jointIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearBlendShapeChannelNames() {
    dna.definition.blendShapeChannelNames.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setBlendShapeChannelName(std::uint16_t index, const char* name) {
    setAt(dna.definition.blendShapeChannelNames, index, name);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearBlendShapeChannelIndices() {
    dna.definition.lodBlendShapeMapping.resetIndices();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setBlendShapeChannelIndices(std::uint16_t index,
                                                                 const std::uint16_t* blendShapeChannelIndices,
                                                                 std::uint16_t count) {
    dna.definition.lodBlendShapeMapping.clearIndices(index);
    dna.definition.lodBlendShapeMapping.addIndices(index, blendShapeChannelIndices, count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearLODBlendShapeChannelMappings() {
    dna.definition.lodBlendShapeMapping.resetLODs();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setLODBlendShapeChannelMapping(std::uint16_t lod, std::uint16_t index) {
    dna.definition.lodBlendShapeMapping.associateLODWithIndices(lod, index);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearAnimatedMapNames() {
    dna.definition.animatedMapNames.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setAnimatedMapName(std::uint16_t index, const char* name) {
    setAt(dna.definition.animatedMapNames, index, name);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearAnimatedMapIndices() {
    dna.definition.lodAnimatedMapMapping.resetIndices();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setAnimatedMapIndices(std::uint16_t index,
                                                           const std::uint16_t* animatedMapIndices,
                                                           std::uint16_t count) {
    dna.definition.lodAnimatedMapMapping.clearIndices(index);
    dna.definition.lodAnimatedMapMapping.addIndices(index, animatedMapIndices, count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearLODAnimatedMapMappings() {
    dna.definition.lodAnimatedMapMapping.resetLODs();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setLODAnimatedMapMapping(std::uint16_t lod, std::uint16_t index) {
    dna.definition.lodAnimatedMapMapping.associateLODWithIndices(lod, index);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMeshNames() {
    dna.definition.meshNames.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMeshName(std::uint16_t index, const char* name) {
    setAt(dna.definition.meshNames, index, name);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMeshIndices() {
    dna.definition.lodMeshMapping.resetIndices();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMeshIndices(std::uint16_t index, const std::uint16_t* meshIndices, std::uint16_t count) {
    dna.definition.lodMeshMapping.clearIndices(index);
    dna.definition.lodMeshMapping.addIndices(index, meshIndices, count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearLODMeshMappings() {
    dna.definition.lodMeshMapping.resetLODs();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setLODMeshMapping(std::uint16_t lod, std::uint16_t index) {
    dna.definition.lodMeshMapping.associateLODWithIndices(lod, index);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMeshBlendShapeChannelMappings() {
    dna.definition.meshBlendShapeChannelMapping.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMeshBlendShapeChannelMapping(std::uint32_t index,
                                                                     std::uint16_t meshIndex,
                                                                     std::uint16_t blendShapeChannelIndex) {
    dna.definition.meshBlendShapeChannelMapping.set(index, meshIndex, blendShapeChannelIndex);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setNeutralJointTranslations(const Vector3* translations, std::uint16_t count) {
    dna.definition.neutralJointTranslations.assign(translations, translations + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setNeutralJointRotations(const Vector3* rotations, std::uint16_t count) {
    dna.definition.neutralJointRotations.assign(rotations, rotations + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setGUIToRawInputIndices(const std::uint16_t* inputIndices, std::uint16_t count) {
    dna.behavior.controls.conditionals.inputIndices.assign(inputIndices, inputIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setGUIToRawOutputIndices(const std::uint16_t* outputIndices, std::uint16_t count) {
    dna.behavior.controls.conditionals.outputIndices.assign(outputIndices, outputIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setGUIToRawFromValues(const float* fromValues, std::uint16_t count) {
    dna.behavior.controls.conditionals.fromValues.assign(fromValues, fromValues + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setGUIToRawToValues(const float* toValues, std::uint16_t count) {
    dna.behavior.controls.conditionals.toValues.assign(toValues, toValues + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setGUIToRawSlopeValues(const float* slopeValues, std::uint16_t count) {
    dna.behavior.controls.conditionals.slopeValues.assign(slopeValues, slopeValues + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setGUIToRawCutValues(const float* cutValues, std::uint16_t count) {
    dna.behavior.controls.conditionals.cutValues.assign(cutValues, cutValues + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setPSDCount(std::uint16_t count) {
    dna.behavior.controls.psdCount = count;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setPSDRowIndices(const std::uint16_t* rowIndices, std::uint16_t count) {
    dna.behavior.controls.psds.rows.assign(rowIndices, rowIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setPSDColumnIndices(const std::uint16_t* columnIndices, std::uint16_t count) {
    dna.behavior.controls.psds.columns.assign(columnIndices, columnIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setPSDValues(const float* weights, std::uint16_t count) {
    dna.behavior.controls.psds.values.assign(weights, weights + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointRowCount(std::uint16_t rowCount) {
    dna.behavior.joints.rowCount = rowCount;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointColumnCount(std::uint16_t columnCount) {
    dna.behavior.joints.colCount = columnCount;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearJointGroups() {
    dna.behavior.joints.jointGroups.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::deleteJointGroup(std::uint16_t jointGroupIndex) {
    if (jointGroupIndex < dna.behavior.joints.jointGroups.size()) {
        auto it = extd::advanced(dna.behavior.joints.jointGroups.begin(), jointGroupIndex);
        dna.behavior.joints.jointGroups.erase(it);
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointGroupLODs(std::uint16_t jointGroupIndex,
                                                       const std::uint16_t* lods,
                                                       std::uint16_t count) {
    auto& jointGroup = getAt(dna.behavior.joints.jointGroups, jointGroupIndex);
    jointGroup.lods.assign(lods, lods + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointGroupInputIndices(std::uint16_t jointGroupIndex,
                                                               const std::uint16_t* inputIndices,
                                                               std::uint16_t count) {
    auto& jointGroup = getAt(dna.behavior.joints.jointGroups, jointGroupIndex);
    jointGroup.inputIndices.assign(inputIndices, inputIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointGroupOutputIndices(std::uint16_t jointGroupIndex,
                                                                const std::uint16_t* outputIndices,
                                                                std::uint16_t count) {
    auto& jointGroup = getAt(dna.behavior.joints.jointGroups, jointGroupIndex);
    jointGroup.outputIndices.assign(outputIndices, outputIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointGroupValues(std::uint16_t jointGroupIndex,
                                                         const float* values,
                                                         std::uint32_t count) {
    auto& jointGroup = getAt(dna.behavior.joints.jointGroups, jointGroupIndex);
    jointGroup.values.assign(values, values + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointGroupJointIndices(std::uint16_t jointGroupIndex,
                                                               const std::uint16_t* jointIndices,
                                                               std::uint16_t count) {
    auto& jointGroup = getAt(dna.behavior.joints.jointGroups, jointGroupIndex);
    jointGroup.jointIndices.assign(jointIndices, jointIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setBlendShapeChannelLODs(const std::uint16_t* lods, std::uint16_t count) {
    dna.behavior.blendShapeChannels.lods.assign(lods, lods + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setBlendShapeChannelInputIndices(const std::uint16_t* inputIndices, std::uint16_t count) {
    dna.behavior.blendShapeChannels.inputIndices.assign(inputIndices, inputIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setBlendShapeChannelOutputIndices(const std::uint16_t* outputIndices, std::uint16_t count) {
    dna.behavior.blendShapeChannels.outputIndices.assign(outputIndices, outputIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setAnimatedMapLODs(const std::uint16_t* lods, std::uint16_t count) {
    dna.behavior.animatedMaps.lods.assign(lods, lods + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setAnimatedMapInputIndices(const std::uint16_t* inputIndices, std::uint16_t count) {
    dna.behavior.animatedMaps.conditionals.inputIndices.assign(inputIndices, inputIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setAnimatedMapOutputIndices(const std::uint16_t* outputIndices, std::uint16_t count) {
    dna.behavior.animatedMaps.conditionals.outputIndices.assign(outputIndices, outputIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setAnimatedMapFromValues(const float* fromValues, std::uint16_t count) {
    dna.behavior.animatedMaps.conditionals.fromValues.assign(fromValues, fromValues + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setAnimatedMapToValues(const float* toValues, std::uint16_t count) {
    dna.behavior.animatedMaps.conditionals.toValues.assign(toValues, toValues + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setAnimatedMapSlopeValues(const float* slopeValues, std::uint16_t count) {
    dna.behavior.animatedMaps.conditionals.slopeValues.assign(slopeValues, slopeValues + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setAnimatedMapCutValues(const float* cutValues, std::uint16_t count) {
    dna.behavior.animatedMaps.conditionals.cutValues.assign(cutValues, cutValues + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMeshes() {
    dna.geometry.meshes.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::deleteMesh(std::uint16_t meshIndex) {
    if (meshIndex < dna.geometry.meshes.size()) {
        auto it = extd::advanced(dna.geometry.meshes.begin(), meshIndex);
        dna.geometry.meshes.erase(it);
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setVertexPositions(std::uint16_t meshIndex, const Position* positions, std::uint32_t count) {
    auto& mesh = getAt(dna.geometry.meshes, meshIndex);
    mesh.positions.assign(positions, positions + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setVertexTextureCoordinates(std::uint16_t meshIndex,
                                                                 const TextureCoordinate* textureCoordinates,
                                                                 std::uint32_t count) {
    auto& mesh = getAt(dna.geometry.meshes, meshIndex);
    auto& destination = mesh.textureCoordinates;
    destination.clear();
    destination.us.resize_uninitialized(count);
    destination.vs.resize_uninitialized(count);
    for (std::size_t i = 0ul; i < count; ++i) {
        destination.us[i] = textureCoordinates[i].u;
        destination.vs[i] = textureCoordinates[i].v;
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setVertexNormals(std::uint16_t meshIndex, const Normal* normals, std::uint32_t count) {
    auto& mesh = getAt(dna.geometry.meshes, meshIndex);
    mesh.normals.assign(normals, normals + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setVertexLayouts(std::uint16_t meshIndex, const VertexLayout* layouts, std::uint32_t count) {
    auto& mesh = getAt(dna.geometry.meshes, meshIndex);
    auto& destination = mesh.layouts;
    destination.clear();
    destination.positions.resize_uninitialized(count);
    destination.textureCoordinates.resize_uninitialized(count);
    destination.normals.resize_uninitialized(count);
    for (std::size_t i = 0ul; i < count; ++i) {
        destination.positions[i] = layouts[i].position;
        destination.textureCoordinates[i] = layouts[i].textureCoordinate;
        destination.normals[i] = layouts[i].normal;
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearFaceVertexLayoutIndices(std::uint16_t meshIndex) {
    if (meshIndex < dna.geometry.meshes.size()) {
        dna.geometry.meshes[meshIndex].faces.clear();
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setFaceVertexLayoutIndices(std::uint16_t meshIndex,
                                                                std::uint32_t faceIndex,
                                                                const std::uint32_t* layoutIndices,
                                                                std::uint32_t count) {
    auto& mesh = getAt(dna.geometry.meshes, meshIndex);
    auto& face = getAt(mesh.faces, faceIndex);
    face.layoutIndices.assign(layoutIndices, layoutIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMaximumInfluencePerVertex(std::uint16_t meshIndex, std::uint16_t maxInfluenceCount) {
    auto& mesh = getAt(dna.geometry.meshes, meshIndex);
    mesh.maximumInfluencePerVertex = maxInfluenceCount;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearSkinWeights(std::uint16_t meshIndex) {
    if (meshIndex < dna.geometry.meshes.size()) {
        dna.geometry.meshes[meshIndex].skinWeights.clear();
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setSkinWeightsValues(std::uint16_t meshIndex,
                                                          std::uint32_t vertexIndex,
                                                          const float* weights,
                                                          std::uint16_t count) {
    auto& mesh = getAt(dna.geometry.meshes, meshIndex);
    auto& vertexSkinWeights = getAt(mesh.skinWeights, vertexIndex);
    vertexSkinWeights.weights.assign(weights, weights + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setSkinWeightsJointIndices(std::uint16_t meshIndex,
                                                                std::uint32_t vertexIndex,
                                                                const std::uint16_t* jointIndices,
                                                                std::uint16_t count) {
    auto& mesh = getAt(dna.geometry.meshes, meshIndex);
    auto& vertexSkinWeights = getAt(mesh.skinWeights, vertexIndex);
    vertexSkinWeights.jointIndices.assign(jointIndices, jointIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearBlendShapeTargets(std::uint16_t meshIndex) {
    if (meshIndex < dna.geometry.meshes.size()) {
        dna.geometry.meshes[meshIndex].blendShapeTargets.clear();
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setBlendShapeChannelIndex(std::uint16_t meshIndex,
                                                               std::uint16_t blendShapeTargetIndex,
                                                               std::uint16_t blendShapeChannelIndex) {
    auto& mesh = getAt(dna.geometry.meshes, meshIndex);
    auto& blendShapeTarget = getAt(mesh.blendShapeTargets, blendShapeTargetIndex);
    blendShapeTarget.blendShapeChannelIndex = blendShapeChannelIndex;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setBlendShapeTargetDeltas(std::uint16_t meshIndex,
                                                               std::uint16_t blendShapeTargetIndex,
                                                               const Delta* deltas,
                                                               std::uint32_t count) {
    auto& mesh = getAt(dna.geometry.meshes, meshIndex);
    auto& blendShapeTarget = getAt(mesh.blendShapeTargets, blendShapeTargetIndex);
    blendShapeTarget.deltas.assign(deltas, deltas + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                                      std::uint16_t blendShapeTargetIndex,
                                                                      const std::uint32_t* vertexIndices,
                                                                      std::uint32_t count) {
    auto& mesh = getAt(dna.geometry.meshes, meshIndex);
    auto& blendShapeTarget = getAt(mesh.blendShapeTargets, blendShapeTargetIndex);
    blendShapeTarget.vertexIndices.assign(vertexIndices, vertexIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMLControlNames() {
    dna.machineLearnedBehavior.mlControlNames.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMLControlName(std::uint16_t index, const char* name) {
    setAt(dna.machineLearnedBehavior.mlControlNames, index, name);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearNeuralNetworks() {
    dna.machineLearnedBehavior.neuralNetworks.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearNeuralNetworkIndices() {
    dna.machineLearnedBehavior.lodNeuralNetworkMapping.resetIndices();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setNeuralNetworkIndices(std::uint16_t index,
                                                             const std::uint16_t* netIndices,
                                                             std::uint16_t count) {
    dna.machineLearnedBehavior.lodNeuralNetworkMapping.clearIndices(index);
    dna.machineLearnedBehavior.lodNeuralNetworkMapping.addIndices(index, netIndices, count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearLODNeuralNetworkMappings() {
    dna.machineLearnedBehavior.lodNeuralNetworkMapping.resetLODs();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setLODNeuralNetworkMapping(std::uint16_t lod, std::uint16_t index) {
    dna.machineLearnedBehavior.lodNeuralNetworkMapping.associateLODWithIndices(lod, index);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMeshRegionNames() {
    dna.machineLearnedBehavior.neuralNetworkToMeshRegion.regionNames.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMeshRegionNames(std::uint16_t meshIndex) {
    if (meshIndex < dna.machineLearnedBehavior.neuralNetworkToMeshRegion.regionNames.size()) {
        dna.machineLearnedBehavior.neuralNetworkToMeshRegion.regionNames[meshIndex].clear();
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMeshRegionName(std::uint16_t meshIndex, std::uint16_t regionIndex, const char* name) {
    auto& meshRegionNames = getAt(dna.machineLearnedBehavior.neuralNetworkToMeshRegion.regionNames, meshIndex);
    setAt(meshRegionNames, regionIndex, name);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearNeuralNetworkIndicesPerMeshRegion() {
    dna.machineLearnedBehavior.neuralNetworkToMeshRegion.indices.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setNeuralNetworkIndicesForMeshRegion(std::uint16_t meshIndex,
                                                                          std::uint16_t regionIndex,
                                                                          const std::uint16_t* netIndices,
                                                                          std::uint16_t count) {
    auto& neuralNetworkToMeshRegionIndices = getAt(dna.machineLearnedBehavior.neuralNetworkToMeshRegion.indices, meshIndex);
    auto& region = getAt(neuralNetworkToMeshRegionIndices, regionIndex);
    region.assign(netIndices, netIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::deleteNeuralNetwork(std::uint16_t netIndex) {
    if (netIndex < dna.machineLearnedBehavior.neuralNetworks.size()) {
        auto it = extd::advanced(dna.machineLearnedBehavior.neuralNetworks.begin(), netIndex);
        dna.machineLearnedBehavior.neuralNetworks.erase(it);
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setNeuralNetworkInputIndices(std::uint16_t netIndex,
                                                                  const std::uint16_t* inputIndices,
                                                                  std::uint16_t count) {
    auto& neuralNet = getAt(dna.machineLearnedBehavior.neuralNetworks, netIndex);
    neuralNet.inputIndices.assign(inputIndices, inputIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setNeuralNetworkOutputIndices(std::uint16_t netIndex,
                                                                   const std::uint16_t* outputIndices,
                                                                   std::uint16_t count) {
    auto& neuralNet = getAt(dna.machineLearnedBehavior.neuralNetworks, netIndex);
    neuralNet.outputIndices.assign(outputIndices, outputIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearNeuralNetworkLayers(std::uint16_t netIndex) {
    auto& neuralNet = getAt(dna.machineLearnedBehavior.neuralNetworks, netIndex);
    neuralNet.layers.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setNeuralNetworkLayerActivationFunction(std::uint16_t netIndex,
                                                                             std::uint16_t layerIndex,
                                                                             ActivationFunction function) {
    auto& neuralNet = getAt(dna.machineLearnedBehavior.neuralNetworks, netIndex);
    auto& layer = getAt(neuralNet.layers, layerIndex);
    layer.activationFunction.functionId = static_cast<std::uint16_t>(function);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setNeuralNetworkLayerActivationFunctionParameters(std::uint16_t netIndex,
                                                                                       std::uint16_t layerIndex,
                                                                                       const float* activationFunctionParameters,
                                                                                       std::uint16_t count) {
    auto& neuralNet = getAt(dna.machineLearnedBehavior.neuralNetworks, netIndex);
    auto& layer = getAt(neuralNet.layers, layerIndex);
    layer.activationFunction.parameters.assign(activationFunctionParameters, activationFunctionParameters + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setNeuralNetworkLayerBiases(std::uint16_t netIndex,
                                                                 std::uint16_t layerIndex,
                                                                 const float* biases,
                                                                 std::uint32_t count) {
    auto& neuralNet = getAt(dna.machineLearnedBehavior.neuralNetworks, netIndex);
    auto& layer = getAt(neuralNet.layers, layerIndex);
    layer.biases.assign(biases, biases + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setNeuralNetworkLayerWeights(std::uint16_t netIndex,
                                                                  std::uint16_t layerIndex,
                                                                  const float* weights,
                                                                  std::uint32_t count) {
    auto& neuralNet = getAt(dna.machineLearnedBehavior.neuralNetworks, netIndex);
    auto& layer = getAt(neuralNet.layers, layerIndex);
    layer.weights.assign(weights, weights + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearRBFPoses() {
    dna.rbfBehavior.poses.clear();
    dna.rbfBehaviorExt.poses.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFPoseName(std::uint16_t poseIndex, const char* name) {
    auto& pose = getAt(dna.rbfBehavior.poses, poseIndex);
    pose.name = name;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFPoseScale(std::uint16_t poseIndex, float scale) {
    auto& pose = getAt(dna.rbfBehavior.poses, poseIndex);
    pose.scale = scale;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearRBFPoseControlNames() {
    dna.rbfBehaviorExt.poseControlNames.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFPoseControlName(std::uint16_t poseControlIndex, const char* name) {
    setAt(dna.rbfBehaviorExt.poseControlNames, poseControlIndex, name);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFPoseInputControlIndices(std::uint16_t poseIndex,
                                                                   const std::uint16_t* controlIndices,
                                                                   std::uint16_t controlIndexCount) {
    auto& pose = getAt(dna.rbfBehaviorExt.poses, poseIndex);
    pose.inputControlIndices.assign(controlIndices, controlIndices + controlIndexCount);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFPoseOutputControlIndices(std::uint16_t poseIndex,
                                                                    const std::uint16_t* controlIndices,
                                                                    std::uint16_t controlIndexCount) {
    auto& pose = getAt(dna.rbfBehaviorExt.poses, poseIndex);
    pose.outputControlIndices.assign(controlIndices, controlIndices + controlIndexCount);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFPoseOutputControlWeights(std::uint16_t poseIndex,
                                                                    const float* controlWeights,
                                                                    std::uint16_t controlWeightCount) {
    auto& pose = getAt(dna.rbfBehaviorExt.poses, poseIndex);
    pose.outputControlWeights.assign(controlWeights, controlWeights + controlWeightCount);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearRBFSolvers() {
    dna.rbfBehavior.solvers.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearRBFSolverIndices() {
    dna.rbfBehavior.lodSolverMapping.resetIndices();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverIndices(std::uint16_t index,
                                                         const std::uint16_t* solverIndices,
                                                         std::uint16_t count) {
    dna.rbfBehavior.lodSolverMapping.clearIndices(index);
    dna.rbfBehavior.lodSolverMapping.addIndices(index, solverIndices, count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearLODRBFSolverMappings() {
    dna.rbfBehavior.lodSolverMapping.resetLODs();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setLODRBFSolverMapping(std::uint16_t lod, std::uint16_t index) {
    dna.rbfBehavior.lodSolverMapping.associateLODWithIndices(lod, index);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverName(std::uint16_t solverIndex, const char* name) {
    auto& solver = getAt(dna.rbfBehavior.solvers, solverIndex);
    solver.name = name;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverRawControlIndices(std::uint16_t solverIndex,
                                                                   const std::uint16_t* rawControlIndices,
                                                                   std::uint16_t count) {
    auto& solver = getAt(dna.rbfBehavior.solvers, solverIndex);
    solver.rawControlIndices.assign(rawControlIndices, rawControlIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverPoseIndices(std::uint16_t solverIndex,
                                                             const std::uint16_t* poseIndices,
                                                             std::uint16_t count) {
    auto& solver = getAt(dna.rbfBehavior.solvers, solverIndex);
    solver.poseIndices.assign(poseIndices, poseIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverRawControlValues(std::uint16_t solverIndex,
                                                                  const float* values,
                                                                  std::uint16_t count) {
    auto& solver = getAt(dna.rbfBehavior.solvers, solverIndex);
    solver.rawControlValues.assign(values, values + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverType(std::uint16_t solverIndex, RBFSolverType type) {
    auto& solver = getAt(dna.rbfBehavior.solvers, solverIndex);
    solver.solverType = static_cast<std::uint16_t>(type);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverRadius(std::uint16_t solverIndex, float radius) {
    auto& solver = getAt(dna.rbfBehavior.solvers, solverIndex);
    solver.radius = radius;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverAutomaticRadius(std::uint16_t solverIndex, AutomaticRadius automaticRadius) {
    auto& solver = getAt(dna.rbfBehavior.solvers, solverIndex);
    solver.automaticRadius = static_cast<std::uint16_t>(automaticRadius);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverWeightThreshold(std::uint16_t solverIndex, float weightThreshold) {
    auto& solver = getAt(dna.rbfBehavior.solvers, solverIndex);
    solver.weightThreshold = weightThreshold;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverDistanceMethod(std::uint16_t solverIndex, RBFDistanceMethod distanceMethod) {
    auto& solver = getAt(dna.rbfBehavior.solvers, solverIndex);
    solver.distanceMethod = static_cast<std::uint16_t>(distanceMethod);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverNormalizeMethod(std::uint16_t solverIndex, RBFNormalizeMethod normalizeMethod) {
    auto& solver = getAt(dna.rbfBehavior.solvers, solverIndex);
    solver.normalizeMethod = static_cast<std::uint16_t>(normalizeMethod);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverFunctionType(std::uint16_t solverIndex, RBFFunctionType functionType) {
    auto& solver = getAt(dna.rbfBehavior.solvers, solverIndex);
    solver.functionType = static_cast<std::uint16_t>(functionType);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRBFSolverTwistAxis(std::uint16_t solverIndex, TwistAxis twistAxis) {
    auto& solver = getAt(dna.rbfBehavior.solvers, solverIndex);
    solver.twistAxis = static_cast<std::uint16_t>(twistAxis);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearJointRepresentations() {
    dna.jointBehaviorMetadata.jointRepresentations.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointTranslationRepresentation(std::uint16_t jointIndex,
                                                                       TranslationRepresentation representation) {
    auto& jointRepresentation = getAt(dna.jointBehaviorMetadata.jointRepresentations, jointIndex);
    jointRepresentation.translation = static_cast<std::uint16_t>(representation);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointRotationRepresentation(std::uint16_t jointIndex,
                                                                    RotationRepresentation representation) {
    auto& jointRepresentation = getAt(dna.jointBehaviorMetadata.jointRepresentations, jointIndex);
    jointRepresentation.rotation = static_cast<std::uint16_t>(representation);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointScaleRepresentation(std::uint16_t jointIndex, ScaleRepresentation representation) {
    auto& jointRepresentation = getAt(dna.jointBehaviorMetadata.jointRepresentations, jointIndex);
    jointRepresentation.scale = static_cast<std::uint16_t>(representation);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearTwists() {
    dna.twistSwingBehavior.twists.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::deleteTwist(std::uint16_t twistIndex) {
    if (twistIndex < dna.twistSwingBehavior.twists.size()) {
        auto it = extd::advanced(dna.twistSwingBehavior.twists.begin(), twistIndex);
        dna.twistSwingBehavior.twists.erase(it);
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setTwistSetupTwistAxis(std::uint16_t twistIndex, TwistAxis twistAxis) {
    auto& twist = getAt(dna.twistSwingBehavior.twists, twistIndex);
    twist.twistAxis = static_cast<std::uint16_t>(twistAxis);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setTwistInputControlIndices(std::uint16_t twistIndex,
                                                                 const std::uint16_t* controlIndices,
                                                                 std::uint16_t controlIndexCount) {
    auto& twist = getAt(dna.twistSwingBehavior.twists, twistIndex);
    twist.twistInputControlIndices.assign(controlIndices, controlIndices + controlIndexCount);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setTwistOutputJointIndices(std::uint16_t twistIndex,
                                                                const std::uint16_t* jointIndices,
                                                                std::uint16_t jointIndexCount) {
    auto& twist = getAt(dna.twistSwingBehavior.twists, twistIndex);
    twist.twistOutputJointIndices.assign(jointIndices, jointIndices + jointIndexCount);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setTwistBlendWeights(std::uint16_t twistIndex,
                                                          const float* blendWeights,
                                                          std::uint16_t blendWeightCount) {
    auto& twist = getAt(dna.twistSwingBehavior.twists, twistIndex);
    twist.twistBlendWeights.assign(blendWeights, blendWeights + blendWeightCount);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearSwings() {
    dna.twistSwingBehavior.swings.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::deleteSwing(std::uint16_t swingIndex) {
    if (swingIndex < dna.twistSwingBehavior.swings.size()) {
        auto it = extd::advanced(dna.twistSwingBehavior.swings.begin(), swingIndex);
        dna.twistSwingBehavior.swings.erase(it);
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setSwingSetupTwistAxis(std::uint16_t swingIndex, TwistAxis twistAxis) {
    auto& swing = getAt(dna.twistSwingBehavior.swings, swingIndex);
    swing.twistAxis = static_cast<std::uint16_t>(twistAxis);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setSwingInputControlIndices(std::uint16_t swingIndex,
                                                                 const std::uint16_t* controlIndices,
                                                                 std::uint16_t controlIndexCount) {
    auto& swing = getAt(dna.twistSwingBehavior.swings, swingIndex);
    swing.swingInputControlIndices.assign(controlIndices, controlIndices + controlIndexCount);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setSwingOutputJointIndices(std::uint16_t swingIndex,
                                                                const std::uint16_t* jointIndices,
                                                                std::uint16_t jointIndexCount) {
    auto& swing = getAt(dna.twistSwingBehavior.swings, swingIndex);
    swing.swingOutputJointIndices.assign(jointIndices, jointIndices + jointIndexCount);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setSwingBlendWeights(std::uint16_t swingIndex,
                                                          const float* blendWeights,
                                                          std::uint16_t blendWeightCount) {
    auto& swing = getAt(dna.twistSwingBehavior.swings, swingIndex);
    swing.swingBlendWeights.assign(blendWeights, blendWeights + blendWeightCount);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMLExtData() {
    dna.machineLearnedBehaviorExt.mlbTypeData.clear();
    dna.machineLearnedBehaviorExt.mlbJoints.parameterKeys.clear();
    dna.machineLearnedBehaviorExt.mlbJoints.parameterValues.clear();
    dna.machineLearnedBehaviorExt.mlbJoints.inputIndices.clear();
    dna.machineLearnedBehaviorExt.mlbJoints.outputIndices.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMLOperationSets(std::uint16_t mlTypeIndex) {
    if (mlTypeIndex < dna.machineLearnedBehaviorExt.mlbTypeData.size()) {
        dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex].operations.clear();
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMLOperations(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) {
    if (mlTypeIndex < dna.machineLearnedBehaviorExt.mlbTypeData.size()) {
        auto& mlbTypeData = dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex];
        if (mlOperationSetIndex < mlbTypeData.operations.size()) {
            mlbTypeData.operations[mlOperationSetIndex].clear();
        }
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMLOperationType(std::uint16_t mlTypeIndex,
                                                        std::uint16_t mlOperationSetIndex,
                                                        std::uint16_t mlOperationIndex,
                                                        MachineLearnedBehaviorOperationType operationType) {
    auto& mlbTypeData = getAt(dna.machineLearnedBehaviorExt.mlbTypeData, mlTypeIndex);
    auto& operationSet = getAt(mlbTypeData.operations, mlOperationSetIndex);
    auto& operation = getAt(operationSet, mlOperationIndex);
    operation.operationType = static_cast<std::uint16_t>(operationType);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMLOperationParameters(std::uint16_t mlTypeIndex,
                                                              std::uint16_t mlOperationSetIndex,
                                                              std::uint16_t mlOperationIndex,
                                                              const std::uint32_t* parameters,
                                                              std::uint16_t count) {
    auto& mlbTypeData = getAt(dna.machineLearnedBehaviorExt.mlbTypeData, mlTypeIndex);
    auto& operationSet = getAt(mlbTypeData.operations, mlOperationSetIndex);
    auto& operation = getAt(operationSet, mlOperationIndex);
    operation.parameters.assign(parameters, parameters + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMLOperationDependencyOperationSetIndices(std::uint16_t mlTypeIndex,
                                                                                 std::uint16_t mlOperationSetIndex,
                                                                                 std::uint16_t mlOperationIndex,
                                                                                 const std::uint16_t* indices,
                                                                                 std::uint16_t count) {
    auto& mlbTypeData = getAt(dna.machineLearnedBehaviorExt.mlbTypeData, mlTypeIndex);
    auto& operationSet = getAt(mlbTypeData.operations, mlOperationSetIndex);
    auto& operation = getAt(operationSet, mlOperationIndex);
    operation.mlOperationSetDependencyIndices.assign(indices, indices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMLOperationDependencyOperationIndices(std::uint16_t mlTypeIndex,
                                                                              std::uint16_t mlOperationSetIndex,
                                                                              std::uint16_t mlOperationIndex,
                                                                              const std::uint16_t* indices,
                                                                              std::uint16_t count) {
    auto& mlbTypeData = getAt(dna.machineLearnedBehaviorExt.mlbTypeData, mlTypeIndex);
    auto& operationSet = getAt(mlbTypeData.operations, mlOperationSetIndex);
    auto& operation = getAt(operationSet, mlOperationIndex);
    operation.mlOperationDependencyIndices.assign(indices, indices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMLOperationIndicesAndLODMappings() {
    for (auto& typeData : dna.machineLearnedBehaviorExt.mlbTypeData) {
        typeData.lodMLOperationMappings.clear();
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearMLOperationIndices(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) {
    if (mlTypeIndex < dna.machineLearnedBehaviorExt.mlbTypeData.size()) {
        if (mlOperationSetIndex < dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex].lodMLOperationMappings.size()) {
            dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex].lodMLOperationMappings[mlOperationSetIndex].resetIndices();
        }
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearLODMLOperationMappings(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) {
    if (mlTypeIndex < dna.machineLearnedBehaviorExt.mlbTypeData.size()) {
        if (mlOperationSetIndex < dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex].lodMLOperationMappings.size()) {
            dna.machineLearnedBehaviorExt.mlbTypeData[mlTypeIndex].lodMLOperationMappings[mlOperationSetIndex].resetLODs();
        }
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMLOperationIndices(std::uint16_t mlTypeIndex,
                                                           std::uint16_t mlOperationSetIndex,
                                                           std::uint16_t index,
                                                           const std::uint16_t* mlOperationIndices,
                                                           std::uint16_t count) {
    auto& mlbTypeData = getAt(dna.machineLearnedBehaviorExt.mlbTypeData, mlTypeIndex);
    auto& mapping = getAt(mlbTypeData.lodMLOperationMappings, mlOperationSetIndex);
    mapping.clearIndices(index);
    mapping.addIndices(index, mlOperationIndices, count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setLODMLOperationMapping(std::uint16_t mlTypeIndex,
                                                              std::uint16_t mlOperationSetIndex,
                                                              std::uint16_t lod,
                                                              std::uint16_t index) {
    auto& mlbTypeData = getAt(dna.machineLearnedBehaviorExt.mlbTypeData, mlTypeIndex);
    auto& mapping = getAt(mlbTypeData.lodMLOperationMappings, mlOperationSetIndex);
    mapping.associateLODWithIndices(lod, index);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMLJointsParameterKeys(const std::uint16_t* parameterKeys, std::uint16_t count) {
    dna.machineLearnedBehaviorExt.mlbJoints.parameterKeys.assign(parameterKeys, parameterKeys + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMLJointsParameterValues(const std::uint16_t* parameterValues, std::uint16_t count) {
    dna.machineLearnedBehaviorExt.mlbJoints.parameterValues.assign(parameterValues, parameterValues + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMLJointsInputIndices(const std::uint16_t* inputIndices, std::uint16_t count) {
    dna.machineLearnedBehaviorExt.mlbJoints.inputIndices.assign(inputIndices, inputIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setMLJointsOutputIndices(const std::uint16_t* outputIndices, std::uint16_t count) {
    dna.machineLearnedBehaviorExt.mlbJoints.outputIndices.assign(outputIndices, outputIndices + count);
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif

}  // namespace gs4
