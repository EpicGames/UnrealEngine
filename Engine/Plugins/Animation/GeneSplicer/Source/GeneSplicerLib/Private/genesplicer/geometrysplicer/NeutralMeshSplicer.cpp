// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/geometrysplicer/NeutralMeshSplicer.h"

#include "genesplicer/GeneSplicerDNAReaderImpl.h"
#include "genesplicer/Macros.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/PoolSpliceParamsFilter.h"
#include "genesplicer/splicedata/PoolSpliceParamsImpl.h"
#include "genesplicer/splicedata/SpliceDataImpl.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/Vec3.h"

#include <cstdint>

namespace gs4 {

template<CalculationType CT>
void NeutralMeshSplicer<CT>::splice(const SpliceDataInterface* spliceData, GeneSplicerDNAReader* output_) {
    auto output = static_cast<GeneSplicerDNAReaderImpl*>(output_);
    auto outputMemRes = output->getMemoryResource();

    const auto& baseArch = spliceData->getBaseArchetype();
    const auto& baseArchMeshes = baseArch.getNeutralMeshes();

    auto neutralMeshPredicate = [](const RawGenes& baseArchetype, const PoolSpliceParamsImpl* pool, std::uint16_t meshIndex) {
        return baseArchetype.getNeutralMeshes()[meshIndex].size() == pool->getGenePool()->getVertexPositionCount(meshIndex);
    };
    auto poolsToSplicePerMesh = filterPoolSpliceParamsPerMesh(spliceData, neutralMeshPredicate, outputMemRes);

    for (std::uint16_t meshIdx = 0u; meshIdx < poolsToSplicePerMesh.rowCount(); meshIdx++) {

        const auto& baseArchMesh = baseArchMeshes[meshIdx];
        RawVector3Vector resultingVertices = constructWithPadding(baseArchMesh, outputMemRes);

        for (auto poolParams : poolsToSplicePerMesh[meshIdx]) {
            auto genePool = poolParams->getGenePool();
            const auto& vertexWeights = poolParams->getVertexWeightsData();
            const auto& neutralMeshes = genePool->getNeutralMeshes();
            float scale = poolParams->getScale();

            BlockSplicer<CT>::splice(Matrix2DView<const XYZBlock<16u>>{neutralMeshes[meshIdx]},
                                     Matrix2DView<const VBlock<16u>>{vertexWeights[meshIdx]},
                                     poolParams->getDNAIndices(),
                                     resultingVertices,
                                     scale);
        }
        resultingVertices.resize(output->getVertexPositionCount(meshIdx));
        output->setVertexPositions(meshIdx, std::move(resultingVertices));
    }
}

template class NeutralMeshSplicer<CalculationType::Scalar>;
template class NeutralMeshSplicer<CalculationType::SSE>;
template class NeutralMeshSplicer<CalculationType::AVX>;

}  // namespace gs4
