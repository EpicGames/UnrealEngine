// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/jointbehaviorsplicer/JointBehaviorSplicer.h"

#include "genesplicer/CalculationType.h"
#include "genesplicer/GeneSplicerDNAReaderImpl.h"
#include "genesplicer/Macros.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/PoolSpliceParamsImpl.h"
#include "genesplicer/splicedata/SpliceData.h"
#include "genesplicer/splicedata/SpliceDataImpl.h"
#include "genesplicer/types/Block.h"
#include "genesplicer/types/BlockStorage.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cmath>
#include <cstdint>
#include <numeric>
#include <type_traits>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

namespace {

template<CalculationType CT>
struct OutputIndexSplicer {
    static void splice(ConstArrayView<std::uint16_t> dnaIndices,
                       const TiledMatrix2D<16>& outputIndicesValueBlocks,
                       std::uint8_t vBlockRemainder,
                       std::uint8_t jointOffset,
                       ConstArrayView<VBlock<16u>> blockWeights,
                       float* dest,
                       float scale = 1.0f) {
        using TF256 = typename GetTF256<CT>::type;
        static constexpr auto blockSize = 16u;
        auto vBlockCount = static_cast<std::uint16_t>(outputIndicesValueBlocks.rowCount());
        auto endIdx = static_cast<std::uint16_t>(vBlockCount - (1u && vBlockRemainder));

        TF256 scale256(scale);

        for (std::uint16_t vBlockIndex = 0u; vBlockIndex < endIdx; vBlockIndex++) {
            const auto& vBlockDNAs = outputIndicesValueBlocks[vBlockIndex];
            auto result0 = TF256::fromUnalignedSource(dest + 0u);
            auto result1 = TF256::fromUnalignedSource(dest + 8u);

            for (std::uint16_t dnaIdx : dnaIndices) {
                auto weight = TF256(*(blockWeights[dnaIdx].v + jointOffset)) * scale256;
                const auto& vBlock = vBlockDNAs[dnaIdx];
                const auto dna0 = TF256::fromAlignedSource(vBlock.v + 0u);
                const auto dna1 = TF256::fromAlignedSource(vBlock.v + 8u);
                result0 += dna0 * weight;
                result1 += dna1 * weight;
            }
            result0.unalignedStore(dest + 0u);
            result1.unalignedStore(dest + 8u);
            dest += blockSize;
        }
        if (endIdx < vBlockCount) {
            const auto& vBlockDNAs = outputIndicesValueBlocks[endIdx];
            for (std::uint16_t dnaIdx : dnaIndices) {
                const auto& weight = blockWeights[dnaIdx].v[jointOffset] * scale;
                for (std::uint16_t i = 0u; i < vBlockRemainder; i++) {
                    dest[i] += weight * vBlockDNAs[dnaIdx].v[i];
                }
            }
        }
    }
};

template<>
struct OutputIndexSplicer<CalculationType::SSE> {
    static void splice(ConstArrayView<std::uint16_t> dnaIndices,
                       const TiledMatrix2D<16>& outputIndicesValueBlocks,
                       std::uint8_t vBlockRemainder,
                       std::uint8_t jointOffset,
                       ConstArrayView<VBlock<16u>> blockWeights,
                       float* dest,
                       float scale = 1.0f) {
        using TF128 = typename GetTF128<CalculationType::SSE>::type;
        static constexpr auto blockSize = 16u;
        const auto vBlockCount = static_cast<std::uint16_t>(outputIndicesValueBlocks.rowCount());
        const auto endIdx = static_cast<std::uint16_t>(vBlockCount - (1u && vBlockRemainder));

        const TF128 scale128(scale);

        for (std::uint16_t vBlockIndex = 0u; vBlockIndex < endIdx; vBlockIndex++) {
            const auto& vBlockDNAs = outputIndicesValueBlocks[vBlockIndex];
            auto result0 = TF128::fromUnalignedSource(dest + 0u);
            auto result1 = TF128::fromUnalignedSource(dest + 4u);
            auto result2 = TF128::fromUnalignedSource(dest + 8u);
            auto result3 = TF128::fromUnalignedSource(dest + 12u);

            for (std::uint16_t dnaIdx : dnaIndices) {
                const auto weight = TF128(*(blockWeights[dnaIdx].v + jointOffset)) * scale128;
                const auto& vBlock = vBlockDNAs[dnaIdx];
                const auto dna0 = TF128::fromAlignedSource(vBlock.v + 0u);
                const auto dna1 = TF128::fromAlignedSource(vBlock.v + 4u);
                const auto dna2 = TF128::fromAlignedSource(vBlock.v + 8u);
                const auto dna3 = TF128::fromAlignedSource(vBlock.v + 12u);
                result0 += dna0 * weight;
                result1 += dna1 * weight;
                result2 += dna2 * weight;
                result3 += dna3 * weight;
            }
            result0.unalignedStore(dest + 0u);
            result1.unalignedStore(dest + 4u);
            result2.unalignedStore(dest + 8u);
            result3.unalignedStore(dest + 12u);
            dest += blockSize;
        }
        if (endIdx < vBlockCount) {
            const auto& vBlockDNAs = outputIndicesValueBlocks[endIdx];
            for (std::uint16_t dnaIdx : dnaIndices) {
                const auto& weight = blockWeights[dnaIdx].v[jointOffset] * scale;
                for (std::uint16_t i = 0u; i < vBlockRemainder; i++) {
                    dest[i] += weight * vBlockDNAs[dnaIdx].v[i];
                }
            }
        }
    }
};

}  // namespace

template<CalculationType CT>
void JointBehaviorSplicer<CT>::splice(const SpliceDataInterface* spliceData, GeneSplicerDNAReader* output_) {
    auto output = static_cast<GeneSplicerDNAReaderImpl*>(output_);
    auto outputMemRes = output->getMemoryResource();

    const auto& baseArchJointGroups = spliceData->getBaseArchetype().getJointGroups();
    Vector<RawJointGroup> resultingJointGroups{baseArchJointGroups.begin(), baseArchJointGroups.end(), outputMemRes};
    auto pools = spliceData->getAllPoolParams();

    std::uint16_t jointCount = spliceData->getBaseArchetype().getJointCount();
    Vector<std::uint16_t> jointToGroupMapping{jointCount, {}, outputMemRes};

    for (std::uint16_t jointGroupIdx = 0u; jointGroupIdx < resultingJointGroups.size(); jointGroupIdx++) {
        for (auto jointIndex : resultingJointGroups[jointGroupIdx].jointIndices) {
            jointToGroupMapping[jointIndex] = jointGroupIdx;
        }
    }

    for (auto poolParams : pools) {
        auto genePool = poolParams->getGenePool();
        if ((genePool->getJointGroupCount() == 0) || (genePool->getJointGroupCount() != resultingJointGroups.size())) {
            continue;
        }
        const auto& jointValues = genePool->getJointBehaviorValues();
        const auto& jointWeightsData = poolParams->getJointWeightsData();
        auto dnaIndices = poolParams->getDNAIndices();
        float scale = poolParams->getScale();

        const auto& outputIndexTargetOffsets = poolParams->getJointBehaviorOutputIndexTargetOffsets();

        static constexpr auto blockSize = 16u;

        for (std::uint16_t jntIdx = 0u; jntIdx < jointCount; jntIdx++) {

            auto jointOutputTargetOffsets = outputIndexTargetOffsets[jntIdx];  // mapping
            auto jointOutputOffsets = jointValues[jntIdx].getOutputOffsets();  // 0-9 (Joint attributes)

            const auto& outputIndicesValueBlocks = jointValues[jntIdx].getValues();
            auto blockIndex = static_cast<std::uint16_t>(jntIdx / blockSize);
            auto blockWeights = jointWeightsData[blockIndex];
            auto jntOffset = static_cast<std::uint8_t>(jntIdx % blockSize);

            std::uint16_t jntGroupIdx = jointToGroupMapping[jntIdx];
            auto& resultingJointGroup = resultingJointGroups[jntGroupIdx];

            const auto inputCount = resultingJointGroup.inputIndices.size();
            auto vBlockRemainder = static_cast<uint8_t>(inputCount % blockSize);

            auto dest = resultingJointGroup.values.data();

            std::uint8_t i = 0u;
            std::uint8_t translationCount = jointValues[jntIdx].getTranslationCount();
            for (; i < translationCount; i++) {
                std::uint8_t outputOffset = jointOutputOffsets[i];
                OutputIndexSplicer<CT>::splice(dnaIndices,
                                               outputIndicesValueBlocks[outputOffset],
                                               vBlockRemainder,
                                               jntOffset,
                                               blockWeights,
                                               dest + jointOutputTargetOffsets[outputOffset] * inputCount,
                                               scale);
            }

            for (; i < jointOutputOffsets.size(); i++) {
                std::uint8_t outputOffset = jointOutputOffsets[i];
                OutputIndexSplicer<CT>::splice(dnaIndices,
                                               outputIndicesValueBlocks[outputOffset],
                                               vBlockRemainder,
                                               jntOffset,
                                               blockWeights,
                                               dest + jointOutputTargetOffsets[outputOffset] * inputCount);
            }
        }
    }
    output->setJointGroups(std::move(resultingJointGroups));
}

template class JointBehaviorSplicer<CalculationType::Scalar>;
template class JointBehaviorSplicer<CalculationType::SSE>;
template class JointBehaviorSplicer<CalculationType::AVX>;

}  // namespace gs4
