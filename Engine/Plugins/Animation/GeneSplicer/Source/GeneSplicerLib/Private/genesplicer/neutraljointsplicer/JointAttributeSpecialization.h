// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/GeneSplicerDNAReaderImpl.h"
#include "genesplicer/Macros.h"
#include "genesplicer/neutraljointsplicer/JointAttribute.h"
#include "genesplicer/types/Vec3.h"

namespace gs4 {

template<JointAttribute T>
FORCE_INLINE ConstVec3VectorView getJointAttributeValues(const Reader* const reader);

template<>
FORCE_INLINE ConstVec3VectorView getJointAttributeValues<JointAttribute::Translation>(const Reader* const reader) {
    return ConstVec3VectorView{reader->getNeutralJointTranslationXs(),
                               reader->getNeutralJointTranslationYs(),
                               reader->getNeutralJointTranslationZs()};
}

template<>
FORCE_INLINE ConstVec3VectorView getJointAttributeValues<JointAttribute::Rotation>(const Reader* const reader) {
    return ConstVec3VectorView{reader->getNeutralJointRotationXs(),
                               reader->getNeutralJointRotationYs(),
                               reader->getNeutralJointRotationZs()};
}

template<JointAttribute T>
FORCE_INLINE Vec3 getJointAttributeValue(const Reader* const reader, std::uint16_t jointIndex);

template<>
FORCE_INLINE Vec3 getJointAttributeValue<JointAttribute::Translation>(const Reader* const reader, std::uint16_t jointIndex) {
    return reader->getNeutralJointTranslation(jointIndex);
}

template<>
FORCE_INLINE Vec3 getJointAttributeValue<JointAttribute::Rotation>(const Reader* const reader, std::uint16_t jointIndex) {
    return reader->getNeutralJointRotation(jointIndex);
}

template<JointAttribute T>
FORCE_INLINE void setJointAttributeValues(GeneSplicerDNAReaderImpl* output, RawVector3Vector& result);

template<>
FORCE_INLINE void setJointAttributeValues<JointAttribute::Translation>(GeneSplicerDNAReaderImpl* output,
                                                                       RawVector3Vector& result) {
    output->setNeutralJointTranslations(std::move(result));
}

template<>
FORCE_INLINE void setJointAttributeValues<JointAttribute::Rotation>(GeneSplicerDNAReaderImpl* output, RawVector3Vector& result) {
    output->setNeutralJointRotations(std::move(result));
}

}  // namespace gs4
