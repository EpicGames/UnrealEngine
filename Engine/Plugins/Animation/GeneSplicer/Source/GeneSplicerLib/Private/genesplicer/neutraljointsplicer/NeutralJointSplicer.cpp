// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/neutraljointsplicer/NeutralJointSplicer.h"

#include "genesplicer/GeneSplicerDNAReaderImpl.h"
#include "genesplicer/Macros.h"
#include "genesplicer/neutraljointsplicer/JointAttribute.h"
#include "genesplicer/neutraljointsplicer/JointAttributeSpecialization.h"
#include "genesplicer/splicedata/PoolSpliceParams.h"
#include "genesplicer/splicedata/PoolSpliceParamsImpl.h"
#include "genesplicer/splicedata/SpliceDataImpl.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"
#include "genesplicer/splicedata/genepool/RawNeutralJoints.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/Matrix.h"
#include "genesplicer/types/PImplExtractor.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

template<CalculationType CT>
void NeutralJointSplicer<CT>::splice(const SpliceDataInterface* spliceData, GeneSplicerDNAReader* output_) {
    auto output = static_cast<GeneSplicerDNAReaderImpl*>(output_);
    auto splice = [spliceData, output](JointAttribute jointAttribute,
                                       std::function<float(const PoolSpliceParamsImpl* poolParams)> scaleGetter) {
        auto outputMemRes = output->getMemoryResource();
        const auto& baseArch = spliceData->getBaseArchetype();
        auto poolParams = spliceData->getAllPoolParams();
        const auto& baseArchJoints = baseArch.getNeutralJoints(jointAttribute);
        RawVector3Vector resultingJoints = constructWithPadding(baseArchJoints, outputMemRes);

        for (auto pool : poolParams) {
            auto genePool = pool->getGenePool();
            if (genePool->getNeutralJointCount() != baseArch.getJointCount()) {
                continue;
            }
            const auto& neutralJoints = genePool->getNeutralJoints(jointAttribute);
            const auto& jointWeights = pool->getJointWeightsData();
            auto dnaIndices = pool->getDNAIndices();
            BlockSplicer<CT>::splice(Matrix2DView<const XYZBlock<16u>>{neutralJoints},
                                     Matrix2DView<const VBlock<16u>>{jointWeights},
                                     dnaIndices,
                                     resultingJoints,
                                     scaleGetter(pool));
        }
        resultingJoints.resize(baseArch.getJointCount());
        return resultingJoints;
    };
    RawNeutralJoints neutralJoints{output->getMemoryResource()};
    neutralJoints.translations =
        splice(JointAttribute::Translation, [](const PoolSpliceParamsImpl* poolParams) { return poolParams->getScale(); });
    // we want to avoid scaling of rotations, so we are pegging it to 1.0f
    neutralJoints.rotations = splice(JointAttribute::Rotation, [](const PoolSpliceParamsImpl* /*unused*/) { return 1.0f; });
    neutralJoints.coordinateSystem = output->getCoordinateSystem();
    neutralJoints.rotationSequence = output->getRotationSequence();
    neutralJoints.rotationSigns = output->getRotationSign();
    auto getJointParentIndex = std::bind(&dna::DefinitionReader::getJointParentIndex, output_, std::placeholders::_1);
    // Since GenePools and RawGenes of archetype hold joints in world space we need to convert them to local space
    toLocalSpace(getJointParentIndex, neutralJoints);

    auto toDegrees = [](float& angle) { angle = tdm::degrees(angle); };

    auto& rotations = neutralJoints.rotations;
    std::for_each(rotations.xs.begin(), rotations.xs.end(), toDegrees);
    std::for_each(rotations.ys.begin(), rotations.ys.end(), toDegrees);
    std::for_each(rotations.zs.begin(), rotations.zs.end(), toDegrees);

    output->setNeutralJointTranslations(std::move(neutralJoints.translations));
    output->setNeutralJointRotations(std::move(neutralJoints.rotations));
}

template class NeutralJointSplicer<CalculationType::Scalar>;
template class NeutralJointSplicer<CalculationType::SSE>;
template class NeutralJointSplicer<CalculationType::AVX>;

}  // namespace gs4
