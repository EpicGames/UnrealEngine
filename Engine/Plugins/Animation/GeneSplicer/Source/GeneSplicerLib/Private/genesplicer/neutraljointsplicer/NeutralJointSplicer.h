// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/Splicer.h"
#include "genesplicer/neutraljointsplicer/JointAttribute.h"

namespace gs4 {

class GeneSplicerDNAReader;
class SpliceData;

template<CalculationType CT>
class NeutralJointSplicer : public Splicer {
public:
    using Splicer::Splicer;

    void splice(const SpliceDataInterface* spliceData, GeneSplicerDNAReader* output) override;
};

}  // namespace gs4
