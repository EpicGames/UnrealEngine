// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/skinweightsplicer/SkinWeightMeshSplicer.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4242 4244 4365 4987)
#endif
#include <numeric>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

template<CalculationType CT>
SkinWeightMeshSplicer<CT>::SkinWeightMeshSplicer(ConstArrayView<const PoolSpliceParamsImpl*> poolSpliceParams,
                                                 std::uint16_t meshIndex,
                                                 MemoryResource* memRes) :
    genePools{memRes},
    jointIndices{},
    skinWeightCount{},
    maximumInfluences{} {
    if (poolSpliceParams.size() > 0) {
        auto genePool = poolSpliceParams[0]->getGenePool();
        jointIndices = &genePool->getSkinWeightJointIndices()[meshIndex];
        maximumInfluences = genePool->getMaximumInfluencesPerVertex(meshIndex);
        skinWeightCount = genePool->getSkinWeightsCount(meshIndex);
        genePools.reserve(poolSpliceParams.size());
        for (auto pool : poolSpliceParams) {
            genePools.push_back(GenePoolDetails{pool, meshIndex});
        }
    }
}

template<CalculationType CT>
ConstArrayView<std::uint16_t> SkinWeightMeshSplicer<CT>::getJointIndices(std::uint32_t vertexIndex) const {
    return (*jointIndices)[vertexIndex];
}

template<CalculationType CT>
MemoryResource* SkinWeightMeshSplicer<CT>::getMemoryResource() const {
    return genePools.get_allocator().getMemoryResource();
}

template<CalculationType CT>
SkinWeightMeshSplicer<CT>::GenePoolDetails::GenePoolDetails(const PoolSpliceParamsImpl* pool, std::uint16_t meshIndex) :
    meshDNABlocks{pool->getGenePool()->getSkinWeightValues()[meshIndex]},
    meshWeights{pool->getVertexWeightsData()[meshIndex]},
    dnaIndices{pool->getDNAIndices()} {
}

template<CalculationType CT>
Vector<RawVertexSkinWeights> SkinWeightMeshSplicer<CT>::spliceMesh(const Vector<RawVertexSkinWeights>& baseArchSkinWeights,
                                                                   MemoryResource* outputMemRes) const {
    Vector<RawVertexSkinWeights> outputSkinWeights{baseArchSkinWeights.begin(), baseArchSkinWeights.end(), outputMemRes};
    const std::size_t blockCount = getBlockCount(skinWeightCount);
    constexpr std::uint16_t blockCapacity = 16u;
    if (blockCount > 0) {
        std::uint32_t blockIndex = 0u;
        for (; blockIndex < blockCount - 1u; ++blockIndex) {
            spliceBlock(blockIndex, {outputSkinWeights.data() + blockIndex * blockCapacity, blockCapacity});
        }
        const auto blockSize = skinWeightCount - blockIndex * blockCapacity;
        spliceBlock(blockIndex, {outputSkinWeights.data() + blockIndex * blockCapacity, blockSize});
    }
    return outputSkinWeights;
}

template<CalculationType CT>
void SkinWeightMeshSplicer<CT>::spliceBlock(std::uint32_t blockIndex, ArrayView<RawVertexSkinWeights> outputBlock) const {
    auto maximumJointCount = static_cast<std::uint16_t>(genePools[0].meshDNABlocks[blockIndex].columnCount());
    AlignedVBlock16Vector weights{maximumJointCount, {}, getMemoryResource()};
    const auto weightSum = spliceAndNormalize(blockIndex, weights);

    constexpr std::uint16_t blockSize = 16u;
    std::uint32_t vtxIndex = blockIndex * blockSize;
    for (std::uint32_t vtxOffset = 0; vtxOffset < outputBlock.size(); vtxOffset++) {
        if (weightSum[vtxOffset] == 0.0f) {
            continue;
        }
        auto& vtxRes = outputBlock[vtxOffset];
        auto vtxJointIndices = getJointIndices(vtxIndex + vtxOffset);
        vtxRes.jointIndices.assign(vtxJointIndices.begin(), vtxJointIndices.end());
        vtxRes.weights.resize(vtxRes.jointIndices.size());

        for (std::uint16_t jntPos = 0; jntPos < vtxRes.jointIndices.size(); jntPos++) {
            vtxRes.weights[jntPos] = weights[jntPos][vtxOffset];
        }
        if (vtxRes.jointIndices.size() > maximumInfluences) {
            prune(vtxRes);
        }
    }
}

template<CalculationType CT>
VBlock<16u> SkinWeightMeshSplicer<CT>::spliceAndNormalize(std::uint32_t blockIndex, AlignedVBlock16Vector& blockResult) const {
    using TF256 = typename GetTF256<CT>::type;
    alignas(64) VBlock<16> totalSumBlock{};
    TF256 totalSum0{};
    TF256 totalSum1{};

    for (const auto& genePool : genePools) {
        const auto& weights = genePool.meshWeights[blockIndex];
        const auto& dnas = genePool.meshDNABlocks[blockIndex];
        for (std::uint16_t dnaIdx : genePool.dnaIndices) {
            const auto weight0 = trimd::abs(TF256::fromAlignedSource(weights[dnaIdx].v + 0u));
            const auto weight1 = trimd::abs(TF256::fromAlignedSource(weights[dnaIdx].v + 8u));

            for (std::size_t jntPos = 0; jntPos < blockResult.size(); jntPos++) {
                auto res0 = TF256::fromAlignedSource(blockResult[jntPos].v + 0u);
                auto res1 = TF256::fromAlignedSource(blockResult[jntPos].v + 8u);

                const auto dna0 = TF256::fromAlignedSource(dnas[dnaIdx][jntPos].v + 0u) * weight0;
                const auto dna1 = TF256::fromAlignedSource(dnas[dnaIdx][jntPos].v + 8u) * weight1;

                res0 += dna0;
                res1 += dna1;

                totalSum0 += dna0;
                totalSum1 += dna1;

                res0.alignedStore(blockResult[jntPos].v + 0u);
                res1.alignedStore(blockResult[jntPos].v + 8u);
            }
        }
    }
    totalSum0.alignedStore(totalSumBlock.v + 0u);
    totalSum1.alignedStore(totalSumBlock.v + 8u);

    TF256 mmOnes{1.0f};
    totalSum0 = mmOnes / totalSum0;
    totalSum1 = mmOnes / totalSum1;

    for (std::size_t jntPos = 0; jntPos < blockResult.size(); jntPos++) {
        const auto res0 = TF256::fromAlignedSource(blockResult[jntPos].v + 0u) * totalSum0;
        const auto res1 = TF256::fromAlignedSource(blockResult[jntPos].v + 8u) * totalSum1;

        res0.alignedStore(blockResult[jntPos].v + 0u);
        res1.alignedStore(blockResult[jntPos].v + 8u);
    }
    return totalSumBlock;
}

template<>
VBlock<16u> SkinWeightMeshSplicer<CalculationType::SSE>::spliceAndNormalize(std::uint32_t blockIndex,
                                                                            AlignedVBlock16Vector& blockResult) const {
    using TF128 = typename GetTF128<CalculationType::SSE>::type;
    alignas(64) VBlock<16> totalSumBlock{};
    TF128 totalSum0{};
    TF128 totalSum1{};
    TF128 totalSum2{};
    TF128 totalSum3{};

    for (const auto& genePool : genePools) {
        const auto& weights = genePool.meshWeights[blockIndex];
        const auto& dnas = genePool.meshDNABlocks[blockIndex];
        for (std::uint16_t dnaIdx : genePool.dnaIndices) {
            const auto weight0 = (TF128::fromAlignedSource(weights[dnaIdx].v + 0u));
            const auto weight1 = (TF128::fromAlignedSource(weights[dnaIdx].v + 4u));
            const auto weight2 = (TF128::fromAlignedSource(weights[dnaIdx].v + 8u));
            const auto weight3 = (TF128::fromAlignedSource(weights[dnaIdx].v + 12u));

            for (std::size_t jntPos = 0; jntPos < blockResult.size(); jntPos++) {
                auto res0 = TF128::fromAlignedSource(blockResult[jntPos].v + 0u);
                auto res1 = TF128::fromAlignedSource(blockResult[jntPos].v + 4u);
                auto res2 = TF128::fromAlignedSource(blockResult[jntPos].v + 8u);
                auto res3 = TF128::fromAlignedSource(blockResult[jntPos].v + 12u);

                const auto dna0 = TF128::fromAlignedSource(dnas[dnaIdx][jntPos].v + 0u) * weight0;
                const auto dna1 = TF128::fromAlignedSource(dnas[dnaIdx][jntPos].v + 4u) * weight1;
                const auto dna2 = TF128::fromAlignedSource(dnas[dnaIdx][jntPos].v + 8u) * weight2;
                const auto dna3 = TF128::fromAlignedSource(dnas[dnaIdx][jntPos].v + 12u) * weight3;

                res0 += dna0;
                res1 += dna1;
                res2 += dna2;
                res3 += dna3;

                totalSum0 += dna0;
                totalSum1 += dna1;
                totalSum2 += dna2;
                totalSum3 += dna3;

                res0.alignedStore(blockResult[jntPos].v + 0u);
                res1.alignedStore(blockResult[jntPos].v + 4u);
                res2.alignedStore(blockResult[jntPos].v + 8u);
                res3.alignedStore(blockResult[jntPos].v + 12u);
            }
        }
    }
    totalSum0.alignedStore(totalSumBlock.v + 0u);
    totalSum1.alignedStore(totalSumBlock.v + 4u);
    totalSum2.alignedStore(totalSumBlock.v + 8u);
    totalSum3.alignedStore(totalSumBlock.v + 12u);

    const TF128 ones{1.0f};
    totalSum0 = ones / totalSum0;
    totalSum1 = ones / totalSum1;
    totalSum2 = ones / totalSum2;
    totalSum3 = ones / totalSum3;

    for (std::size_t jntPos = 0; jntPos < blockResult.size(); jntPos++) {
        const auto res0 = TF128::fromAlignedSource(blockResult[jntPos].v + 0u) * totalSum0;
        const auto res1 = TF128::fromAlignedSource(blockResult[jntPos].v + 4u) * totalSum1;
        const auto res2 = TF128::fromAlignedSource(blockResult[jntPos].v + 8u) * totalSum2;
        const auto res3 = TF128::fromAlignedSource(blockResult[jntPos].v + 12u) * totalSum3;

        res0.alignedStore(blockResult[jntPos].v + 0u);
        res1.alignedStore(blockResult[jntPos].v + 4u);
        res2.alignedStore(blockResult[jntPos].v + 8u);
        res3.alignedStore(blockResult[jntPos].v + 12u);
    }
    return totalSumBlock;
}

template<CalculationType CT>
void SkinWeightMeshSplicer<CT>::prune(RawVertexSkinWeights& vtxRes) const {
    auto& weights = vtxRes.weights;
    auto& jntIndices = vtxRes.jointIndices;
    auto pruningSize = static_cast<std::uint32_t>(jntIndices.size() - maximumInfluences);

    for (std::uint32_t i = pruningSize; i < weights.size(); i++) {
        for (std::uint32_t j = i - pruningSize; j < i; j++) {
            if (weights[j] < weights[i]) {
                std::swap(weights[j], weights[i]);
                std::swap(jntIndices[j], jntIndices[i]);
                break;
            }
        }
    }
    /*
    w[0] + w[1] +...+ w[n] = 1
    w[n] + w[n-1] +...+ w[n-pruningSize] = PSum
    w[0] + w[1] +...+ w[n-pruningSize] = 1 - PSum
    normRatio = 1/(1 - PSum )
    w[0] * normRatio + w[1] * normRatio + ... + w[n-pruningSize]*normRatio = (1 - PSum) * normRatio
    (1 - PSum) * normRatio = (1 - PSum) * 1/(1-PSum) = 1
     */
    const int distance = static_cast<int>(pruningSize) * -1;
    const float normRatio = 1.0f / (1.0f - std::accumulate(std::next(weights.end(), distance), weights.end(), 0.0f));
    weights.resize(maximumInfluences);
    jntIndices.resize(maximumInfluences);
    for (auto& weight : weights) {
        weight *= normRatio;
    }
}

template class SkinWeightMeshSplicer<CalculationType::AVX>;
template class SkinWeightMeshSplicer<CalculationType::SSE>;
template class SkinWeightMeshSplicer<CalculationType::Scalar>;

}  // namespace gs4
