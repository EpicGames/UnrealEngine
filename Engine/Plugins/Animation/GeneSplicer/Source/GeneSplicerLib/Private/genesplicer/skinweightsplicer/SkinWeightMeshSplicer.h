// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/splicedata/PoolSpliceParamsImpl.h"
#include "genesplicer/splicedata/SpliceDataImpl.h"
#include "genesplicer/splicedata/genepool/GenePoolInterface.h"
#include "genesplicer/splicedata/genepool/SkinWeightPool.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/Matrix.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#include <cstdint>

namespace gs4 {

template<CalculationType CT>
class SkinWeightMeshSplicer {
public:
    using AlignedVBlock16Vector = Vector<VBlock<16>, PolyAllocator<VBlock<16>, 64>>;

private:
    struct GenePoolDetails {
    public:
        GenePoolDetails(const PoolSpliceParamsImpl* pool, std::uint16_t meshIndex);
        ConstArrayView<TiledMatrix2D<16u>> meshDNABlocks;
        Matrix2DView<const VBlock<16>> meshWeights;
        ConstArrayView<std::uint16_t> dnaIndices;
    };

public:
    SkinWeightMeshSplicer(ConstArrayView<const PoolSpliceParamsImpl*> pools, std::uint16_t meshIndex, MemoryResource* memRes);
    Vector<RawVertexSkinWeights> spliceMesh(const Vector<RawVertexSkinWeights>& baseArchetypeMeshSkinWeights,
                                            MemoryResource* outputMemRes) const;

private:
    ConstArrayView<std::uint16_t> getJointIndices(std::uint32_t vertexIndex) const;
    MemoryResource* getMemoryResource() const;

    void prune(RawVertexSkinWeights& vtxRes) const;
    VBlock<16> spliceAndNormalize(std::uint32_t blockIndex, AlignedVBlock16Vector& blockResult) const;

    void spliceBlock(std::uint32_t blockIndex, ArrayView<RawVertexSkinWeights> outputBlock) const;

private:
    Vector<GenePoolDetails> genePools;
    const VariableWidthMatrix<uint16_t>* jointIndices;
    std::uint32_t skinWeightCount;
    std::uint16_t maximumInfluences;
};

}  // namespace gs4
