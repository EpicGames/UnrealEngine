// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/skinweightsplicer/SkinWeightSplicer.h"

#include "genesplicer/GeneSplicerDNAReaderImpl.h"
#include "genesplicer/Macros.h"
#include "genesplicer/skinweightsplicer/SkinWeightMeshSplicer.h"
#include "genesplicer/splicedata/GenePool.h"
#include "genesplicer/splicedata/PoolSpliceParams.h"
#include "genesplicer/splicedata/PoolSpliceParamsFilter.h"
#include "genesplicer/splicedata/PoolSpliceParamsImpl.h"
#include "genesplicer/splicedata/SpliceDataImpl.h"
#include "genesplicer/splicedata/VertexWeights.h"
#include "genesplicer/splicedata/genepool/GenePoolInterface.h"
#include "genesplicer/splicedata/genepool/SkinWeightPool.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/Matrix.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cmath>
#include <cstdint>
#include <numeric>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

template<CalculationType CT>
void SkinWeightSplicer<CT>::splice(const SpliceDataInterface* spliceData, GeneSplicerDNAReader* output_) {
    auto output = static_cast<GeneSplicerDNAReaderImpl*>(output_);
    auto outputMemRes = output->getMemoryResource();

    const auto& baseArch = spliceData->getBaseArchetype();

    auto skinWeightPredicate = [](const RawGenes& baseArchetype, const PoolSpliceParamsImpl* pool, std::uint16_t meshIndex) {
        return baseArchetype.getSkinWeightsCount(meshIndex) == pool->getGenePool()->getSkinWeightsCount(meshIndex);
    };
    auto poolsToSplice = filterPoolSpliceParamsPerMesh(spliceData, skinWeightPredicate, outputMemRes);

    for (std::uint16_t meshIdx = 0u; meshIdx < baseArch.getMeshCount(); meshIdx++) {
        auto baseArchSkinWeights = baseArch.getSkinWeights();
        SkinWeightMeshSplicer<CT> meshSkinWeightPoolViewer{poolsToSplice[meshIdx], meshIdx, outputMemRes};
        auto meshResult = meshSkinWeightPoolViewer.spliceMesh(baseArchSkinWeights[meshIdx], outputMemRes);
        output->setSkinWeights(meshIdx, std::move(meshResult));
    }
}

template class SkinWeightSplicer<CalculationType::Scalar>;
template class SkinWeightSplicer<CalculationType::SSE>;
template class SkinWeightSplicer<CalculationType::AVX>;

}  // namespace gs4
