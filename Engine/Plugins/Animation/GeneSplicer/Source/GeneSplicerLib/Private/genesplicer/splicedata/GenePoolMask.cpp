// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/GenePoolMask.h"

#include <type_traits>

namespace gs4 {

GenePoolMask operator~(GenePoolMask a) {
    return static_cast<GenePoolMask>(~static_cast<std::uint32_t>(a));
}

GenePoolMask operator|(GenePoolMask a, GenePoolMask b) {
    return static_cast<GenePoolMask>(static_cast<std::uint32_t>(a) | static_cast<std::uint32_t>(b));
}

GenePoolMask operator&(GenePoolMask a, GenePoolMask b) {
    return static_cast<GenePoolMask>(static_cast<std::uint32_t>(a) & static_cast<std::uint32_t>(b));
}

}  // namespace gs4
