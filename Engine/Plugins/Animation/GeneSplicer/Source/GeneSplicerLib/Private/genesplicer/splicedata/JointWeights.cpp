// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/JointWeights.h"

#include "genesplicer/Macros.h"
#include "genesplicer/splicedata/RegionAffiliation.h"
#include "genesplicer/splicedata/SpliceWeights.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"

namespace gs4 {

JointWeights::JointWeights(const JointRegionAffiliationReader* regionAffiliationReader, MemoryResource* memRes) :
    weights{0ul, 0ul, memRes},
    regionAffiliations{memRes} {
    std::uint16_t jointCount = regionAffiliationReader->getJointCount();
    regionAffiliations.reserve(jointCount);
    for (std::uint16_t jointIdx = 0; jointIdx < jointCount; jointIdx++) {
        const auto indices = regionAffiliationReader->getJointRegionIndices(jointIdx);
        const auto values = regionAffiliationReader->getJointRegionAffiliation(jointIdx);
        regionAffiliations.emplace_back(indices, values, memRes);
    }
}

bool JointWeights::empty() const {
    return weights.size() == 0ul;
}

void JointWeights::clear() {
    auto memRes = weights.getAllocator().getMemoryResource();
    weights = TiledMatrix2D<16u>{0u, 0u, memRes};
}

void JointWeights::compute(const SpliceWeights& spliceWeights, ConstArrayView<std::uint16_t> dnaIndices) {
    clear();
    auto memRes = weights.getAllocator().getMemoryResource();
    const auto spliceWeightsData = spliceWeights.getData();
    const auto jointCount = static_cast<std::uint16_t>(regionAffiliations.size());
    std::uint16_t dnaCount = spliceWeights.getDNACount();

    constexpr auto blockSize = TiledMatrix2D<16u>::value_type::size();
    const auto endIdx = static_cast<std::uint16_t>(jointCount / blockSize);
    const auto remainder = static_cast<std::uint16_t>(jointCount % blockSize);
    const auto blockCount = static_cast<std::uint16_t>(endIdx + (1u && remainder));

    weights = TiledMatrix2D<16u>{blockCount, dnaCount, memRes};
    if (jointCount == 0) {
        return;
    }

    for (std::uint16_t blockIdx = 0u; blockIdx < endIdx; ++blockIdx) {

        for (std::uint16_t dnaIdx : dnaIndices) {
            const auto spliceWeightsPerRegion = spliceWeightsData[dnaIdx];
            auto& block = weights[blockIdx][dnaIdx];
            auto jntIdx = static_cast<std::uint16_t>(blockIdx * blockSize);

            for (std::size_t i = 0; i < blockSize; i += 4u) {
                const auto targetIndex = static_cast<std::uint16_t>(jntIdx + i);
                block[i] = regionAffiliations[targetIndex].totalWeightAcrossRegions(spliceWeightsPerRegion);
                block[i + 1u] = regionAffiliations[targetIndex + 1u].totalWeightAcrossRegions(spliceWeightsPerRegion);
                block[i + 2u] = regionAffiliations[targetIndex + 2u].totalWeightAcrossRegions(spliceWeightsPerRegion);
                block[i + 3u] = regionAffiliations[targetIndex + 3u].totalWeightAcrossRegions(spliceWeightsPerRegion);
            }
        }
    }
    auto lastBlock = weights[blockCount - 1u];
    auto jntIdx = static_cast<std::uint16_t>(jointCount - remainder);
    for (std::uint16_t i = 0; i < remainder; i++) {
        for (std::uint16_t dnaIdx : dnaIndices) {
            lastBlock[dnaIdx][i] = regionAffiliations[jntIdx].totalWeightAcrossRegions(spliceWeightsData[dnaIdx]);
        }
        jntIdx++;
    }
}

// [blockIndex][dnaIdx][jointOffset]
const TiledMatrix2D<16u>& JointWeights::getData() const {
    return weights;
}

}  // namespace gs4
