// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/RegionAffiliation.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"

#include <cstdint>

namespace gs4 {

class SpliceWeights;

class JointWeights {
public:
    explicit JointWeights(const JointRegionAffiliationReader* jointRegionAffiliation, MemoryResource* memRes);

    void compute(const SpliceWeights& spliceWeights, ConstArrayView<std::uint16_t> dnaIndices);

    bool empty() const;
    void clear();
    const TiledMatrix2D<16u>& getData() const;

private:
    TiledMatrix2D<16u> weights;
    Vector<RegionAffiliation<>> regionAffiliations;
};

}  // namespace gs4
