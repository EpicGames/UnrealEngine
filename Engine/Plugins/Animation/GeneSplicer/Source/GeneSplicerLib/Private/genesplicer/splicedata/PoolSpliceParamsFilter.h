// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/splicedata/PoolSpliceParamsImpl.h"
#include "genesplicer/splicedata/SpliceDataImpl.h"
#include "genesplicer/splicedata/rawgenes/RawGenes.h"
#include "genesplicer/types/VariableWidthMatrix.h"

namespace gs4 {

template<typename TFilter>
VariableWidthMatrix<const PoolSpliceParamsImpl*> filterPoolSpliceParamsPerMesh(const SpliceDataInterface* spliceData,
                                                                               TFilter predicate,
                                                                               MemoryResource* memRes) {
    const auto& baseArchetype = spliceData->getBaseArchetype();
    VariableWidthMatrix<const PoolSpliceParamsImpl*> poolParams{memRes};
    auto pools = spliceData->getAllPoolParams();
    const std::uint16_t meshCount = baseArchetype.getMeshCount();
    poolParams.reserve(meshCount, meshCount * pools.size());

    for (std::uint16_t meshIdx = 0; meshIdx < meshCount; meshIdx++) {
        poolParams.appendRow(0u);
        for (auto pool : pools) {
            if (pool->isMeshEnabled(meshIdx) && predicate(baseArchetype, pool, meshIdx)) {
                poolParams.append(meshIdx, pool);
            }
        }
    }
    poolParams.shrinkToFit();
    return poolParams;
}

}  // namespace gs4
