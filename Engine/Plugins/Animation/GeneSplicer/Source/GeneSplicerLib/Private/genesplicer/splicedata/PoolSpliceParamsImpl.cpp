// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/PoolSpliceParamsImpl.h"

#include "genesplicer/Macros.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/GenePool.h"
#include "genesplicer/splicedata/PoolSpliceParams.h"
#include "genesplicer/splicedata/SpliceWeights.h"
#include "genesplicer/splicedata/VertexWeights.h"
#include "genesplicer/splicedata/genepool/GenePoolInterface.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/Matrix.h"
#include "genesplicer/types/PImplExtractor.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cassert>
#include <cstdint>
#include <limits>
#include <numeric>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

PoolSpliceParams::~PoolSpliceParams() = default;

const StatusCode PoolSpliceParams::GenePoolIncompatible{1004, "GenePool is not compatible with RegionAffiliation, %s.\n"};
const StatusCode PoolSpliceParams::WeightsInvalid{1005, "Incorrect weight count, expected %zu.\n"};

#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wglobal-constructors"
sc::StatusProvider PoolSpliceParamsImpl::status{PoolSpliceParams::GenePoolIncompatible, PoolSpliceParams::WeightsInvalid};
#pragma clang diagnostic pop

bool PoolSpliceParamsImpl::compatible(const RegionAffiliationReader* regionAffiliationReader,
                                      const GenePool* genePool_,
                                      MemoryResource* memRes) {

    status.reset();
    auto genePool = PImplExtractor<GenePool>::get(genePool_);
    if (genePool == nullptr) {
        status.set(PoolSpliceParams::GenePoolIncompatible, "GenePool is std::moved and thus nullptr");
        return false;
    }

    auto addReport = [memRes](String<char>& message, std::uint32_t genePoolCount, std::uint32_t rafCount) {
        message += String<char>{" GenePool has ", memRes} + String<char>{std::to_string(genePoolCount).c_str(), memRes} +
                   String<char>{", RegionAffilation has ", memRes} + String<char>{std::to_string(rafCount).c_str(), memRes};
    };
    const std::uint16_t genePoolMeshCount = genePool->getMeshCount();
    const std::uint16_t rafMeshCount = regionAffiliationReader->getMeshCount();
    if (genePoolMeshCount != rafMeshCount) {
        String<char> errorMessage{"Mesh count:", memRes};
        addReport(errorMessage, genePoolMeshCount, rafMeshCount);
        status.set(PoolSpliceParams::GenePoolIncompatible, errorMessage.c_str());
        return false;
    }
    for (std::uint16_t meshIdx = 0; meshIdx < genePoolMeshCount; meshIdx++) {
        const std::uint32_t genePoolVertexCount = genePool->getVertexCount(meshIdx);
        const std::uint32_t rafVertexCount = regionAffiliationReader->getVertexCount(meshIdx);

        if (genePoolVertexCount != rafVertexCount) {
            String<char> errorMessage =
                String<char>{"Vertex count at mesh index ", memRes} + std::to_string(meshIdx).c_str() + ":";
            addReport(errorMessage, genePoolVertexCount, rafVertexCount);
            status.set(PoolSpliceParams::GenePoolIncompatible, errorMessage.c_str());
            return false;
        }
    }
    const std::uint16_t genePoolJointCount = genePool->getJointCount();
    const std::uint16_t rafJointCount = regionAffiliationReader->getJointCount();
    if (genePoolJointCount != rafJointCount) {
        String<char> errorMessage{"Joint count:", memRes};
        addReport(errorMessage, genePoolJointCount, rafJointCount);
        status.set(PoolSpliceParams::GenePoolIncompatible, errorMessage.c_str());
        return false;
    }
    return true;
}

PoolSpliceParamsImpl* PoolSpliceParamsImpl::create(const RegionAffiliationReader* regionAffiliationReader,
                                                   const GenePool* genePool,
                                                   MemoryResource* memRes) {
    PolyAllocator<PoolSpliceParamsImpl> alloc{memRes};
    if (compatible(regionAffiliationReader, genePool, memRes)) {
        return alloc.newObject(regionAffiliationReader, PImplExtractor<GenePool>::get(genePool), memRes);
    }
    return nullptr;
}

void PoolSpliceParamsImpl::destroy(PoolSpliceParamsImpl* instance) {
    PolyAllocator<PoolSpliceParamsImpl> alloc{instance->memRes};
    alloc.deleteObject(instance);
}

PoolSpliceParamsImpl::PoolSpliceParamsImpl(const RegionAffiliationReader* regionAffiliationReader_,
                                           const GenePoolInterface* genePool_,
                                           MemoryResource* memRes_) :
    memRes{memRes_},
    genePool{genePool_},
    spliceWeights{genePool->getDNACount(), regionAffiliationReader_->getRegionCount(), memRes_},
    vertexWeights{regionAffiliationReader_, memRes_},
    jointWeights{regionAffiliationReader_, memRes_},
    dnaIndices{genePool->getDNACount(), {}, memRes},
    meshIndices{regionAffiliationReader_->getMeshCount(), {}, memRes_},
    jointBehaviorOutputIndexTargets{genePool->getJointCount(), memRes},
    scale{1.0f} {
    std::iota(meshIndices.begin(), meshIndices.end(), static_cast<std::uint16_t>(0u));
    std::iota(dnaIndices.begin(), dnaIndices.end(), static_cast<std::uint16_t>(0u));
}

ConstArrayView<float> PoolSpliceParamsImpl::getSpliceWeights(std::uint16_t dnaIndex) const {
    return spliceWeights.get(dnaIndex);
}

void PoolSpliceParamsImpl::setSpliceWeights(std::uint16_t dnaStartIndex, const float* weights, std::uint32_t count) {
    clearAll();
    spliceWeights.set(dnaStartIndex, ConstArrayView<float>{weights, count});
}

/**
        @brief [meshIdx][dnaIdx][vertexPositionIdx]
*/
const Vector<TiledMatrix2D<16u>>& PoolSpliceParamsImpl::getVertexWeightsData() const {
    if (vertexWeights.empty()) {
        vertexWeights.compute(spliceWeights, getMeshIndices(), getDNAIndices());
    }
    return vertexWeights.getData();
}

const TiledMatrix2D<16u>& PoolSpliceParamsImpl::getJointWeightsData() const {
    if (jointWeights.empty()) {
        jointWeights.compute(spliceWeights, getDNAIndices());
    }
    return jointWeights.getData();
}

const Matrix2D<float>& PoolSpliceParamsImpl::getSpliceWeightsData() const {
    return spliceWeights.getData();
}

void PoolSpliceParamsImpl::setMeshFilter(const std::uint16_t* meshIndices_, std::uint16_t count) {
    meshIndices.assign(meshIndices_, meshIndices_ + count);
    clearAll();
}

void PoolSpliceParamsImpl::setDNAFilter(const std::uint16_t* dnaIndices_, std::uint16_t count) {
    dnaIndices.assign(dnaIndices_, dnaIndices_ + count);
    std::sort(dnaIndices.begin(), dnaIndices.end());
    clearAll();
}

void PoolSpliceParamsImpl::clearFilters() {
    std::uint16_t meshCount = genePool->getMeshCount();
    meshIndices.resize(meshCount);
    std::iota(meshIndices.begin(), meshIndices.end(), std::uint16_t{});
    std::uint16_t dnaCount = genePool->getDNACount();
    dnaIndices.resize(dnaCount);
    std::iota(dnaIndices.begin(), dnaIndices.end(), std::uint16_t{});
    clearAll();
}

ConstArrayView<std::uint16_t> PoolSpliceParamsImpl::getMeshIndices() const {
    return ConstArrayView<std::uint16_t>{meshIndices};
}

ConstArrayView<std::uint16_t> PoolSpliceParamsImpl::getDNAIndices() const {
    return {dnaIndices.data(), dnaIndices.size()};
}

bool PoolSpliceParamsImpl::isMeshEnabled(std::uint16_t meshIndex) const {
    return std::find(meshIndices.begin(), meshIndices.end(), meshIndex) != meshIndices.end();
}

const GenePoolInterface* PoolSpliceParamsImpl::getGenePool() const {
    return genePool;
}

void PoolSpliceParamsImpl::generateJointBehaviorOutputIndexTargetOffsets(const RawGenes& baseArchetype) {
    const auto& jointBehaviorPoolOutput = genePool->getJointBehaviorOutputIndices();
    auto baseArchJointBehavior = baseArchetype.getJointGroups();
    const auto jointGroupCount = static_cast<std::uint16_t>(baseArchJointBehavior.size());
    if (jointBehaviorPoolOutput.rowCount() == jointGroupCount) {
        for (std::uint16_t jointGroupIdx = 0u; jointGroupIdx < jointGroupCount; jointGroupIdx++) {
            jointBehaviorOutputIndexTargets.mapJointGroup(
                jointBehaviorPoolOutput[jointGroupIdx],
                ConstArrayView<std::uint16_t>{baseArchJointBehavior[jointGroupIdx].outputIndices});
        }
    }
}

const Matrix2D<std::uint8_t>& PoolSpliceParamsImpl::getJointBehaviorOutputIndexTargetOffsets() const {
    return jointBehaviorOutputIndexTargets.get();
}

void PoolSpliceParamsImpl::setScale(float scale_) {
    scale = scale_;
}

float PoolSpliceParamsImpl::getScale() const {
    return scale;
}

void PoolSpliceParamsImpl::clearAll() {
    vertexWeights.clear();
    jointWeights.clear();
}

void PoolSpliceParamsImpl::cacheAll() {
    vertexWeights.compute(spliceWeights, getMeshIndices(), getDNAIndices());
    jointWeights.compute(spliceWeights, getDNAIndices());
}

std::uint16_t PoolSpliceParamsImpl::getDNACount() const {
    return spliceWeights.getDNACount();
}

std::uint16_t PoolSpliceParamsImpl::getRegionCount() const {
    return spliceWeights.getRegionCount();
}

}  // namespace gs4
