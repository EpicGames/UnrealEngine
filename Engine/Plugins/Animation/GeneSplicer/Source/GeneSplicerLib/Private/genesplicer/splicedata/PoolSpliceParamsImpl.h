// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/JointWeights.h"
#include "genesplicer/splicedata/PoolSpliceParams.h"
#include "genesplicer/splicedata/SpliceWeights.h"
#include "genesplicer/splicedata/VertexWeights.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"
#include "genesplicer/splicedata/genepool/OutputIndexTargetOffsets.h"
#include "genesplicer/splicedata/rawgenes/RawGenes.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/Matrix.h"
#include "genesplicer/types/PImplExtractor.h"

#include <cstddef>
#include <cstdint>

namespace gs4 {

class PoolSpliceParamsImpl : public PoolSpliceParams {

public:
    static PoolSpliceParamsImpl* create(const RegionAffiliationReader* regionAffiliationReader,
                                        const GenePool* genePool,
                                        MemoryResource* memRes_);
    static void destroy(PoolSpliceParamsImpl* instance);

private:
    static bool compatible(const RegionAffiliationReader* regionAffiliationReader,
                           const GenePool* genePool,
                           MemoryResource* memRes_);

public:
    PoolSpliceParamsImpl(const RegionAffiliationReader* regionAffiliationReader,
                         const GenePoolInterface* genePool,
                         MemoryResource* memRes_);

    void setSpliceWeights(std::uint16_t dnaStartIndex, const float* weights, std::uint32_t count) override;
    void setMeshFilter(const std::uint16_t* meshIndices, std::uint16_t count) override;
    void setDNAFilter(const std::uint16_t* dnaIndices, std::uint16_t count) override;
    void clearFilters() override;
    void setScale(float scale) override;

    void generateJointBehaviorOutputIndexTargetOffsets(const RawGenes& baseArchetype);
    const Matrix2D<std::uint8_t>& getJointBehaviorOutputIndexTargetOffsets() const;

    ConstArrayView<float> getSpliceWeights(std::uint16_t dnaIndex) const;
    ConstArrayView<std::uint16_t> getMeshIndices() const;
    ConstArrayView<std::uint16_t> getDNAIndices() const;

    bool isMeshEnabled(std::uint16_t meshIndex) const;

    const Matrix2D<float>& getSpliceWeightsData() const;
    const Vector<TiledMatrix2D<16u>>& getVertexWeightsData() const;
    const TiledMatrix2D<16u>& getJointWeightsData() const;
    const GenePoolInterface* getGenePool() const;
    float getScale() const;
    std::uint16_t getDNACount() const override;
    std::uint16_t getRegionCount() const override;

    void cacheAll();
    void clearAll();

private:
    static sc::StatusProvider status;
    MemoryResource* memRes;
    const GenePoolInterface* genePool;
    SpliceWeights spliceWeights;
    mutable VertexWeights vertexWeights;
    mutable JointWeights jointWeights;
    Vector<std::uint16_t> dnaIndices;
    Vector<std::uint16_t> meshIndices;
    OutputIndexTargetOffsets jointBehaviorOutputIndexTargets;
    float scale;
};

}  // namespace gs4

namespace pma {

template<>
struct DefaultInstanceCreator<gs4::PoolSpliceParamsImpl> {
    using type = FactoryCreate<gs4::PoolSpliceParamsImpl>;
};

template<>
struct DefaultInstanceDestroyer<gs4::PoolSpliceParamsImpl> {
    using type = FactoryDestroy<gs4::PoolSpliceParamsImpl>;
};

}  // namespace pma
