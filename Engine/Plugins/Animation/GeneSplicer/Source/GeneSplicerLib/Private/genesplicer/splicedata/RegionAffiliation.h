// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/Macros.h"
#include "genesplicer/TypeDefs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

template<std::uint16_t N = 1u>
struct RegionAffiliation {
public:
    struct IndexValue {
        float value;
        std::uint16_t index;
    };

public:
    static constexpr std::size_t firstNSize() {
        return N;
    }

    explicit RegionAffiliation(MemoryResource* memRes) :
        rest{memRes},
        firstN{} {
    }

    RegionAffiliation(ConstArrayView<std::uint16_t> regionIndices, ConstArrayView<float> values, MemoryResource* memRes) :
        rest{memRes},
        firstN{} {

        assert(regionIndices.size() == values.size());
        ASSUME_TRUE(regionIndices.size() == values.size());

        const std::size_t firstNLimit = std::min(firstNSize(), regionIndices.size());
#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 28020)
#endif
        for (std::size_t i = 0u; i < firstNLimit; i++) {
            firstN[i] = IndexValue{values[i], regionIndices[i]};
        }
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

        rest.reserve(regionIndices.size() - firstNLimit);
        for (std::size_t i = firstNLimit; i < regionIndices.size(); i++) {
            rest.push_back(IndexValue{values[i], regionIndices[i]});
        }
    }

    float totalWeightAcrossRegions(ConstArrayView<float> weightsPerRegion) const {
        float weight = 0;
#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 28020)
#endif
        for (std::size_t i = 0u; i < firstNSize(); i++) {
            weight += firstN[i].value * weightsPerRegion[firstN[i].index];
        }
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

        for (std::size_t i = 0; i < rest.size(); i++) {
            weight += rest[i].value * weightsPerRegion[rest[i].index];
        }
        return weight;
    }

public:
    Vector<IndexValue> rest;
    std::array<IndexValue, N> firstN;
};

}  // namespace gs4
