// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/SpliceDataImpl.h"

#include "genesplicer/splicedata/GenePool.h"
#include "genesplicer/splicedata/PoolSpliceParams.h"
#include "genesplicer/splicedata/PoolSpliceParamsImpl.h"
#include "genesplicer/splicedata/SpliceData.h"
#include "genesplicer/splicedata/genepool/GenePoolInterface.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/PImplExtractor.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cassert>
#include <cstdint>
#include <iterator>
#include <limits>
#include <numeric>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

namespace {
StringView getStringView(const char* array) {
    auto end_array = array;
    while (*end_array != '\0') {
        end_array++;
    }
    return StringView{array, static_cast<std::size_t>(end_array - array)};
}

}  // namespace

SpliceData::~SpliceData() = default;

SpliceData::SpliceData(SpliceData&& rhs) = default;
SpliceData& SpliceData::operator=(SpliceData&& rhs) = default;

SpliceData::SpliceData(MemoryResource* memRes) :
    pImpl{makePImpl<Impl>(memRes)} {
}

void SpliceData::registerGenePool(const char* name, const RegionAffiliationReader* reader, const GenePool* genePool) {
    pImpl->registerGenePool(getStringView(name), reader, genePool);
}

void SpliceData::unregisterGenePool(const char* name) {
    pImpl->unregisterGenePool(getStringView(name));
}

PoolSpliceParams* SpliceData::getPoolParams(const char* name) {
    return pImpl->getPoolParams(getStringView(name));
}

void SpliceData::setBaseArchetype(const dna::Reader* baseArchetype) {
    pImpl->setBaseArchetype(baseArchetype);
}

template struct PImplExtractor<SpliceData>;

SpliceData::Impl* SpliceData::Impl::create(MemoryResource* memRes) {
    PolyAllocator<Impl> alloc{memRes};
    return alloc.newObject(memRes);
}

void SpliceData::Impl::destroy(SpliceData::Impl* instance) {
    PolyAllocator<Impl> alloc{instance->memRes};
    alloc.deleteObject(instance);
}

SpliceData::Impl::Impl(MemoryResource* memRes_) :
    memRes{memRes_},
    pools{memRes},
    baseArchetype{memRes} {
}

void SpliceData::Impl::registerGenePool(StringView nameView, const RegionAffiliationReader* reader, const GenePool* genePool) {
    auto poolSpliceParams = makeScoped<PoolSpliceParamsImpl>(reader, genePool, memRes);
    if (poolSpliceParams) {
        std::string poolName{nameView.c_str()};
        pools[poolName] = std::move(poolSpliceParams);
        accustimizePoolsAndBaseArch();
    }
}

void SpliceData::Impl::unregisterGenePool(StringView nameView) {
    std::string poolName{nameView.c_str()};
    auto it = pools.find(poolName);
    if (it != pools.end()) {
        pools.erase(it);
    }
}

PoolSpliceParams* SpliceData::Impl::getPoolParams(StringView nameView) {
    std::string poolName{nameView.c_str()};
    auto it = pools.find(poolName);
    if (it != pools.end()) {
        return it->second.get();
    }
    return nullptr;
}

void SpliceData::Impl::setBaseArchetype(const dna::Reader* baseArchetypeReader) {
    baseArchetype.set(baseArchetypeReader);
    accustimizePoolsAndBaseArch();
}

void SpliceData::Impl::accustimizePoolsAndBaseArch() {
    for (const auto& it : pools) {
        baseArchetype.accustomize(it.second->getGenePool());
    }
    for (const auto& it : pools) {
        it.second->generateJointBehaviorOutputIndexTargetOffsets(baseArchetype);
    }
}

Vector<const PoolSpliceParamsImpl*> SpliceData::Impl::getAllPoolParams() const {
    Vector<const PoolSpliceParamsImpl*> poolParams{memRes};
    poolParams.reserve(pools.size());
    for (const auto& it : pools) {
        poolParams.push_back(it.second.get());
    }
    poolParams.shrink_to_fit();
    return poolParams;
}

const RawGenes& SpliceData::Impl::getBaseArchetype() const {
    return baseArchetype;
}

void SpliceData::Impl::cacheAll() {
    for (auto& it : pools) {
        it.second->cacheAll();
    }
}

}  // namespace gs4
