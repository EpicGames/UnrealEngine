// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/splicedata/SpliceData.h"

#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/PoolSpliceParams.h"
#include "genesplicer/splicedata/PoolSpliceParamsImpl.h"
#include "genesplicer/splicedata/rawgenes/RawGenes.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/PImplExtractor.h"

#include <cstddef>
#include <cstdint>
#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <limits>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

using SpliceDataInterface = PImplExtractor<SpliceData>::impl_type;
class SpliceData::Impl {
public:
    using pool_name_type = String<char>;

public:
    static Impl* create(MemoryResource* memRes_);
    static void destroy(Impl* instance);

public:
    Impl(MemoryResource* memRes_);

    void registerGenePool(StringView name, const RegionAffiliationReader* reader, const GenePool* genePool);
    void unregisterGenePool(StringView name);

    PoolSpliceParams* getPoolParams(StringView name);
    Vector<const PoolSpliceParamsImpl*> getAllPoolParams() const;

    void setBaseArchetype(const dna::Reader* baseArchetype);
    const RawGenes& getBaseArchetype() const;

    void cacheAll();

private:
    void accustimizePoolsAndBaseArch();

private:
    MemoryResource* memRes;
    UnorderedMap<std::string, ScopedPtr<PoolSpliceParamsImpl>> pools;
    RawGenes baseArchetype;
};

}  // namespace gs4
