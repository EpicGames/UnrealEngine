// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/SpliceWeights.h"

#include "genesplicer/types/Aliases.h"
#include "genesplicer/utils/IterTools.h"

#include <cassert>
#include <cstddef>
#include <cstdint>

namespace gs4 {

SpliceWeights::SpliceWeights(std::uint16_t dnaCount, std::uint16_t regionCount, MemoryResource* memRes) :
    weights{dnaCount, regionCount, memRes} {
}

ConstArrayView<float> SpliceWeights::get(std::uint16_t dnaIndex) const {
    assert(dnaIndex < weights.columnCount());
    return weights[dnaIndex];
}

const Matrix2D<float>& SpliceWeights::getData() const {
    return weights;
}

std::uint16_t SpliceWeights::getDNACount() const {
    return static_cast<std::uint16_t>(weights.rowCount());
}

std::uint16_t SpliceWeights::getRegionCount() const {
    return static_cast<std::uint16_t>(weights.columnCount());
}

void SpliceWeights::set(std::uint16_t dnaStartIndex, ConstArrayView<float> weights_) {
    assert(dnaStartIndex + weights_.size() / getRegionCount() <= getDNACount());
    float* destination = weights.data() + dnaStartIndex * getRegionCount();
    safeCopy(weights_.data(), weights_.data() + weights_.size(), destination, weights_.size());
}

}  // namespace gs4
