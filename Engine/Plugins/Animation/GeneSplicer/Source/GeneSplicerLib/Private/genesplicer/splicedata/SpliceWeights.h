// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/Matrix.h"

#include <cstddef>
#include <cstdint>

namespace gs4 {

class SpliceWeights {
public:
    explicit SpliceWeights(std::uint16_t dnaCount, std::uint16_t regionCount, MemoryResource* memRes);
    void set(std::uint16_t dnaStartIndex, ConstArrayView<float> weights);

    ConstArrayView<float> get(std::uint16_t dnaIdx) const;
    const Matrix2D<float>& getData() const;
    std::uint16_t getDNACount() const;
    std::uint16_t getRegionCount() const;

private:
    Matrix2D<float> weights;
};

}  // namespace gs4
