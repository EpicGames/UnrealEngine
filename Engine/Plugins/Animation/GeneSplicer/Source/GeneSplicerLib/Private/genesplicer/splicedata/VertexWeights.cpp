// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/VertexWeights.h"

#include "genesplicer/Macros.h"
#include "genesplicer/splicedata/RegionAffiliation.h"
#include "genesplicer/splicedata/SpliceWeights.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"
#include "genesplicer/types/Aliases.h"

#include <cassert>
#include <cstdint>

namespace gs4 {

VertexWeights::VertexWeights(const VertexRegionAffiliationReader* regionAffiliationReader, MemoryResource* memRes) :
    weights{memRes},
    regionAffiliations{memRes} {
    std::uint16_t meshCount = regionAffiliationReader->getMeshCount();

    for (std::uint16_t meshIdx = 0; meshIdx < meshCount; meshIdx++) {
        const std::uint32_t vertexCount = regionAffiliationReader->getVertexCount(meshIdx);
        regionAffiliations.appendRow(vertexCount, RegionAffiliation<>{memRes});
        auto mesh = regionAffiliations[meshIdx];
        for (std::uint32_t vtxIdx = 0; vtxIdx < vertexCount; vtxIdx++) {
            auto indices = regionAffiliationReader->getVertexRegionIndices(meshIdx, vtxIdx);
            auto values = regionAffiliationReader->getVertexRegionAffiliation(meshIdx, vtxIdx);
            mesh[vtxIdx] = RegionAffiliation<>{indices, values, memRes};
        }
    }
}

bool VertexWeights::empty() const {
    return weights.empty();
}

void VertexWeights::clear() {
    weights.clear();
}

void VertexWeights::compute(const SpliceWeights& spliceWeights,
                            ConstArrayView<std::uint16_t> meshIndices,
                            ConstArrayView<std::uint16_t> dnaIndices) {
    clear();
    auto meshCount = static_cast<std::uint16_t>(regionAffiliations.rowCount());
    weights.resize(meshCount);
    const auto spliceWeightsData = spliceWeights.getData();
    const constexpr auto blockSize = XYZTiledMatrix<16u>::value_type::size();
    const std::uint16_t dnaCount = spliceWeights.getDNACount();
    auto memRes = weights.get_allocator().getMemoryResource();

    for (std::uint16_t meshIndex : meshIndices) {

        auto meshRAF = regionAffiliations[meshIndex];
        const auto vertexCount = static_cast<std::uint32_t>(meshRAF.size());
        const auto endIdx = vertexCount / blockSize;
        const auto remainder = vertexCount % blockSize;
        const auto blockCount = endIdx + (1u && remainder);

        auto& mesh = weights[meshIndex];
        mesh = TiledMatrix2D<16u>{blockCount, dnaCount, memRes};

        for (std::uint32_t blockIdx = 0u; blockIdx < endIdx; ++blockIdx) {

            for (std::uint16_t dnaIdx : dnaIndices) {
                auto vtxIdx = blockIdx * blockSize;
                const auto spliceWeightsPerRegion = spliceWeightsData[dnaIdx];
                auto& block = mesh[blockIdx][dnaIdx];

                for (std::size_t i = 0; i < blockSize; i += 4u) {
                    const auto targetIndex = static_cast<std::uint32_t>(vtxIdx + i);
                    block[i] = meshRAF[targetIndex + 0u].totalWeightAcrossRegions(spliceWeightsPerRegion);
                    block[i + 1u] = meshRAF[targetIndex + 1u].totalWeightAcrossRegions(spliceWeightsPerRegion);
                    block[i + 2u] = meshRAF[targetIndex + 2u].totalWeightAcrossRegions(spliceWeightsPerRegion);
                    block[i + 3u] = meshRAF[targetIndex + 3u].totalWeightAcrossRegions(spliceWeightsPerRegion);
                }
            }
        }
        auto vtxIdx = vertexCount - remainder;
        for (std::uint16_t i = 0; i < remainder; i++) {
            for (std::uint16_t dnaIdx : dnaIndices) {
                mesh[blockCount - 1u][dnaIdx][i] = meshRAF[vtxIdx].totalWeightAcrossRegions(spliceWeightsData[dnaIdx]);
            }
            vtxIdx++;
        }
    }
}

/**
    @brief [meshIdx][dnaIdx][vertexPositionIdx]
    @note
*/
const Vector<TiledMatrix2D<16u>>& VertexWeights::getData() const {
    return weights;
}

}  // namespace gs4
