// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/RegionAffiliation.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#include <cstdint>

namespace gs4 {

class SpliceWeights;

class VertexWeights {

public:
    explicit VertexWeights(const VertexRegionAffiliationReader* regionAffiliationReader, MemoryResource* memRes);
    void compute(const SpliceWeights& spliceWeights,
                 ConstArrayView<std::uint16_t> meshIndices,
                 ConstArrayView<std::uint16_t> dnaIndices);
    void clear();
    bool empty() const;
    const Vector<TiledMatrix2D<16u>>& getData() const;

private:
    Vector<TiledMatrix2D<16u>> weights;
    VariableWidthMatrix<RegionAffiliation<>> regionAffiliations;
};

}  // namespace gs4
