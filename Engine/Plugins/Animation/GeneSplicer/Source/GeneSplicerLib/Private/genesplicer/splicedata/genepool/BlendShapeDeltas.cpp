// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/genepool/BlendShapeDeltas.h"

#include <cstddef>
#include <limits>

namespace gs4 {

template<std::uint16_t BlockSize>
class BlendShapeBlockFactory {
public:
    BlendShapeBlockFactory(const Reader* dna, std::uint16_t meshIndex, std::uint16_t blendShapeIndex) :
        xs{dna->getBlendShapeTargetDeltaXs(meshIndex, blendShapeIndex)},
        ys{dna->getBlendShapeTargetDeltaYs(meshIndex, blendShapeIndex)},
        zs{dna->getBlendShapeTargetDeltaZs(meshIndex, blendShapeIndex)},
        indices{dna->getBlendShapeTargetVertexIndices(meshIndex, blendShapeIndex)},
        i{} {
    }

    std::uint32_t vertexIndex() {
        if (i < indices.size()) {
            return indices[i];
        }
        return std::numeric_limits<std::uint32_t>::max();
    }

    void advanceTo(std::uint32_t targetIndex) {
        while (vertexIndex() < targetIndex) {
            ++i;
        }
    }

    XYZBlock<BlockSize> makeBlock() {
        XYZBlock<BlockSize> block{};
        const std::uint32_t blockIndex = vertexIndex() / BlockSize;
        const std::uint32_t minVtxIndex = blockIndex * BlockSize;
        const std::uint32_t maxVtxIndex = (blockIndex + 1u) * BlockSize;
        while (i < indices.size() && indices[i] < maxVtxIndex && indices[i] >= minVtxIndex) {
            const std::uint32_t j = indices[i] - minVtxIndex;
            block.Xs[j] = xs[i];
            block.Ys[j] = ys[i];
            block.Zs[j] = zs[i];
            i++;
        }
        return block;
    }

private:
    ConstArrayView<float> xs;
    ConstArrayView<float> ys;
    ConstArrayView<float> zs;
    ConstArrayView<std::uint32_t> indices;
    std::uint32_t i;
};

template<std::uint16_t BlockSize>
BlendShapeDeltas<BlockSize> BlendShapeDeltasFactory<BlockSize>::operator()(const Reader* deltaArchetype,
                                                                           ConstArrayView<const Reader*> dnas,
                                                                           MemoryResource* memRes) {
    BlendShapeDeltas<BlockSize> deltas{memRes};
    const std::uint16_t meshCount = deltaArchetype->getMeshCount();
    std::uint32_t totalBlendShapeCount = 0;
    std::size_t totalVertexCount = 0u;
    for (std::uint16_t meshIndex = 0u; meshIndex < meshCount; meshIndex++) {
        const std::uint16_t bsCount = deltaArchetype->getBlendShapeTargetCount(meshIndex);
        totalBlendShapeCount += bsCount;
        for (std::uint16_t bsIndex = 0u; bsIndex < bsCount; bsIndex++) {
            totalVertexCount += deltaArchetype->getBlendShapeTargetVertexIndices(meshIndex, bsIndex).size();
        }
    }
    if (totalVertexCount == 0u) {
        return 0;
    }
    const std::size_t approxBucketCount = totalVertexCount / BlockSize;
    const std::size_t approxBlockCount = approxBucketCount * (dnas.size());
    deltas.bucketOffsets.reserve(meshCount, totalBlendShapeCount);
    deltas.bucketVertexIndices.reserve(approxBucketCount);
    deltas.archBlocks.reserve(approxBucketCount);
    deltas.bucketDNABlockOffsets.reserve(approxBucketCount);
    deltas.dnaIndices.reserve(approxBlockCount);
    deltas.dnaBlocks.reserve(approxBlockCount);

    const auto dnaCount = static_cast<std::uint16_t>(dnas.size());

    std::size_t blocksOffset = 0u;
    for (std::uint16_t meshIdx = 0u; meshIdx < meshCount; meshIdx++) {
        const std::uint32_t vertexCount = deltaArchetype->getVertexPositionCount(meshIdx);
        const std::uint16_t bsCount = deltaArchetype->getBlendShapeTargetCount(meshIdx);
        const auto maxBlockCount = vertexCount / BlockSize + (1u && vertexCount % BlockSize);
        deltas.bucketOffsets.appendRow(bsCount);

        for (std::uint16_t bsIdx = 0; bsIdx < bsCount; bsIdx++) {
            deltas.bucketOffsets[meshIdx][bsIdx] = deltas.archBlocks.size();
            BlendShapeBlockFactory<BlockSize> archBlendShapeBlockFactory{deltaArchetype, meshIdx, bsIdx};
            Vector<BlendShapeBlockFactory<BlockSize>> dnaBlendShapeBlockFactories{memRes};
            dnaBlendShapeBlockFactories.reserve(dnaCount);
            for (std::size_t dnaIdx = 0; dnaIdx < dnaCount; dnaIdx++) {
                dnaBlendShapeBlockFactories.emplace_back(dnas[dnaIdx], meshIdx, bsIdx);
            }

            for (std::uint32_t bucketIdx = 0u; bucketIdx < maxBlockCount; bucketIdx++) {
                const std::uint32_t minVtxIndex = bucketIdx * BlockSize;
                const std::uint32_t maxVtxIndex = (bucketIdx + 1u) * BlockSize;
                std::uint16_t blocksAdded = 0u;

                for (std::uint16_t dnaIdx = 0; dnaIdx < dnaCount; dnaIdx++) {
                    dnaBlendShapeBlockFactories[dnaIdx].advanceTo(minVtxIndex);
                    if (dnaBlendShapeBlockFactories[dnaIdx].vertexIndex() < maxVtxIndex) {
                        deltas.dnaBlocks.push_back(dnaBlendShapeBlockFactories[dnaIdx].makeBlock());
                        deltas.dnaIndices.push_back(dnaIdx);
                        blocksAdded++;
                    }
                }

                archBlendShapeBlockFactory.advanceTo(minVtxIndex);
                if (archBlendShapeBlockFactory.vertexIndex() < maxVtxIndex) {
                    deltas.archBlocks.push_back(archBlendShapeBlockFactory.makeBlock());
                    deltas.bucketDNABlockOffsets.push_back(blocksOffset);
                    deltas.bucketVertexIndices.push_back(minVtxIndex);
                    blocksOffset += blocksAdded;
                } else if (blocksAdded > 0) {
                    // this should never happen as archetype is created from DNAs provided it must contain vertex delta
                    // indices of any DNA
                    deltas.archBlocks.push_back({});
                    deltas.bucketDNABlockOffsets.push_back(blocksOffset);
                    deltas.bucketVertexIndices.push_back(minVtxIndex);
                    blocksOffset += blocksAdded;
                }
            }
        }
        deltas.bucketOffsets.append(deltas.bucketOffsets.rowCount() - 1u, deltas.archBlocks.size());
        deltas.shrinkToFit();
    }
    deltas.bucketDNABlockOffsets.push_back(blocksOffset);
    return deltas;
}

template struct BlendShapeDeltasFactory<4u>;

}  // namespace gs4
