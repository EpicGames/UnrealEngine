// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#include <cstddef>
#include <cstdint>

namespace gs4 {

// Each bucket represents 4 consecutive vertex indices. For each bucket we want to have:
// 1. Vertex Index, we only need the first one we can implicitly deduce other 3 (v,v+1,v+2,v+3)
// 2. Values from each DNA that has at least one of 4 vertex indices represented by the bucket.
// 3. DNA indices that correlate each individual block with its corresponding DNA.
// 4. Arch values for the vertex indices

// In order to reduce the size of BlendShapePool and increase cache efficiency we keep each of the above in corresponding vector.
// But we'll need two more vectors:
// 1. bucketOffsets will tell us the offset of the first bucket in each Blend Shape.
// 2. bucketDNABlockOffset indicates offset within the dnaBlocks vector for the initial dnaBlock in each bucket.
// This aligns with the offset of the first dnaIndex within a bucket in the dnaIndices vector.
template<std::uint16_t BlockSize>
struct BlendShapeDeltas {

    BlendShapeDeltas(MemoryResource* memRes) :
        bucketOffsets{memRes},
        bucketVertexIndices{memRes},
        bucketDNABlockOffsets{memRes},
        archBlocks{memRes},
        dnaIndices{memRes},
        dnaBlocks{memRes} {
    }

    std::uint16_t getBlockSize() const {
        return BlockSize;
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(bucketOffsets, bucketVertexIndices, bucketDNABlockOffsets, archBlocks, dnaIndices, dnaBlocks);
    }

    void shrinkToFit() {
        bucketOffsets.shrinkToFit();
        bucketVertexIndices.shrink_to_fit();
        bucketDNABlockOffsets.shrink_to_fit();
        archBlocks.shrink_to_fit();
        dnaIndices.shrink_to_fit();
        dnaBlocks.shrink_to_fit();
    }

    // [meshIndex][bsIndex]
    VariableWidthMatrix<std::size_t> bucketOffsets;

    // [bucketOffset]
    Vector<std::uint32_t> bucketVertexIndices;
    Vector<std::size_t> bucketDNABlockOffsets;
    AlignedVector<XYZBlock<BlockSize>> archBlocks;

    // [blockOffset]
    Vector<std::uint16_t> dnaIndices;
    AlignedVector<XYZBlock<BlockSize>> dnaBlocks;
};

template<std::uint16_t BlockSize>
struct BlendShapeDeltasFactory {
    BlendShapeDeltas<BlockSize> operator()(const Reader* deltaArchetype,
                                           ConstArrayView<const Reader*> dnas,
                                           MemoryResource* memRes);
};

}  // namespace gs4
