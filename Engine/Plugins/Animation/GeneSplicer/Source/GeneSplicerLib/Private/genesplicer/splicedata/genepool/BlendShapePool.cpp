// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/genepool/BlendShapePool.h"

#include "genesplicer/CalculationType.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/types/Block.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/Matrix.h"
#include "genesplicer/types/PImplExtractor.h"
#include "genesplicer/types/Vec3.h"

#include <cstdint>

namespace gs4 {

BlendShapePool::BlendShapePool(MemoryResource* memRes_) :
    vertexIndices{memRes_},
    deltas{memRes_} {
}

BlendShapePool::BlendShapePool(const Reader* deltaArchetype, ConstArrayView<const Reader*> dnas, MemoryResource* memRes_) :
    vertexIndices{memRes_},
    deltas{BlendShapeDeltasFactory<4>()(deltaArchetype, dnas, memRes_)} {
    fillVertexIndices(deltaArchetype, dnas, memRes_);
}

void BlendShapePool::fillVertexIndices(const Reader* deltaArchetype, ConstArrayView<const Reader*> dnas, MemoryResource* memRes) {
    const std::uint16_t meshCount = deltaArchetype->getMeshCount();
    const auto dnaCount = static_cast<std::uint16_t>(dnas.size());
    vertexIndices.resize(meshCount, VariableWidthMatrix<std::uint32_t>{memRes});
    for (std::uint16_t meshIdx = 0u; meshIdx < meshCount; meshIdx++) {
        const std::uint32_t vertexCount = deltaArchetype->getVertexPositionCount(meshIdx);
        const std::uint16_t bsCount = deltaArchetype->getBlendShapeTargetCount(meshIdx);
        vertexIndices[meshIdx].reserve(bsCount, bsCount * vertexCount);
        for (std::uint16_t bsIdx = 0; bsIdx < bsCount; bsIdx++) {
            Vector<char> containsVertexIndex{vertexCount, cFalse, memRes};
            Vector<std::uint32_t> blendShapeVertexIndices{vertexCount, {}, memRes};
            for (std::size_t dnaIdx = 0; dnaIdx < dnaCount; dnaIdx++) {
                for (const auto vertexIndex : dnas[dnaIdx]->getBlendShapeTargetVertexIndices(meshIdx, bsIdx)) {
                    containsVertexIndex[vertexIndex] = cTrue;
                }
            }
            std::size_t vertexIndexCount = 0u;
            for (std::uint32_t i = 0; i < containsVertexIndex.size(); i++) {
                if (containsVertexIndex[i] == cTrue) {
                    blendShapeVertexIndices[vertexIndexCount] = i;
                    vertexIndexCount++;
                }
            }
            vertexIndices[meshIdx].appendRow(ConstArrayView<std::uint32_t>{blendShapeVertexIndices.data(), vertexIndexCount});
        }
        vertexIndices[meshIdx].shrinkToFit();
    }
}

const BlendShapeDeltas<4u>& BlendShapePool::getBlendShapeTargetDeltas() const {
    return deltas;
}

ConstArrayView<VariableWidthMatrix<std::uint32_t>> BlendShapePool::getVertexIndices() const {
    return {vertexIndices.data(), vertexIndices.size()};
}

std::uint16_t BlendShapePool::getBlendShapeCount(std::uint16_t meshIndex) const {
    if (meshIndex >= vertexIndices.size()) {
        return 0u;
    }
    return static_cast<std::uint16_t>(vertexIndices[meshIndex].rowCount());
}

}  // namespace gs4
