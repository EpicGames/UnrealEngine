// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/CalculationType.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/genepool/BlendShapeDeltas.h"
#include "genesplicer/types/Block.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/Matrix.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#include <cstdint>

namespace gs4 {

class BlendShapePool {
public:
    BlendShapePool(MemoryResource* memRes);
    BlendShapePool(const Reader* deltaArchetype, ConstArrayView<const Reader*> dnas, MemoryResource* memRes = nullptr);

    ConstArrayView<VariableWidthMatrix<std::uint32_t>> getVertexIndices() const;
    const BlendShapeDeltas<4u>& getBlendShapeTargetDeltas() const;
    std::uint16_t getBlendShapeCount(std::uint16_t meshIndex) const;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(vertexIndices);
        archive(deltas);
    }

private:
    void fillVertexIndices(const Reader* deltaArchetype, ConstArrayView<const Reader*> dnas, MemoryResource* memRes);

private:
    // [meshIdx][bsIdx][deltaIdx]
    Vector<VariableWidthMatrix<std::uint32_t>> vertexIndices;
    BlendShapeDeltas<4u> deltas;
};

}  // namespace gs4
