// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/genepool/GenePoolImpl.h"

#include "genesplicer/CalculationType.h"
#include "genesplicer/Macros.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/neutraljointsplicer/JointAttribute.h"
#include "genesplicer/splicedata/GenePool.h"
#include "genesplicer/splicedata/genepool/NullGenePoolImpl.h"
#include "genesplicer/splicedata/rawgenes/RawGenes.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/Vec3.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <iterator>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

namespace {

GenePoolImpl::MetaData getMetaData(const Reader* deltaArchetype,
                                   ConstArrayView<const Reader*> dnas,
                                   GenePoolMask genePoolMask,
                                   MemoryResource* memRes) {
    GenePoolImpl::MetaData metadata{memRes};
    metadata.names.reserve(dnas.size());
    metadata.genders.reserve(dnas.size());
    metadata.ages.reserve(dnas.size());
    for (const auto dna : dnas) {
        metadata.names.push_back(String<char>{dna->getName().c_str(), memRes});
        metadata.genders.push_back(static_cast<std::uint16_t>(dna->getGender()));
        metadata.ages.push_back(dna->getAge());
    }
    metadata.jointNames.reserve(deltaArchetype->getJointCount());
    for (std::uint16_t ji = 0u; ji < deltaArchetype->getJointCount(); ji++) {
        metadata.jointNames.push_back(String<char>{deltaArchetype->getJointName(ji).c_str(), memRes});
    }
    metadata.dbMaxLOD = deltaArchetype->getDBMaxLOD();
    metadata.vertexCountPerMesh.reserve(deltaArchetype->getMeshCount());
    metadata.dbName = String<char>{deltaArchetype->getDBName(), memRes};
    metadata.dbComplexity = String<char>{deltaArchetype->getDBComplexity(), memRes};
    metadata.genePoolMask = genePoolMask;
    for (std::uint16_t meshIndex = 0u; meshIndex < deltaArchetype->getMeshCount(); meshIndex++) {
        metadata.vertexCountPerMesh.push_back(deltaArchetype->getVertexPositionCount(meshIndex));
    }
    return metadata;
}

}  // namespace

GenePoolImpl::GenePoolImpl(const Reader* deltaArchetype,
                           ConstArrayView<const Reader*> dnas,
                           GenePoolMask genePoolMask,
                           MemoryResource* memRes_) :
    memRes{memRes_},
    metadata{getMetaData(deltaArchetype, dnas, genePoolMask, memRes_)},
    neutralMeshes{memRes},
    blendShapes{memRes},
    neutralJoints{memRes},
    skinWeights{memRes},
    jointBehavior{memRes} {

    auto isMasked = [this](GenePoolMask poolType) { return (metadata.genePoolMask & poolType) != poolType; };

    if (!isMasked(GenePoolMask::NeutralMeshes)) {
        neutralMeshes = {deltaArchetype, dnas, memRes};
    }
    if (!isMasked(GenePoolMask::BlendShapes)) {
        blendShapes = {deltaArchetype, dnas, memRes};
    }
    if (!isMasked(GenePoolMask::SkinWeights)) {
        skinWeights = {dnas, memRes};
    }
    if (!isMasked(GenePoolMask::NeutralJoints)) {
        neutralJoints = {deltaArchetype, dnas, memRes};
    }
    if (!isMasked(GenePoolMask::JointBehavior)) {
        jointBehavior = {deltaArchetype, dnas, memRes};
    }
}

GenePoolImpl::GenePoolImpl(MemoryResource* memRes_) :
    memRes{memRes_},
    metadata{memRes},
    neutralMeshes{memRes},
    blendShapes{memRes},
    neutralJoints{memRes},
    skinWeights{memRes},
    jointBehavior{memRes} {
}

std::uint16_t GenePoolImpl::getDNACount() const {
    return static_cast<std::uint16_t>(metadata.names.size());
}

StringView GenePoolImpl::getDNAName(std::uint16_t dnaIndex) const {
    if (dnaIndex >= metadata.names.size()) {
        return {};
    }
    return metadata.names[dnaIndex];
}

dna::Gender GenePoolImpl::getDNAGender(std::uint16_t dnaIndex) const {
    if (dnaIndex >= metadata.genders.size()) {
        return {};
    }
    return static_cast<dna::Gender>(metadata.genders[dnaIndex]);
}

std::uint16_t GenePoolImpl::getDNAAge(std::uint16_t dnaIndex) const {
    if (dnaIndex >= metadata.ages.size()) {
        return 0u;
    }
    return metadata.ages[dnaIndex];
}

std::uint16_t GenePoolImpl::getMeshCount() const {
    return static_cast<std::uint16_t>(metadata.vertexCountPerMesh.size());
}

std::uint32_t GenePoolImpl::getVertexCount(std::uint16_t meshIndex) const {
    if (meshIndex < metadata.vertexCountPerMesh.size()) {
        return metadata.vertexCountPerMesh[meshIndex];
    }
    return 0u;
}

std::uint32_t GenePoolImpl::getVertexPositionCount(std::uint16_t meshIndex) const {
    return neutralMeshes.getVertexCount(meshIndex);
}

Vector3 GenePoolImpl::getDNAVertexPosition(std::uint16_t dnaIndex, std::uint16_t meshIndex, std::uint32_t vertexIndex) const {
    return neutralMeshes.getDNAVertexPosition(dnaIndex, meshIndex, vertexIndex);
}

Vector3 GenePoolImpl::getArchetypeVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const {
    return neutralMeshes.getArchetypeVertexPosition(meshIndex, vertexIndex);
}

std::uint16_t GenePoolImpl::getBlendShapeTargetCount(std::uint16_t meshIndex) const {
    return blendShapes.getBlendShapeCount(meshIndex);
}

std::uint32_t GenePoolImpl::getSkinWeightsCount(std::uint16_t meshIndex) const {
    return skinWeights.getSkinWeightsCount(meshIndex);
}

std::uint16_t GenePoolImpl::getMaximumInfluencesPerVertex(std::uint16_t meshIndex) const {
    return skinWeights.getMaximumInfluencesPerVertex(meshIndex);
}

std::uint16_t GenePoolImpl::getNeutralJointCount() const {
    return neutralJoints.getJointCount();
}

std::uint16_t GenePoolImpl::getJointCount() const {
    return static_cast<std::uint16_t>(metadata.jointNames.size());
}

StringView GenePoolImpl::getJointName(std::uint16_t jointIndex) const {
    if (jointIndex >= metadata.jointNames.size()) {
        return {};
    }
    return metadata.jointNames[jointIndex];
}

std::uint16_t GenePoolImpl::getJointGroupCount() const {
    return jointBehavior.getJointGroupCount();
}

Vector3 GenePoolImpl::getDNANeutralJointWorldTranslation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const {
    return neutralJoints.getDNANeutralJointWorldTranslation(dnaIndex, jointIndex);
}

Vector3 GenePoolImpl::getArchetypeNeutralJointWorldTranslation(std::uint16_t jointIndex) const {
    return neutralJoints.getArchetypeNeutralJointWorldTranslation(jointIndex);
}

Vector3 GenePoolImpl::getDNANeutralJointWorldRotation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const {
    return neutralJoints.getDNANeutralJointWorldRotation(dnaIndex, jointIndex);
}

Vector3 GenePoolImpl::getArchetypeNeutralJointWorldRotation(std::uint16_t jointIndex) const {
    return neutralJoints.getArchetypeNeutralJointWorldRotation(jointIndex);
}

ConstArrayView<XYZTiledMatrix<16u>> GenePoolImpl::getNeutralMeshes() const {
    return neutralMeshes.getData();
}

const BlendShapeDeltas<4u>& GenePoolImpl::getBlendShapeTargetDeltas() const {
    return blendShapes.getBlendShapeTargetDeltas();
}

ConstArrayView<VariableWidthMatrix<std::uint32_t>> GenePoolImpl::getBlendShapeTargetVertexIndices() const {
    return blendShapes.getVertexIndices();
}

const VariableWidthMatrix<TiledMatrix2D<16u>>& GenePoolImpl::getSkinWeightValues() const {
    return skinWeights.getWeights();
}

ConstArrayView<VariableWidthMatrix<std::uint16_t>> GenePoolImpl::getSkinWeightJointIndices() const {
    return skinWeights.getJointIndices();
}

const XYZTiledMatrix<16u>& GenePoolImpl::getNeutralJoints(JointAttribute jointAttribute) const {
    if (jointAttribute == JointAttribute::Rotation) {
        return neutralJoints.getDNAData<JointAttribute::Rotation>();
    } else {
        return neutralJoints.getDNAData<JointAttribute::Translation>();
    }
}

const VariableWidthMatrix<std::uint16_t>& GenePoolImpl::getJointBehaviorInputIndices() const {
    return jointBehavior.getInputIndices();
}

const VariableWidthMatrix<std::uint16_t>& GenePoolImpl::getJointBehaviorOutputIndices() const {
    return jointBehavior.getOutputIndices();
}

const VariableWidthMatrix<std::uint16_t>& GenePoolImpl::getJointBehaviorLODs() const {
    return jointBehavior.getLODs();
}

ConstArrayView<SingleJointBehavior> GenePoolImpl::getJointBehaviorValues() const {
    return jointBehavior.getJointValues();
}

MemoryResource* GenePoolImpl::getMemoryResource() const {
    return memRes;
}

}  // namespace gs4
