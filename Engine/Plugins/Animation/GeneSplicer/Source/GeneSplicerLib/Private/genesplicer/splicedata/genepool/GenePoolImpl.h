// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/CalculationType.h"
#include "genesplicer/Defs.h"
#include "genesplicer/GeneSplicerDNAReader.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/neutraljointsplicer/JointAttribute.h"
#include "genesplicer/splicedata/GenePool.h"
#include "genesplicer/splicedata/genepool/BlendShapePool.h"
#include "genesplicer/splicedata/genepool/GenePoolInterface.h"
#include "genesplicer/splicedata/genepool/JointBehaviorPool.h"
#include "genesplicer/splicedata/genepool/NeutralJointPool.h"
#include "genesplicer/splicedata/genepool/NeutralMeshPool.h"
#include "genesplicer/splicedata/genepool/SkinWeightPool.h"
#include "genesplicer/splicedata/rawgenes/RawGenes.h"
#include "genesplicer/types/PImplExtractor.h"
#include "genesplicer/types/Signature.h"
#include "genesplicer/types/Version.h"

#include <dna/layers/Descriptor.h>

#include <cstdint>

namespace gs4 {

class GenePoolImpl : public GenePoolInterface {
public:
    struct SectionLookupTable {
        terse::ArchiveOffset<std::uint64_t> neutralMeshes;
        terse::ArchiveOffset<std::uint64_t> blendShapes;
        terse::ArchiveOffset<std::uint64_t> neutralJoints;
        terse::ArchiveOffset<std::uint64_t> skinWeights;
        terse::ArchiveOffset<std::uint64_t> jointBehavior;
        terse::ArchiveOffset<std::uint64_t> metadata;

        template<class Archive>
        void serialize(Archive& archive) {
            archive(neutralMeshes, blendShapes, neutralJoints, skinWeights, jointBehavior, metadata);
        }
    };

    struct MetaData {
        Vector<String<char>> names;
        Vector<std::uint16_t> genders;
        Vector<std::uint16_t> ages;
        Vector<String<char>> jointNames;
        std::uint16_t dbMaxLOD;
        String<char> dbName;
        String<char> dbComplexity;
        Vector<std::uint32_t> vertexCountPerMesh;
        GenePoolMask genePoolMask;

        explicit MetaData(MemoryResource* memRes) :
            names{memRes},
            genders{memRes},
            ages{memRes},
            jointNames{memRes},
            dbMaxLOD{},
            dbName{memRes},
            dbComplexity{memRes},
            vertexCountPerMesh{memRes},
            genePoolMask{GenePoolMask::All} {
        }

        explicit MetaData(const Vector<String<char>>& names_,
                          const Vector<std::uint16_t>& genders_,
                          const Vector<std::uint16_t>& ages_,
                          const Vector<String<char>>& jointNames_,
                          std::uint16_t dbMaxLOD_,
                          String<char> dbName_,
                          String<char> dbComplexity_,
                          const Vector<std::uint32_t>& vertexCountPerMesh_,
                          GenePoolMask genePoolMask_,
                          MemoryResource* memRes) :
            names{names_, memRes},
            genders{genders_, memRes},
            ages{ages_, memRes},
            jointNames{jointNames_, memRes},
            dbMaxLOD{dbMaxLOD_},
            dbName{dbName_, memRes},
            dbComplexity{dbComplexity_, memRes},
            vertexCountPerMesh{vertexCountPerMesh_, memRes},
            genePoolMask{genePoolMask_} {
        }

        template<class Archive>
        void serialize(Archive& archive) {
            archive(names, genders, ages, jointNames, dbMaxLOD, dbName, dbComplexity, vertexCountPerMesh, genePoolMask);
        }
    };

private:
    const Signature<3> getSignature() {
        return Signature<3>{{'G', 'N', 'P'}};
    }

    const Signature<3> getEoF() {
        return Signature<3>{{'P', 'N', 'G'}};
    }

public:
    GenePoolImpl(const Reader* deltaArchetype,
                 ConstArrayView<const Reader*> dnas,
                 GenePoolMask genePoolMask,
                 MemoryResource* memRes_);

    explicit GenePoolImpl(MemoryResource* memRes_);

    std::uint16_t getDNACount() const override;
    StringView getDNAName(std::uint16_t dnaIndex) const override;
    dna::Gender getDNAGender(std::uint16_t dnaIndex) const override;
    std::uint16_t getDNAAge(std::uint16_t dnaIndex) const override;

    std::uint16_t getMeshCount() const override;
    std::uint32_t getVertexCount(std::uint16_t meshIndex) const override;
    std::uint32_t getVertexPositionCount(std::uint16_t meshIndex) const override;
    Vector3 getDNAVertexPosition(std::uint16_t dnaIndex, std::uint16_t meshIndex, std::uint32_t vertexIndex) const override;
    Vector3 getArchetypeVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override;

    std::uint16_t getBlendShapeTargetCount(std::uint16_t meshIndex) const override;

    std::uint32_t getSkinWeightsCount(std::uint16_t meshIndex) const override;
    std::uint16_t getMaximumInfluencesPerVertex(std::uint16_t meshIdx) const override;

    std::uint16_t getNeutralJointCount() const override;

    std::uint16_t getJointCount() const override;
    StringView getJointName(std::uint16_t jointIndex) const override;
    std::uint16_t getJointGroupCount() const override;

    Vector3 getDNANeutralJointWorldTranslation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const override;
    Vector3 getArchetypeNeutralJointWorldTranslation(std::uint16_t jointIndex) const override;
    Vector3 getDNANeutralJointWorldRotation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const override;
    Vector3 getArchetypeNeutralJointWorldRotation(std::uint16_t jointIndex) const override;

    ConstArrayView<XYZTiledMatrix<16u>> getNeutralMeshes() const override;

    const BlendShapeDeltas<4u>& getBlendShapeTargetDeltas() const override;
    ConstArrayView<VariableWidthMatrix<std::uint32_t>> getBlendShapeTargetVertexIndices() const override;

    const VariableWidthMatrix<TiledMatrix2D<16u>>& getSkinWeightValues() const override;
    ConstArrayView<VariableWidthMatrix<std::uint16_t>> getSkinWeightJointIndices() const override;

    const XYZTiledMatrix<16u>& getNeutralJoints(JointAttribute jointAttribute) const override;

    const VariableWidthMatrix<std::uint16_t>& getJointBehaviorInputIndices() const override;
    const VariableWidthMatrix<std::uint16_t>& getJointBehaviorOutputIndices() const override;
    const VariableWidthMatrix<std::uint16_t>& getJointBehaviorLODs() const override;
    ConstArrayView<SingleJointBehavior> getJointBehaviorValues() const override;

    MemoryResource* getMemoryResource() const override;

    template<class Archive>
    void save(Archive& archive) {
        SectionLookupTable sections;
        archive(getSignature(), version, sections);
        // it is important to process metadata first as it updates GenePoolMask
        archive(terse::proxy(sections.metadata),
                metadata,
                terse::proxy(sections.neutralMeshes),
                neutralMeshes,
                terse::proxy(sections.blendShapes),
                blendShapes,
                terse::proxy(sections.neutralJoints),
                neutralJoints,
                terse::proxy(sections.skinWeights),
                skinWeights,
                terse::proxy(sections.jointBehavior),
                jointBehavior,
                getEoF());
    }

    template<class Archive>
    void load(Archive& archive) {
        SectionLookupTable sections;
        auto signature = getSignature();
        auto eof = getEoF();
        archive(signature, version);
        if (signature.matches() && version.matches()) {
            archive(sections);
            // it is important to process metadata first as it updates GenePoolMask
            archive(terse::proxy(sections.metadata),
                    metadata,
                    terse::proxy(sections.neutralMeshes),
                    neutralMeshes,
                    terse::proxy(sections.blendShapes),
                    blendShapes,
                    terse::proxy(sections.neutralJoints),
                    neutralJoints,
                    terse::proxy(sections.skinWeights),
                    skinWeights,
                    terse::proxy(sections.jointBehavior),
                    jointBehavior,
                    eof);
            assert(eof.matches());
        }
    }

    ~GenePoolImpl() = default;

private:
    MemoryResource* memRes;
    ExpectedVersion version{0, 1};
    MetaData metadata;
    NeutralMeshPool neutralMeshes;
    BlendShapePool blendShapes;
    NeutralJointPool neutralJoints;
    SkinWeightPool skinWeights;
    JointBehaviorPool jointBehavior;
};

}  // namespace gs4
