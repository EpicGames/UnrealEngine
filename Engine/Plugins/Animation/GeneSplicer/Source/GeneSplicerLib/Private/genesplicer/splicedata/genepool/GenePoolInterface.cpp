// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/genepool/GenePoolInterface.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"
#include "genesplicer/splicedata/genepool/NullGenePoolImpl.h"
#include "genesplicer/splicedata/genepool/stream/FilteredInputArchive.h"
#include "genesplicer/splicedata/genepool/stream/FilteredOutputArchive.h"

namespace gs4 {

namespace {

FORCE_INLINE bool compatible(const Reader* lhs, const Reader* rhs) {
    if ((lhs == nullptr) || (rhs == nullptr)) {
        return false;
    }
    if ((lhs->getDBName() != rhs->getDBName()) || (lhs->getDBMaxLOD() != rhs->getDBMaxLOD()) ||
        (lhs->getDBComplexity() != rhs->getDBComplexity())) {
        return false;
    }
    if (lhs->getMeshCount() != rhs->getMeshCount()) {
        return false;
    }
    std::uint16_t meshCount = lhs->getMeshCount();
    if (lhs->getJointCount() != rhs->getJointCount()) {
        return false;
    }
    for (std::uint16_t meshIdx = 0u; meshIdx < meshCount; meshIdx++) {
        if (lhs->getVertexPositionCount(meshIdx) != rhs->getVertexPositionCount(meshIdx)) {
            return false;
        }
        if (lhs->getBlendShapeTargetCount(meshIdx) != rhs->getBlendShapeTargetCount(meshIdx)) {
            return false;
        }
    }
    return true;
}

}  // namespace

const StatusCode GenePool::DNAMismatch{1001, "DNA with index %zu is incompatible with delta archetype.\n"};
const StatusCode GenePool::DNAsEmpty{1002, "DNA list is empty.\n"};

#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wglobal-constructors"
sc::StatusProvider GenePoolInterface::status{GenePool::DNAMismatch, GenePool::DNAsEmpty};
#pragma clang diagnostic pop

GenePool::~GenePool() = default;
GenePool::GenePool(GenePool&& rhs) = default;
GenePool& GenePool::operator=(GenePool&& rhs) = default;
GenePoolInterface::~Impl() = default;
NullGenePoolImpl::~NullGenePoolImpl() = default;

GenePool::GenePool(const Reader* deltaArchetype,
                   const Reader** dnas,
                   std::uint16_t dnaCount,
                   GenePoolMask genePoolMask,
                   MemoryResource* memRes) :
    pImpl{makeScoped<GenePoolInterface>(deltaArchetype, dnas, dnaCount, genePoolMask, memRes)} {
}

GenePool::GenePool(BoundedIOStream* stream, GenePoolMask mask, MemoryResource* memRes) :
    pImpl{makeScoped<GenePoolInterface>(memRes)} {
    FilteredInputArchive archive{stream, mask, memRes};
    archive >> *static_cast<GenePoolImpl*>(pImpl.get());
}

void GenePool::dump(BoundedIOStream* stream, GenePoolMask mask) {
    auto genePoolImpl = pImpl.get();
    if (!genePoolImpl->getIsNullGenePool()) {
        FilteredOutputArchive archive{stream, mask, genePoolImpl->getMemoryResource()};
        archive << *static_cast<GenePoolImpl*>(genePoolImpl);
    }
}

std::uint16_t GenePool::getDNACount() const {
    return pImpl->getDNACount();
}

StringView GenePool::getDNAName(std::uint16_t dnaIndex) const {
    return pImpl->getDNAName(dnaIndex);
}

dna::Gender GenePool::getDNAGender(std::uint16_t dnaIndex) const {
    return pImpl->getDNAGender(dnaIndex);
}

std::uint16_t GenePool::getDNAAge(std::uint16_t dnaIndex) const {
    return pImpl->getDNAAge(dnaIndex);
}

std::uint16_t GenePool::getMeshCount() const {
    return pImpl->getMeshCount();
}

std::uint32_t GenePool::getVertexPositionCount(std::uint16_t meshIndex) const {
    return pImpl->getVertexPositionCount(meshIndex);
}

Vector3 GenePool::getDNAVertexPosition(std::uint16_t dnaIndex, std::uint16_t meshIndex, std::uint32_t vertexIndex) const {
    return pImpl->getDNAVertexPosition(dnaIndex, meshIndex, vertexIndex);
}

Vector3 GenePool::getArchetypeVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const {
    return pImpl->getArchetypeVertexPosition(meshIndex, vertexIndex);
}

std::uint16_t GenePool::getJointCount() const {
    return pImpl->getJointCount();
}

StringView GenePool::getJointName(std::uint16_t jointIndex) const {
    return pImpl->getJointName(jointIndex);
}

Vector3 GenePool::getDNANeutralJointWorldTranslation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const {
    return pImpl->getDNANeutralJointWorldTranslation(dnaIndex, jointIndex);
}

Vector3 GenePool::getArchetypeNeutralJointWorldTranslation(std::uint16_t jointIndex) const {
    return pImpl->getArchetypeNeutralJointWorldTranslation(jointIndex);
}

Vector3 GenePool::getDNANeutralJointWorldRotation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const {
    return pImpl->getDNANeutralJointWorldRotation(dnaIndex, jointIndex);
}

Vector3 GenePool::getArchetypeNeutralJointWorldRotation(std::uint16_t jointIndex) const {
    return pImpl->getArchetypeNeutralJointWorldRotation(jointIndex);
}

GenePoolInterface* GenePoolInterface::create(const Reader* deltaArchetype,
                                             const Reader** dnas,
                                             std::uint16_t dnaCount,
                                             GenePoolMask genePoolMask,
                                             MemoryResource* memRes) {
    status.reset();
    if (dnaCount == 0u) {
        status.set(GenePool::DNAsEmpty);
        PolyAllocator<NullGenePoolImpl> alloc{memRes};
        return alloc.newObject(memRes);
    }
    for (std::size_t i = 0; i < dnaCount; i++) {
        if (!compatible(deltaArchetype, dnas[i])) {
            status.set(GenePool::DNAMismatch, i);
            PolyAllocator<NullGenePoolImpl> alloc{memRes};
            return alloc.newObject(memRes);
        }
    }
    PolyAllocator<GenePoolImpl> alloc{memRes};
    return alloc.newObject(deltaArchetype, ConstArrayView<const Reader*>{dnas, dnaCount}, genePoolMask, memRes);
}

GenePoolInterface* GenePoolInterface::create(MemoryResource* memRes) {
    PolyAllocator<GenePoolImpl> alloc{memRes};
    return alloc.newObject(memRes);
}

void GenePoolInterface::destroy(GenePoolInterface* instance) {
    PolyAllocator<Impl> alloc{instance->getMemoryResource()};
    alloc.deleteObject(instance);
}

}  // namespace gs4
