// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/CalculationType.h"
#include "genesplicer/Defs.h"
#include "genesplicer/GeneSplicerDNAReader.h"
#include "genesplicer/neutraljointsplicer/JointAttribute.h"
#include "genesplicer/splicedata/GenePool.h"
#include "genesplicer/splicedata/genepool/BlendShapeDeltas.h"
#include "genesplicer/splicedata/genepool/SingleJointBehavior.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/PImplExtractor.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#include <cstdint>

namespace gs4 {

using GenePoolInterface = PImplExtractor<GenePool>::impl_type;
class GenePool::Impl {

public:
    static GenePoolInterface* create(const Reader* deltaArchetype,
                                     const Reader** dnas,
                                     std::uint16_t dnaCount,
                                     GenePoolMask genePoolMask,
                                     MemoryResource* memRes);

    static GenePoolInterface* create(MemoryResource* memRes);

    static void destroy(GenePoolInterface* instance);

public:
    virtual ~Impl();

    virtual std::uint16_t getDNACount() const = 0;
    virtual StringView getDNAName(std::uint16_t dnaIndex) const = 0;
    virtual dna::Gender getDNAGender(std::uint16_t dnaIndex) const = 0;
    virtual std::uint16_t getDNAAge(std::uint16_t dnaIndex) const = 0;

    virtual std::uint16_t getMeshCount() const = 0;
    virtual std::uint32_t getVertexCount(std::uint16_t meshIndex) const = 0;
    virtual std::uint32_t getVertexPositionCount(std::uint16_t meshIndex) const = 0;
    virtual Vector3 getDNAVertexPosition(std::uint16_t dnaIndex, std::uint16_t meshIndex, std::uint32_t vertexIndex) const = 0;
    virtual Vector3 getArchetypeVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const = 0;

    virtual std::uint16_t getBlendShapeTargetCount(std::uint16_t meshIndex) const = 0;

    virtual std::uint32_t getSkinWeightsCount(std::uint16_t meshIndex) const = 0;
    virtual std::uint16_t getMaximumInfluencesPerVertex(std::uint16_t meshIdx) const = 0;

    virtual std::uint16_t getNeutralJointCount() const = 0;

    virtual std::uint16_t getJointCount() const = 0;
    virtual StringView getJointName(std::uint16_t jointIndex) const = 0;
    virtual std::uint16_t getJointGroupCount() const = 0;

    virtual Vector3 getDNANeutralJointWorldTranslation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const = 0;
    virtual Vector3 getArchetypeNeutralJointWorldTranslation(std::uint16_t jointIndex) const = 0;
    virtual Vector3 getDNANeutralJointWorldRotation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const = 0;
    virtual Vector3 getArchetypeNeutralJointWorldRotation(std::uint16_t jointIndex) const = 0;

    virtual ConstArrayView<XYZTiledMatrix<16u>> getNeutralMeshes() const = 0;

    virtual const BlendShapeDeltas<4u>& getBlendShapeTargetDeltas() const = 0;
    virtual ConstArrayView<VariableWidthMatrix<std::uint32_t>> getBlendShapeTargetVertexIndices() const = 0;

    virtual const VariableWidthMatrix<TiledMatrix2D<16u>>& getSkinWeightValues() const = 0;
    virtual ConstArrayView<VariableWidthMatrix<std::uint16_t>> getSkinWeightJointIndices() const = 0;

    virtual const XYZTiledMatrix<16u>& getNeutralJoints(JointAttribute jointAttribute) const = 0;

    virtual const VariableWidthMatrix<std::uint16_t>& getJointBehaviorInputIndices() const = 0;
    virtual const VariableWidthMatrix<std::uint16_t>& getJointBehaviorOutputIndices() const = 0;
    virtual const VariableWidthMatrix<std::uint16_t>& getJointBehaviorLODs() const = 0;
    virtual ConstArrayView<SingleJointBehavior> getJointBehaviorValues() const = 0;

    virtual bool getIsNullGenePool() const {
        return false;
    }

    virtual MemoryResource* getMemoryResource() const = 0;

private:
    static sc::StatusProvider status;
};

}  // namespace gs4
namespace pma {
template<>
struct DefaultInstanceCreator<gs4::GenePoolInterface> {
    using type = FactoryCreate<gs4::GenePoolInterface>;
};

template<>
struct DefaultInstanceDestroyer<gs4::GenePoolInterface> {
    using type = FactoryDestroy<gs4::GenePoolInterface>;
};

}  // namespace pma
