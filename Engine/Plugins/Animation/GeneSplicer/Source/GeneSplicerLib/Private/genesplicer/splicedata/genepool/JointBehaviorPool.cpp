// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/genepool/JointBehaviorPool.h"

#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/genepool/SingleJointBehavior.h"
#include "genesplicer/splicedata/rawgenes/JointGroupOutputIndicesMerger.h"
#include "genesplicer/splicedata/rawgenes/RawGenesUtils.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/Matrix.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cstdint>
#include <functional>
#include <iterator>
#include <limits>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

JointBehaviorPool::JointBehaviorPool(MemoryResource* memRes_) :
    memRes{memRes_},
    jointValues{memRes},
    inIndices{memRes},
    outIndices{memRes},
    LODs{memRes} {
}

JointBehaviorPool::JointBehaviorPool(const Reader* deltaArchetype, ConstArrayView<const Reader*> dnas, MemoryResource* memRes_) :
    memRes{memRes_},
    jointValues{memRes},
    inIndices{memRes},
    outIndices{memRes},
    LODs{memRes} {

    const std::size_t jointGroupCount = deltaArchetype->getJointGroupCount();
    if (jointGroupCount == 0) {
        return;
    }
    LODs.reserve(jointGroupCount, jointGroupCount * deltaArchetype->getDBMaxLOD());
    const auto rawControlCount = static_cast<std::size_t>(deltaArchetype->getRawControlCount());
    const auto psdCount = static_cast<std::size_t>(deltaArchetype->getPSDCount());
    inIndices.reserve(jointGroupCount, rawControlCount + psdCount);
    const std::size_t jointCount = deltaArchetype->getJointCount();
    const std::size_t maxOutputCount = jointCount * 9ul;
    outIndices.reserve(jointGroupCount, maxOutputCount);
    jointValues.resize(jointCount);

    Vector<std::uint16_t> outputIndicesHolder{maxOutputCount, {}, memRes};

    for (std::uint16_t jntGroupIdx = 0; jntGroupIdx < jointGroupCount; jntGroupIdx++) {
        inIndices.appendRow(deltaArchetype->getJointGroupInputIndices(jntGroupIdx));
        LODs.appendRow(deltaArchetype->getJointGroupLODs(jntGroupIdx).size());
        JointGroupOutputIndicesMerger merger{deltaArchetype->getJointGroupJointIndices(jntGroupIdx), memRes};
        for (auto dna : dnas) {
            merger.add(dna->getJointGroupOutputIndices(jntGroupIdx), dna->getJointGroupLODs(jntGroupIdx));
        }
        merger.add(deltaArchetype->getJointGroupOutputIndices(jntGroupIdx), deltaArchetype->getJointGroupLODs(jntGroupIdx));
        merger.merge(outputIndicesHolder.begin(), LODs[jntGroupIdx].end());
        outIndices.appendRow({outputIndicesHolder.data(), LODs[jntGroupIdx][0u]});

        for (auto outIdx : outIndices[jntGroupIdx]) {
            auto archValues = getJointValuesForOutputIndex(deltaArchetype, jntGroupIdx, outIdx);
            Vector<ConstArrayView<float>> dnaBlock{dnas.size(), {}, memRes};
            std::transform(dnas.begin(), dnas.end(), dnaBlock.begin(), [jntGroupIdx, outIdx](const Reader* dna) {
                return getJointValuesForOutputIndex(dna, jntGroupIdx, outIdx);
            });
            auto inputCount = static_cast<std::uint16_t>(inIndices[jntGroupIdx].size());
            auto outPos = static_cast<std::uint8_t>(outIdx % 9);
            auto jointIndex = static_cast<std::uint16_t>(outIdx / 9);
            jointValues[jointIndex].setValues(inputCount, outPos, archValues, {dnaBlock.data(), dnaBlock.size()});
        }
    }

    LODs.shrinkToFit();
    inIndices.shrinkToFit();
    outIndices.shrinkToFit();
}

ConstArrayView<SingleJointBehavior> JointBehaviorPool::getJointValues() const {
    return {jointValues.data(), jointValues.size()};
}

const VariableWidthMatrix<std::uint16_t>& JointBehaviorPool::getInputIndices() const {
    return inIndices;
}

const VariableWidthMatrix<std::uint16_t>& JointBehaviorPool::getOutputIndices() const {
    return outIndices;
}

const VariableWidthMatrix<std::uint16_t>& JointBehaviorPool::getLODs() const {
    return LODs;
}

std::uint16_t JointBehaviorPool::getJointGroupCount() const {
    return static_cast<std::uint16_t>(inIndices.rowCount());
}

}  // namespace gs4
