// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/genepool/SingleJointBehavior.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/Matrix.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <cstdint>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

class JointBehaviorPool {
public:
    JointBehaviorPool(MemoryResource* memRes);
    JointBehaviorPool(const Reader* deltaArchetype, ConstArrayView<const Reader*> dnas, MemoryResource* memRes);

    const VariableWidthMatrix<std::uint16_t>& getInputIndices() const;
    const VariableWidthMatrix<std::uint16_t>& getOutputIndices() const;
    const VariableWidthMatrix<std::uint16_t>& getLODs() const;
    ConstArrayView<SingleJointBehavior> getJointValues() const;
    std::uint16_t getJointGroupCount() const;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(jointValues, inIndices, outIndices, LODs);
    }

private:
    MemoryResource* memRes;
    Vector<SingleJointBehavior> jointValues;
    VariableWidthMatrix<std::uint16_t> inIndices;
    VariableWidthMatrix<std::uint16_t> outIndices;
    VariableWidthMatrix<std::uint16_t> LODs;
};

}  // namespace gs4
