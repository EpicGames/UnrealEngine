// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/genepool/NeutralJointPool.h"

#include "genesplicer/splicedata/genepool/RawNeutralJoints.h"
#include "genesplicer/utils/Algorithm.h"

#include <cstddef>
#include <cstdint>

namespace gs4 {

NeutralJointPool::NeutralJointPool(MemoryResource* memRes) :
    dnaTranslations{memRes},
    dnaRotations{memRes},
    archJoints{memRes} {
}

NeutralJointPool::NeutralJointPool(const Reader* deltaArchetype, ConstArrayView<const Reader*> dnas, MemoryResource* memRes) :
    dnaTranslations{memRes},
    dnaRotations{memRes},
    archJoints{deltaArchetype, memRes} {
    const std::size_t dnaCount = dnas.size();
    const auto jointCount = static_cast<std::uint16_t>(archJoints.translations.size());
    static constexpr std::uint16_t blockSize = XYZTiledMatrix<16u>::value_type::size();
    const auto endIdx = static_cast<std::uint16_t>(jointCount / blockSize);
    const auto remainder = static_cast<std::uint16_t>(jointCount % blockSize);
    const auto blockCount = static_cast<std::uint16_t>(endIdx + (1u && remainder));

    dnaTranslations = XYZTiledMatrix<16u>{blockCount, dnaCount, memRes};
    dnaRotations = XYZTiledMatrix<16u>{blockCount, dnaCount, memRes};
    if (jointCount == 0) {
        return;
    }

    auto getJointParentIndex = std::bind(&dna::DefinitionReader::getJointParentIndex, deltaArchetype, std::placeholders::_1);

    toWorldSpace(getJointParentIndex, archJoints);
    for (std::size_t dnaIdx = 0; dnaIdx < dnaCount; dnaIdx++) {
        RawNeutralJoints dnaJoints{dnas[dnaIdx], memRes};
        toWorldSpace(getJointParentIndex, dnaJoints);

        const auto setValues = [](const RawVector3Vector& dnaJoints_,
                                  const RawVector3Vector& archJoints_,
                                  XYZBlock<16>& dest,
                                  std::uint16_t jntIdx) {
            const auto offset = static_cast<std::uint16_t>(jntIdx % blockSize);
            dest.Xs[offset] = dnaJoints_.xs[jntIdx] - archJoints_.xs[jntIdx];
            dest.Ys[offset] = dnaJoints_.ys[jntIdx] - archJoints_.ys[jntIdx];
            dest.Zs[offset] = dnaJoints_.zs[jntIdx] - archJoints_.zs[jntIdx];
        };

        for (std::uint16_t blockIdx = 0u; blockIdx < endIdx; blockIdx++) {
            auto& destTranslation = dnaTranslations[blockIdx][dnaIdx];
            auto& destRotation = dnaRotations[blockIdx][dnaIdx];
            const auto startIdx = static_cast<std::uint16_t>(blockIdx * blockSize);
            for (std::uint16_t offset = 0; offset < blockSize; offset++) {
                const auto jntIdx = static_cast<std::uint16_t>(startIdx + offset);
                setValues(dnaJoints.translations, archJoints.translations, destTranslation, jntIdx);
                setValues(dnaJoints.rotations, archJoints.rotations, destRotation, jntIdx);
            }
        }
        const auto lastBlockIndex = blockCount - 1u;
        auto& destTranslation = dnaTranslations[lastBlockIndex][dnaIdx];
        auto& destRotation = dnaRotations[lastBlockIndex][dnaIdx];
        const auto jointStartIndex = static_cast<std::uint16_t>(endIdx * blockSize);
        for (auto jntIdx = jointStartIndex; jntIdx < jointCount; jntIdx++) {
            setValues(dnaJoints.translations, archJoints.translations, destTranslation, jntIdx);
            setValues(dnaJoints.rotations, archJoints.rotations, destRotation, jntIdx);
        }
    }
}

template<>
const XYZTiledMatrix<16u>& NeutralJointPool::getDNAData<JointAttribute::Translation>() const {
    return dnaTranslations;
}

template<>
const XYZTiledMatrix<16u>& NeutralJointPool::getDNAData<JointAttribute::Rotation>() const {
    return dnaRotations;
}

std::uint16_t NeutralJointPool::getJointCount() const {
    return static_cast<std::uint16_t>(archJoints.translations.size());
}

Vector3 NeutralJointPool::getDNANeutralJointWorldTranslation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const {
    if (jointIndex >= archJoints.translations.size()) {
        return {};
    }
    const constexpr auto blockSize = XYZTiledMatrix<16u>::value_type::size();
    const auto bucketIndex = static_cast<std::size_t>(jointIndex / blockSize);
    auto bucket = dnaTranslations[bucketIndex];
    if (dnaIndex >= bucket.size()) {
        return {};
    }
    Vector3 jointTranslation{archJoints.translations.xs[jointIndex],
                             archJoints.translations.ys[jointIndex],
                             archJoints.translations.zs[jointIndex]};
    const auto& dnaBlock = bucket[dnaIndex];
    const auto offset = static_cast<std::uint32_t>(jointIndex % blockSize);
    jointTranslation.x += dnaBlock.Xs[offset];
    jointTranslation.y += dnaBlock.Ys[offset];
    jointTranslation.z += dnaBlock.Zs[offset];
    return jointTranslation;
}

Vector3 NeutralJointPool::getArchetypeNeutralJointWorldTranslation(std::uint16_t jointIndex) const {
    if (jointIndex >= archJoints.translations.size()) {
        return {};
    }
    return {archJoints.translations.xs[jointIndex],
            archJoints.translations.ys[jointIndex],
            archJoints.translations.zs[jointIndex]};
}

Vector3 NeutralJointPool::getDNANeutralJointWorldRotation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const {
    if (jointIndex >= archJoints.rotations.size()) {
        return {};
    }
    const constexpr auto blockSize = XYZTiledMatrix<16u>::value_type::size();
    const auto bucketIndex = static_cast<std::size_t>(jointIndex / blockSize);
    auto bucket = dnaRotations[bucketIndex];
    if (dnaIndex >= bucket.size()) {
        return {};
    }
    Vector3 jointRotation{archJoints.rotations.xs[jointIndex],
                          archJoints.rotations.ys[jointIndex],
                          archJoints.rotations.zs[jointIndex]};
    const auto& dnaBlock = bucket[dnaIndex];
    const auto offset = static_cast<std::size_t>(jointIndex % blockSize);
    jointRotation.x += dnaBlock.Xs[offset];
    jointRotation.y += dnaBlock.Ys[offset];
    jointRotation.z += dnaBlock.Zs[offset];
    return jointRotation;
}

Vector3 NeutralJointPool::getArchetypeNeutralJointWorldRotation(std::uint16_t jointIndex) const {
    if (jointIndex >= archJoints.rotations.size()) {
        return {};
    }
    return {archJoints.rotations.xs[jointIndex], archJoints.rotations.ys[jointIndex], archJoints.rotations.zs[jointIndex]};
}

}  // namespace gs4
