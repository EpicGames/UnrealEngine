// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/neutraljointsplicer/JointAttribute.h"
#include "genesplicer/splicedata/genepool/RawNeutralJoints.h"
#include "genesplicer/types/BlockStorage.h"

namespace tdm {
#ifndef TDM_COORD_SYS_SERIALIZER_DEFINED
    #define TDM_COORD_SYS_SERIALIZER_DEFINED

template<typename Archive>
void load(Archive& archive, tdm::coord_sys& value) {
    std::uint32_t tmp = {};

    archive.label("x");
    archive(tmp);
    value.x = static_cast<tdm::axis_dir>(tmp);

    archive.label("y");
    archive(tmp);
    value.y = static_cast<tdm::axis_dir>(tmp);

    archive.label("z");
    archive(tmp);
    value.z = static_cast<tdm::axis_dir>(tmp);
}

template<typename Archive>
void save(Archive& archive, tdm::coord_sys& value) {
    archive.label("x");
    archive(static_cast<std::uint32_t>(value.x));

    archive.label("y");
    archive(static_cast<std::uint32_t>(value.y));

    archive.label("z");
    archive(static_cast<std::uint32_t>(value.z));
}

#endif  // TDM_COORD_SYS_SERIALIZER_DEFINED

#ifndef TDM_ROT_SEQ_SERIALIZER_DEFINED
    #define TDM_ROT_SEQ_SERIALIZER_DEFINED

template<typename Archive>
void load(Archive& archive, tdm::rot_seq& value) {
    archive.label("value");
    std::uint32_t tmp = {};
    archive(tmp);
    value = static_cast<tdm::rot_seq>(tmp);
}

template<typename Archive>
void save(Archive& archive, tdm::rot_seq& value) {
    archive.label("value");
    archive(static_cast<std::uint32_t>(value));
}

#endif  // TDM_ROT_SEQ_SERIALIZER_DEFINED

#ifndef TDM_ROT_SIGN_SERIALIZER_DEFINED
    #define TDM_ROT_SIGN_SERIALIZER_DEFINED

template<typename Archive>
void load(Archive& archive, tdm::rot_sign& value) {
    std::uint32_t tmp = {};

    archive.label("x");
    archive(tmp);
    value.x = static_cast<tdm::rot_dir>(tmp);

    archive.label("y");
    archive(tmp);
    value.y = static_cast<tdm::rot_dir>(tmp);

    archive.label("z");
    archive(tmp);
    value.z = static_cast<tdm::rot_dir>(tmp);
}

template<typename Archive>
void save(Archive& archive, tdm::rot_sign& value) {
    archive.label("x");
    archive(static_cast<std::uint32_t>(value.x));

    archive.label("y");
    archive(static_cast<std::uint32_t>(value.y));

    archive.label("z");
    archive(static_cast<std::uint32_t>(value.z));
}

#endif  // TDM_ROT_SIGN_SERIALIZER_DEFINED
}  // namespace tdm

namespace gs4 {

class NeutralJointPool {
public:
    NeutralJointPool(MemoryResource* memRes);
    NeutralJointPool(const Reader* deltaArchetype, ConstArrayView<const Reader*> dnas, MemoryResource* memRes);
    template<JointAttribute JT>
    const XYZTiledMatrix<16u>& getDNAData() const;

    std::uint16_t getJointCount() const;

    Vector3 getDNANeutralJointWorldTranslation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const;
    Vector3 getArchetypeNeutralJointWorldTranslation(std::uint16_t jointIndex) const;
    Vector3 getDNANeutralJointWorldRotation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const;
    Vector3 getArchetypeNeutralJointWorldRotation(std::uint16_t jointIndex) const;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(dnaTranslations, dnaRotations, archJoints);
    }

private:
    XYZTiledMatrix<16u> dnaTranslations;
    XYZTiledMatrix<16u> dnaRotations;
    RawNeutralJoints archJoints;
};

template<>
const XYZTiledMatrix<16u>& NeutralJointPool::getDNAData<JointAttribute::Translation>() const;

template<>
const XYZTiledMatrix<16u>& NeutralJointPool::getDNAData<JointAttribute::Rotation>() const;

}  // namespace gs4
