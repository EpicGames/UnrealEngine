// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/genepool/NeutralMeshPool.h"

#include "genesplicer/TypeDefs.h"
#include "genesplicer/types/Vec3.h"

#include <cstddef>
#include <cstdint>

namespace gs4 {

NeutralMeshPool::NeutralMeshPool(MemoryResource* memRes) :
    dnas{memRes},
    arch{memRes} {
}

NeutralMeshPool::NeutralMeshPool(const Reader* deltaArchetype, ConstArrayView<const Reader*> dnaReaders, MemoryResource* memRes) :
    dnas{memRes},
    arch{memRes} {
    const constexpr auto blockSize = XYZTiledMatrix<16u>::value_type::size();
    std::size_t dnaCount = dnaReaders.size();

    std::uint16_t meshCount = deltaArchetype->getMeshCount();
    dnas.reserve(meshCount);
    arch.reserve(meshCount);

    for (std::uint16_t meshIdx = 0u; meshIdx < meshCount; meshIdx++) {
        std::uint32_t vertexCount = deltaArchetype->getVertexPositionCount(meshIdx);
        const auto endIdx = vertexCount / blockSize;
        const auto remainder = vertexCount % blockSize;
        const auto blockCount = endIdx + (1u && remainder);
        dnas.emplace_back(blockCount, dnaCount);
        arch.emplace_back(deltaArchetype->getVertexPositionXs(meshIdx),
                          deltaArchetype->getVertexPositionYs(meshIdx),
                          deltaArchetype->getVertexPositionZs(meshIdx),
                          memRes);
        if (blockCount == 0) {
            continue;
        }

        for (std::size_t dnaIdx = 0; dnaIdx < dnaCount; dnaIdx++) {
            ConstVec3VectorView dnaMeshView{dnaReaders[dnaIdx]->getVertexPositionXs(meshIdx),
                                            dnaReaders[dnaIdx]->getVertexPositionYs(meshIdx),
                                            dnaReaders[dnaIdx]->getVertexPositionZs(meshIdx)};

            for (std::uint32_t blockIdx = 0u; blockIdx < endIdx; ++blockIdx) {
                for (std::uint32_t i = 0; i < blockSize; i++) {
                    const std::uint32_t vtxIdx = blockIdx * blockSize + i;
                    dnas[meshIdx][blockIdx][dnaIdx].Xs[i] = dnaMeshView.Xs[vtxIdx] - arch[meshIdx].xs[vtxIdx];
                    dnas[meshIdx][blockIdx][dnaIdx].Ys[i] = dnaMeshView.Ys[vtxIdx] - arch[meshIdx].ys[vtxIdx];
                    dnas[meshIdx][blockIdx][dnaIdx].Zs[i] = dnaMeshView.Zs[vtxIdx] - arch[meshIdx].zs[vtxIdx];
                }
            }

            for (std::uint32_t i = 0u; i < remainder; i++) {
                const std::uint32_t vtxIdx = endIdx * blockSize + i;
                dnas[meshIdx][blockCount - 1u][dnaIdx].Xs[i] = dnaMeshView.Xs[vtxIdx] - arch[meshIdx].xs[vtxIdx];
                dnas[meshIdx][blockCount - 1u][dnaIdx].Ys[i] = dnaMeshView.Ys[vtxIdx] - arch[meshIdx].ys[vtxIdx];
                dnas[meshIdx][blockCount - 1u][dnaIdx].Zs[i] = dnaMeshView.Zs[vtxIdx] - arch[meshIdx].zs[vtxIdx];
            }
        }
    }
}

ConstArrayView<XYZTiledMatrix<16u>> NeutralMeshPool::getData() const {
    return {dnas.data(), dnas.size()};
}

Vec3 NeutralMeshPool::getDNAVertexPosition(std::uint16_t dnaIndex, std::uint16_t meshIndex, std::uint32_t vertexIndex) const {
    if ((meshIndex >= arch.size()) || (vertexIndex >= arch[meshIndex].size())) {
        return {};
    }
    if ((meshIndex >= dnas.size()) || (dnas[meshIndex].size() == 0u) || (dnaIndex >= dnas[meshIndex][0].size())) {
        return {};
    }
    Vec3 vertex{arch[meshIndex].xs[vertexIndex], arch[meshIndex].ys[vertexIndex], arch[meshIndex].zs[vertexIndex]};
    const constexpr auto blockSize = XYZTiledMatrix<16u>::value_type::size();
    const auto& block = dnas[meshIndex][vertexIndex / blockSize][dnaIndex];
    const std::uint32_t offset = vertexIndex % blockSize;
    vertex.x += block.Xs[offset];
    vertex.y += block.Ys[offset];
    vertex.z += block.Zs[offset];
    return vertex;
}

Vec3 NeutralMeshPool::getArchetypeVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const {
    if ((meshIndex >= arch.size()) || (vertexIndex >= arch[meshIndex].size())) {
        return {};
    }
    return {arch[meshIndex].xs[vertexIndex], arch[meshIndex].ys[vertexIndex], arch[meshIndex].zs[vertexIndex]};
}

std::uint32_t NeutralMeshPool::getVertexCount(std::uint16_t meshIndex) const {
    if (meshIndex >= arch.size()) {
        return 0u;
    }
    return static_cast<std::uint32_t>(arch[meshIndex].size());
}

}  // namespace gs4
