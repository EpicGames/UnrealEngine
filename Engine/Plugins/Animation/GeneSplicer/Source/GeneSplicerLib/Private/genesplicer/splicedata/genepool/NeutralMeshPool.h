// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/types/Block.h"
#include "genesplicer/types/BlockStorage.h"
#include "pma/TypeDefs.h"

#include <cstdint>

namespace gs4 {

class NeutralMeshPool {
public:
    NeutralMeshPool(MemoryResource* memRes);
    NeutralMeshPool(const Reader* deltaArchetype, ConstArrayView<const Reader*> dnaReaders, MemoryResource* memRes);
    ConstArrayView<XYZTiledMatrix<16u>> getData() const;
    Vec3 getDNAVertexPosition(std::uint16_t dnaIndex, std::uint16_t meshIndex, std::uint32_t vertexIndex) const;
    Vec3 getArchetypeVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const;
    std::uint32_t getVertexCount(std::uint16_t meshIndex) const;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(dnas, arch);
    }

private:
    Vector<XYZTiledMatrix<16u>> dnas;
    Vector<RawVector3Vector> arch;
};

}  // namespace gs4
