// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/splicedata/genepool/GenePoolInterface.h"
#include "genesplicer/types/BlockStorage.h"

#include <cstdint>

namespace gs4 {

class NullGenePoolImpl : public GenePoolInterface {

public:
    NullGenePoolImpl(MemoryResource* memRes_) :
        memRes{memRes_},
        emptyXYZTiledMatrix{memRes},
        emptyVariableWidth16iMatrix{memRes},
        emptyVariableTiledMatrix2DMatrix{memRes},
        blendShapeDeltas{memRes_} {
    }

    ~NullGenePoolImpl();

    std::uint16_t getDNACount() const override {
        return 0u;
    }

    StringView getDNAName(std::uint16_t) const override {
        return {};
    }

    dna::Gender getDNAGender(std::uint16_t) const override {
        return dna::Gender::other;
    }

    std::uint16_t getDNAAge(std::uint16_t) const override {
        return 0u;
    }

    std::uint16_t getMeshCount() const override {
        return 0u;
    }

    std::uint32_t getVertexCount(std::uint16_t) const override {
        return 0u;
    }

    std::uint32_t getVertexPositionCount(std::uint16_t) const override {
        return 0u;
    }

    Vector3 getDNAVertexPosition(std::uint16_t, std::uint16_t, std::uint32_t) const override {
        return {};
    }

    Vector3 getArchetypeVertexPosition(std::uint16_t, std::uint32_t) const override {
        return {};
    }

    std::uint16_t getBlendShapeTargetCount(std::uint16_t) const override {
        return 0u;
    }

    std::uint32_t getSkinWeightsCount(std::uint16_t) const override {
        return 0u;
    }

    std::uint16_t getMaximumInfluencesPerVertex(std::uint16_t) const override {
        return 0u;
    }

    std::uint16_t getNeutralJointCount() const override {
        return 0u;
    }

    std::uint16_t getJointCount() const override {
        return 0u;
    }

    StringView getJointName(std::uint16_t) const override {
        return {};
    }

    std::uint16_t getJointGroupCount() const override {
        return 0u;
    }

    Vector3 getDNANeutralJointWorldTranslation(std::uint16_t, std::uint16_t) const override {
        return {};
    }

    Vector3 getArchetypeNeutralJointWorldTranslation(std::uint16_t) const override {
        return {};
    }

    Vector3 getDNANeutralJointWorldRotation(std::uint16_t, std::uint16_t) const override {
        return {};
    }

    Vector3 getArchetypeNeutralJointWorldRotation(std::uint16_t) const override {
        return {};
    }

    ConstArrayView<XYZTiledMatrix<16u>> getNeutralMeshes() const override {
        return {};
    }

    const BlendShapeDeltas<4u>& getBlendShapeTargetDeltas() const override {
        return blendShapeDeltas;
    }

    ConstArrayView<VariableWidthMatrix<std::uint32_t>> getBlendShapeTargetVertexIndices() const override {
        return {};
    }

    const VariableWidthMatrix<TiledMatrix2D<16u>>& getSkinWeightValues() const override {
        return emptyVariableTiledMatrix2DMatrix;
    }

    ConstArrayView<VariableWidthMatrix<std::uint16_t>> getSkinWeightJointIndices() const override {
        return {};
    }

    const XYZTiledMatrix<16u>& getNeutralJoints(JointAttribute) const override {
        return emptyXYZTiledMatrix;
    }

    const VariableWidthMatrix<std::uint16_t>& getJointBehaviorInputIndices() const override {
        return emptyVariableWidth16iMatrix;
    }

    const VariableWidthMatrix<std::uint16_t>& getJointBehaviorOutputIndices() const override {
        return emptyVariableWidth16iMatrix;
    }

    const VariableWidthMatrix<std::uint16_t>& getJointBehaviorLODs() const override {
        return emptyVariableWidth16iMatrix;
    }

    ConstArrayView<SingleJointBehavior> getJointBehaviorValues() const override {
        return {};
    }

    bool getIsNullGenePool() const override {
        return true;
    }

    MemoryResource* getMemoryResource() const override {
        return memRes;
    }

private:
    MemoryResource* memRes;
    XYZTiledMatrix<16u> emptyXYZTiledMatrix;
    VariableWidthMatrix<std::uint16_t> emptyVariableWidth16iMatrix;
    VariableWidthMatrix<TiledMatrix2D<16u>> emptyVariableTiledMatrix2DMatrix;
    BlendShapeDeltas<4u> blendShapeDeltas;
};

}  // namespace gs4
