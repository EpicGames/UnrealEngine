// Copyright Epic Games, Inc. All Rights Reserved.
#include "genesplicer/splicedata/genepool/OutputIndexTargetOffsets.h"

#include <cstdint>

namespace gs4 {

OutputIndexTargetOffsets::OutputIndexTargetOffsets(std::uint16_t jointCount, MemoryResource* memRes_) :
    offsets{memRes_} {
    offsets = Matrix2D<std::uint8_t>{jointCount, 9u, memRes_};
}

const Matrix2D<std::uint8_t>& OutputIndexTargetOffsets::OutputIndexTargetOffsets::get() const {
    return offsets;
}

void OutputIndexTargetOffsets::mapJointGroup(ConstArrayView<std::uint16_t> outputIndices,
                                             ConstArrayView<std::uint16_t> targetOutputIndices) {
    for (auto outIndex : outputIndices) {
        auto jntIndex = static_cast<std::uint16_t>(outIndex / 9);
        auto outIndexTargetOffsetIt = std::find(targetOutputIndices.begin(), targetOutputIndices.end(), outIndex);
        auto outPos = static_cast<std::size_t>(outIndex % 9);
        offsets[jntIndex][outPos] = static_cast<std::uint8_t>(std::distance(targetOutputIndices.begin(), outIndexTargetOffsetIt));
    }
}

}  // namespace gs4
