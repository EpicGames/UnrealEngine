// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/Matrix.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <cstdint>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

class OutputIndexTargetOffsets {

public:
    explicit OutputIndexTargetOffsets(std::uint16_t jointCount, MemoryResource* memRes);
    void mapJointGroup(ConstArrayView<std::uint16_t> poolOutputIndices, ConstArrayView<std::uint16_t> archOutputIndices);
    const Matrix2D<std::uint8_t>& get() const;

private:
    // [jnt][outPos]=offset
    Matrix2D<std::uint8_t> offsets;
};

}  // namespace gs4
