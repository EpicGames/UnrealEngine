// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/genepool/RawNeutralJoints.h"
#include "genesplicer/utils/Algorithm.h"

namespace gs4 {

namespace {

fmat4 getTransformationMatrix(const RawNeutralJoints& neutralJoints, std::uint16_t jointIndex) {
    const auto& translations = neutralJoints.translations;
    const tdm::fvec3 translation{translations.xs[jointIndex], translations.ys[jointIndex], translations.zs[jointIndex]};
    const auto& rotations = neutralJoints.rotations;
    const tdm::frad3 rotation{tdm::frad{rotations.xs[jointIndex]},
                              tdm::frad{rotations.ys[jointIndex]},
                              tdm::frad{rotations.zs[jointIndex]}};
    return tdm::rotate(rotation, neutralJoints.rotationSequence, neutralJoints.rotationSigns) * tdm::translate(translation);
}

void setNeutralJoint(std::uint16_t jointIndex, const fmat4& transformationMatrix, RawNeutralJoints& neutralJoints) {
    auto& translations = neutralJoints.translations;
    const auto t = extractTranslationVector(transformationMatrix);
    translations.xs[jointIndex] = t[0];
    translations.ys[jointIndex] = t[1];
    translations.zs[jointIndex] = t[2];

    auto& rotations = neutralJoints.rotations;
    const auto r = extractRotationVector(transformationMatrix);
    rotations.xs[jointIndex] = r[0].value;
    rotations.ys[jointIndex] = r[1].value;
    rotations.zs[jointIndex] = r[2].value;
}

}  // namespace

void toWorldSpace(std::function<std::uint16_t(std::uint16_t)> getJointParentIndex, RawNeutralJoints& neutralJoints) {
    MemoryResource* memRes = neutralJoints.translations.xs.get_allocator().getMemoryResource();
    const auto jointCount = static_cast<std::uint16_t>(neutralJoints.translations.xs.size());
    Vector<tdm::fmat4> transformationMatrices{jointCount, {}, memRes};

    for (std::uint16_t jntIdx = 0u; jntIdx < jointCount; jntIdx++) {
        const std::uint16_t parentIdx = getJointParentIndex(jntIdx);
        assert(parentIdx <= jntIdx);
        transformationMatrices[jntIdx] = getTransformationMatrix(neutralJoints, jntIdx);
        if (parentIdx != jntIdx) {
            transformationMatrices[jntIdx] *= transformationMatrices[parentIdx];
        }
        setNeutralJoint(jntIdx, transformationMatrices[jntIdx], neutralJoints);
    }
}

void toLocalSpace(std::function<std::uint16_t(std::uint16_t)> getJointParentIndex, RawNeutralJoints& neutralJoints) {
    MemoryResource* memRes = neutralJoints.translations.xs.get_allocator().getMemoryResource();
    const auto jointCount = static_cast<std::uint16_t>(neutralJoints.translations.xs.size());
    Vector<tdm::fmat4> transformationMatrices{jointCount, {}, memRes};

    for (std::uint16_t jntIdx = 0u; jntIdx < jointCount; jntIdx++) {
        const std::uint16_t parentIdx = getJointParentIndex(jntIdx);
        assert(parentIdx <= jntIdx);
        transformationMatrices[jntIdx] = getTransformationMatrix(neutralJoints, jntIdx);
        if (parentIdx == jntIdx) {
            // global space is local as joint is root
            setNeutralJoint(jntIdx, transformationMatrices[jntIdx], neutralJoints);
            continue;
        }
        const auto& parentGlobal = transformationMatrices[parentIdx];
        const auto localSpace = transformationMatrices[jntIdx] * tdm::inverse(parentGlobal);
        setNeutralJoint(jntIdx, localSpace, neutralJoints);
    }
}

}  // namespace gs4
