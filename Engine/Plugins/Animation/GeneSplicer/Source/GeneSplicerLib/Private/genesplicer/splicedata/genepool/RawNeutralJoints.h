// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Aliases.h"

#include <functional>

namespace gs4 {

struct RawNeutralJoints {
    RawNeutralJoints(MemoryResource* memRes) :
        translations{memRes},
        rotations{memRes} {
    }

    RawNeutralJoints(const Reader* dna, MemoryResource* memRes) :
        translations{dna->getNeutralJointTranslationXs(),
                     dna->getNeutralJointTranslationYs(),
                     dna->getNeutralJointTranslationZs(),
                     memRes},
        rotations{dna->getNeutralJointRotationXs(), dna->getNeutralJointRotationYs(), dna->getNeutralJointRotationZs(), memRes},
        coordinateSystem{dna->getCoordinateSystem()},
        rotationSequence{dna->getRotationSequence()},
        rotationSigns{dna->getRotationSign()} {

        auto toRadians = [](float& r) { r = tdm::radians(r); };
        std::for_each(rotations.xs.begin(), rotations.xs.end(), toRadians);
        std::for_each(rotations.ys.begin(), rotations.ys.end(), toRadians);
        std::for_each(rotations.zs.begin(), rotations.zs.end(), toRadians);
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(translations, rotations, coordinateSystem, rotationSequence, rotationSigns);
    }

    RawVector3Vector translations;
    RawVector3Vector rotations;
    dna::CoordinateSystem coordinateSystem;
    dna::RotationSequence rotationSequence;
    dna::RotationSign rotationSigns;
};

void toWorldSpace(std::function<std::uint16_t(std::uint16_t)> getJointParentIndex, RawNeutralJoints& neutralJoints);
void toLocalSpace(std::function<std::uint16_t(std::uint16_t)> getJointParentIndex, RawNeutralJoints& neutralJoints);

}  // namespace gs4
