// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/genepool/SingleJointBehavior.h"
#include "genesplicer/types/Matrix.h"

#include <algorithm>
#include <cstdint>
#include <iterator>

namespace gs4 {

SingleJointBehavior::SingleJointBehavior(const allocator_type& allocator) :
    outputIndexBlocks{allocator},
    outputOffsets{allocator},
    translationCount{0u} {
    outputIndexBlocks.resize(9u);
}

SingleJointBehavior::SingleJointBehavior(const SingleJointBehavior& rhs, const allocator_type& allocator) :
    outputIndexBlocks{rhs.outputIndexBlocks, allocator},
    outputOffsets{rhs.outputOffsets, allocator},
    translationCount{rhs.translationCount} {
}

SingleJointBehavior::SingleJointBehavior(SingleJointBehavior&& rhs, const allocator_type& allocator) :
    outputIndexBlocks{std::move(rhs.outputIndexBlocks), allocator},
    outputOffsets{std::move(rhs.outputOffsets), allocator},
    translationCount{rhs.translationCount} {
}

void SingleJointBehavior::setOutputPositionValues(std::uint8_t outPos,
                                                  std::uint16_t dnaIdx,
                                                  ConstArrayView<float> valuesToOperate) {
    constexpr std::size_t blockSize = 16u;
    auto vBlockCount = outputIndexBlocks[outPos].rowCount();
    auto vBlockRemainder = valuesToOperate.size() % blockSize;

    for (std::size_t vBlockIndex = 0u; vBlockIndex < vBlockCount - (1u && vBlockRemainder); vBlockIndex++) {
        auto blockValues = valuesToOperate.subview(vBlockIndex * blockSize, blockSize);
        std::copy_n(blockValues.data(), blockValues.size(), outputIndexBlocks[outPos][vBlockIndex][dnaIdx].v);
    }
    std::size_t lastVBLockIndex = static_cast<std::uint16_t>(vBlockCount - 1u);
    auto blockValues = valuesToOperate.subview(lastVBLockIndex * blockSize, vBlockRemainder);
    std::copy_n(blockValues.data(), blockValues.size(), outputIndexBlocks[outPos][lastVBLockIndex][dnaIdx].v);
}

void SingleJointBehavior::setValues(std::uint16_t inputCount,
                                    std::uint8_t outPos,
                                    ConstArrayView<float> deltaArchValues,
                                    ConstArrayView<ConstArrayView<float>> dnaOutputIndexBlocks) {
    auto dnaCount = static_cast<std::uint16_t>(dnaOutputIndexBlocks.size());
    std::size_t blockCount = getBlockCount(inputCount);

    if (std::find(outputOffsets.begin(), outputOffsets.end(), outPos) == outputOffsets.end()) {
        outputOffsets.push_back(outPos);
        std::sort(outputOffsets.begin(), outputOffsets.end());
        if (outPos < 3u) {
            translationCount++;
        }
    }

    auto memRes = outputIndexBlocks.get_allocator().getMemoryResource();
    if (outputIndexBlocks[outPos].rowCount() != blockCount) {
        outputIndexBlocks[outPos] = TiledMatrix2D<16>{blockCount, dnaCount, memRes};
    }
    auto archValues = deltaArchValues;
    Vector<float> negativeArch{inputCount, {}, memRes};
    std::transform(archValues.begin(), archValues.end(), negativeArch.begin(), [](float v) { return v * -1; });
    for (std::uint16_t dnaIdx = 0; dnaIdx < dnaCount; dnaIdx++) {
        Vector<float> valueHolder{negativeArch, memRes};
        auto dnaValues = dnaOutputIndexBlocks[dnaIdx];
        for (std::size_t i = 0; i < dnaValues.size(); i++) {
            valueHolder[i] += dnaValues[i];
        }
        setOutputPositionValues(outPos, dnaIdx, {valueHolder.data(), inputCount});
    }
}

ConstArrayView<TiledMatrix2D<16>> SingleJointBehavior::getValues() const {
    return {outputIndexBlocks.data(), outputIndexBlocks.size()};
}

ConstArrayView<std::uint8_t> SingleJointBehavior::getOutputOffsets() const {
    return {outputOffsets.data(), outputOffsets.size()};
}

std::uint8_t SingleJointBehavior::getTranslationCount() const {
    return translationCount;
}

}  // namespace gs4
