// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/Matrix.h"

#include <cstdint>

namespace gs4 {
struct SingleJointBehavior {

    using allocator_type = Vector<TiledMatrix2D<16>>::allocator_type;

public:
    explicit SingleJointBehavior(const allocator_type& allocator);

    SingleJointBehavior(const SingleJointBehavior& rhs, const allocator_type& allocator);
    SingleJointBehavior(SingleJointBehavior&& rhs, const allocator_type& allocator);

    SingleJointBehavior(const SingleJointBehavior&) = default;
    SingleJointBehavior& operator=(const SingleJointBehavior&) = default;

    SingleJointBehavior(SingleJointBehavior&&) = default;
    SingleJointBehavior& operator=(SingleJointBehavior&&) = default;

    void setValues(std::uint16_t inputCount,
                   std::uint8_t outPos,
                   ConstArrayView<float> deltaArchValues,
                   ConstArrayView<ConstArrayView<float>> dnaValues);

    ConstArrayView<TiledMatrix2D<16>> getValues() const;
    ConstArrayView<std::uint8_t> getOutputOffsets() const;
    std::uint8_t getTranslationCount() const;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(outputIndexBlocks, outputOffsets, translationCount);
    }

private:
    void setOutputPositionValues(std::uint8_t outPos, std::uint16_t dnaIdx, ConstArrayView<float> valuesToOperate);

private:
    // [outPos{[0-9)}][vBlockIndex][dnaIdx][input]=value
    Vector<TiledMatrix2D<16>> outputIndexBlocks;
    Vector<std::uint8_t> outputOffsets;
    std::uint8_t translationCount;
};
}  // namespace gs4
