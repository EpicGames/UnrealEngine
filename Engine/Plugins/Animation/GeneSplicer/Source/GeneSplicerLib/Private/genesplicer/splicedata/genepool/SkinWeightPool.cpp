// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/genepool/SkinWeightPool.h"

#include "genesplicer/TypeDefs.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/Matrix.h"
#include "genesplicer/types/Vec3.h"
#include "genesplicer/utils/Algorithm.h"

#include <cstdint>

namespace gs4 {

SkinWeightPool::SkinWeightPool(MemoryResource* memRes) :
    jointIndices{memRes},
    weights{memRes},
    maximumInfluencesPerVertexPerMesh{memRes} {
}

SkinWeightPool::SkinWeightPool(ConstArrayView<const Reader*> dnas, MemoryResource* memRes) :
    jointIndices{memRes},
    weights{memRes},
    maximumInfluencesPerVertexPerMesh{memRes} {
    std::uint16_t meshCount = dnas[0]->getMeshCount();
    resizeAndReserve(dnas[0]);

    for (std::uint16_t meshIdx = 0u; meshIdx < meshCount; meshIdx++) {
        initializeJointIndices(dnas, meshIdx);
        generateBlocks(dnas, meshIdx);
    }
}

void SkinWeightPool::resizeAndReserve(const Reader* dna) {
    const std::uint16_t meshCount = dna->getMeshCount();
    maximumInfluencesPerVertexPerMesh.reserve(meshCount);

    jointIndices.resize(meshCount);
    std::uint32_t totalSkinWeightsCount = 0u;
    for (std::uint16_t meshIdx = 0u; meshIdx < meshCount; meshIdx++) {
        const std::uint32_t skinWeightsCount = dna->getSkinWeightsCount(meshIdx);
        totalSkinWeightsCount += skinWeightsCount;
        const std::uint16_t maximumInfluencePerVertex = dna->getMaximumInfluencePerVertex(meshIdx);
        maximumInfluencesPerVertexPerMesh.push_back(maximumInfluencePerVertex);
        jointIndices[meshIdx].reserve(skinWeightsCount, skinWeightsCount * maximumInfluencePerVertex);
    }
    weights.reserve(meshCount, totalSkinWeightsCount);
}

void SkinWeightPool::initializeJointIndices(ConstArrayView<const Reader*> dnas, std::uint16_t meshIndex) {
    MemoryResource* memRes = jointIndices.get_allocator().getMemoryResource();
    const std::uint16_t jointCount = dnas[0]->getJointCount();
    const std::uint32_t skinWeightsCount = dnas[0]->getSkinWeightsCount(meshIndex);

    Vector<std::uint16_t> jIndices{jointCount, 0u, memRes};
    Vector<ConstArrayView<std::uint16_t>> jointIndicesPerDNA{dnas.size(), {}, memRes};

    for (std::uint32_t vtxIdx = 0u; vtxIdx < skinWeightsCount; vtxIdx++) {
        for (std::size_t dnaIdx = 0u; dnaIdx < dnas.size(); dnaIdx++) {
            jointIndicesPerDNA[dnaIdx] = dnas[dnaIdx]->getSkinWeightsJointIndices(meshIndex, vtxIdx);
        }
        auto jIndicesEnd = mergeIndices({jointIndicesPerDNA.data(), jointIndicesPerDNA.size()},
                                        static_cast<std::uint16_t>(jointCount - 1u),
                                        jIndices.begin(),
                                        memRes);
        auto jIndicesCount = static_cast<std::size_t>(std::distance(jIndices.begin(), jIndicesEnd));
        jointIndices[meshIndex].appendRow({jIndices.data(), jIndicesCount});
    }
    jointIndices[meshIndex].shrinkToFit();
}

void SkinWeightPool::generateBlocks(ConstArrayView<const Reader*> dnas, std::uint16_t meshIndex) {
    constexpr std::uint16_t blockSize = TiledMatrix2D<16u>::value_type::size();
    std::uint32_t skinWeightCount = getSkinWeightsCount(meshIndex);
    const auto endIdx = skinWeightCount / blockSize;
    const auto remainder = static_cast<std::uint16_t>(skinWeightCount % blockSize);
    const auto blockCount = endIdx + (1u && remainder);
    weights.appendRow(blockCount, TiledMatrix2D<16u>{weights.getAllocator().getMemoryResource()});

    for (std::uint32_t blockIdx = 0u; blockIdx < endIdx; ++blockIdx) {
        generateBlock(dnas, meshIndex, blockIdx, blockSize);
    }
    if (remainder != 0u) {
        generateBlock(dnas, meshIndex, blockCount - 1u, remainder);
    }
}

void SkinWeightPool::generateBlock(ConstArrayView<const Reader*> dnas,
                                   std::uint16_t meshIndex,
                                   std::uint32_t blockIdx,
                                   std::uint16_t blockSize) {
    std::uint32_t vtxIdx = blockIdx * TiledMatrix2D<16u>::value_type::size();
    // This function returns maximum influence count per vertex in a block, also maps joint indices to rows
    auto mapBlockJointIndices = [this, blockSize, meshIndex, vtxIdx](Matrix2D<std::uint16_t>& jointIndexToTargetIndex) {
        std::uint8_t maximumJointCount = 0u;
        for (std::uint16_t vtxOffset = 0; vtxOffset < blockSize; vtxOffset++) {
            const auto vtxJntIndices = jointIndices[meshIndex][vtxIdx + vtxOffset];
            inverseMapping<std::uint16_t>(vtxJntIndices, jointIndexToTargetIndex[vtxOffset]);
            if (maximumJointCount < vtxJntIndices.size()) {
                maximumJointCount = static_cast<std::uint8_t>(vtxJntIndices.size());
            }
        }
        return maximumJointCount;
    };

    MemoryResource* memRes = weights.getAllocator().getMemoryResource();
    Matrix2D<std::uint16_t> jointIndexToTargetIndex{blockSize, dnas[0]->getJointCount(), memRes};
    std::uint8_t maximumJointCountInBlock = mapBlockJointIndices(jointIndexToTargetIndex);

    weights[meshIndex][blockIdx] = TiledMatrix2D<16u>{dnas.size(), maximumJointCountInBlock, memRes};
    for (std::uint16_t dnaIdx = 0; dnaIdx < dnas.size(); dnaIdx++) {
        for (std::uint32_t vtxOffset = 0; vtxOffset < blockSize; vtxOffset++) {
            auto dnaSkinWeights = dnas[dnaIdx]->getSkinWeightsValues(meshIndex, vtxIdx + vtxOffset);
            auto dnaJointIndices = dnas[dnaIdx]->getSkinWeightsJointIndices(meshIndex, vtxIdx + vtxOffset);
            for (std::uint16_t j = 0; j < dnaJointIndices.size(); j++) {
                const std::uint16_t dnaJointIndex = dnaJointIndices[j];
                const std::uint16_t targetIndex = jointIndexToTargetIndex[vtxOffset][dnaJointIndex];
                weights[meshIndex][blockIdx][dnaIdx][targetIndex][vtxOffset] = dnaSkinWeights[j];
            }
        }
    }
}

const VariableWidthMatrix<TiledMatrix2D<16u>>& SkinWeightPool::getWeights() const {
    return weights;
}

ConstArrayView<VariableWidthMatrix<std::uint16_t>> SkinWeightPool::getJointIndices() const {
    return {jointIndices.data(), jointIndices.size()};
}

std::uint16_t SkinWeightPool::getMaximumInfluencesPerVertex(std::uint16_t meshIdx) const {
    if (meshIdx < maximumInfluencesPerVertexPerMesh.size()) {
        return maximumInfluencesPerVertexPerMesh[meshIdx];
    }
    return 0u;
}

std::uint32_t SkinWeightPool::getSkinWeightsCount(std::uint16_t meshIdx) const {
    if (meshIdx < jointIndices.size()) {
        return static_cast<std::uint32_t>(jointIndices[meshIdx].rowCount());
    }
    return 0u;
}

}  // namespace gs4
