// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#include <cstdint>

namespace gs4 {

class SkinWeightPool {
public:
    SkinWeightPool(MemoryResource* memRes);
    SkinWeightPool(ConstArrayView<const Reader*> dnas, MemoryResource* memRes);
    const VariableWidthMatrix<TiledMatrix2D<16u>>& getWeights() const;
    ConstArrayView<VariableWidthMatrix<std::uint16_t>> getJointIndices() const;
    std::uint16_t getMaximumInfluencesPerVertex(std::uint16_t meshIdx) const;
    std::uint32_t getSkinWeightsCount(std::uint16_t meshIdx) const;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(jointIndices, weights, maximumInfluencesPerVertexPerMesh);
    }

private:
    void resizeAndReserve(const Reader* dna);
    void initializeJointIndices(ConstArrayView<const Reader*> dnas, std::uint16_t meshIndex);
    void generateBlocks(ConstArrayView<const Reader*> dnas, std::uint16_t meshIndex);
    void generateBlock(ConstArrayView<const Reader*> dnas,
                       std::uint16_t meshIndex,
                       std::uint32_t blockIdx,
                       std::uint16_t blockSize);

private:
    // [mesh][vtxIdx][jntPos]
    Vector<VariableWidthMatrix<std::uint16_t>> jointIndices;
    // [mesh][blockIdx][dnaIdx][jntPos]{0-16}
    VariableWidthMatrix<TiledMatrix2D<16u>> weights;
    Vector<std::uint16_t> maximumInfluencesPerVertexPerMesh;
};

}  // namespace gs4
