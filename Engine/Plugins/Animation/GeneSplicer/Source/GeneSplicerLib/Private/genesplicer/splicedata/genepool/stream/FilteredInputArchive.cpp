// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/genepool/stream/FilteredInputArchive.h"

#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"
#include "genesplicer/splicedata/genepool/NeutralMeshPool.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cstddef>
#include <cstdint>
#include <functional>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

FilteredInputArchive::FilteredInputArchive(BoundedIOStream* stream_, GenePoolMask mask_, MemoryResource* memRes_) :
    BaseArchive{this, stream_},
    stream{stream_},
    memRes{memRes_},
    mask{mask_} {
}

bool FilteredInputArchive::isMasked(GenePoolMask poolType) {
    return (mask & poolType) != poolType;
}

void FilteredInputArchive::process(GenePoolImpl::MetaData& dest) {
    dest.genePoolMask = dest.genePoolMask & mask;
    BaseArchive::process(dest);
    mask = dest.genePoolMask;
}

void FilteredInputArchive::process(NeutralMeshPool& dest) {
    if (isMasked(GenePoolMask::NeutralMeshes)) {
        return;
    }
    BaseArchive::process(dest);
}

void FilteredInputArchive::process(BlendShapePool& dest) {
    if (isMasked(GenePoolMask::BlendShapes)) {
        return;
    }
    BaseArchive::process(dest);
}

void FilteredInputArchive::process(NeutralJointPool& dest) {
    if (isMasked(GenePoolMask::NeutralJoints)) {
        return;
    }
    BaseArchive::process(dest);
}

void FilteredInputArchive::process(SkinWeightPool& dest) {
    if (isMasked(GenePoolMask::SkinWeights)) {
        return;
    }
    BaseArchive::process(dest);
}

void FilteredInputArchive::process(JointBehaviorPool& dest) {
    if (isMasked(GenePoolMask::JointBehavior)) {
        return;
    }
    BaseArchive::process(dest);
}

}  // namespace gs4
