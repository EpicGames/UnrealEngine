// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/GenePoolMask.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"

#include <terse/archives/binary/InputArchive.h>

#include <cstddef>
#include <cstdint>

namespace gs4 {

class FilteredInputArchive final : public terse::ExtendableBinaryInputArchive<FilteredInputArchive,
                                                                              BoundedIOStream,
                                                                              std::uint64_t,
                                                                              std::uint64_t,
                                                                              terse::Endianness::Network> {
private:
    using BaseArchive = terse::ExtendableBinaryInputArchive<FilteredInputArchive,
                                                            BoundedIOStream,
                                                            std::uint64_t,
                                                            std::uint64_t,
                                                            terse::Endianness::Network>;
    friend Archive<FilteredInputArchive>;

public:
    FilteredInputArchive(BoundedIOStream* stream_, GenePoolMask mask_, MemoryResource* memRes_);

private:
    bool isMasked(GenePoolMask poolType);
    void process(GenePoolImpl::MetaData& dest);
    void process(NeutralMeshPool& dest);
    void process(BlendShapePool& dest);
    void process(NeutralJointPool& dest);
    void process(SkinWeightPool& dest);
    void process(JointBehaviorPool& dest);

    template<typename... Args>
    void process(Args&&... args) {
        BaseArchive::process(std::forward<Args>(args)...);
    }

private:
    BoundedIOStream* stream;
    MemoryResource* memRes;
    GenePoolMask mask;
};

}  // namespace gs4
