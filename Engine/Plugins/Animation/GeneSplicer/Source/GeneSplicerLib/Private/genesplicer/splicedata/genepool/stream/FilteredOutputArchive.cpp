// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/genepool/stream/FilteredOutputArchive.h"

#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"
#include "genesplicer/splicedata/genepool/NeutralMeshPool.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cstddef>
#include <cstdint>
#include <functional>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

FilteredOutputArchive::FilteredOutputArchive(BoundedIOStream* stream_, GenePoolMask mask_, MemoryResource* memRes_) :
    BaseArchive{this, stream_},
    stream{stream_},
    memRes{memRes_},
    mask{mask_} {
}

bool FilteredOutputArchive::isMasked(GenePoolMask poolType) {
    return (mask & poolType) != poolType;
}

void FilteredOutputArchive::process(GenePoolImpl::MetaData& source) {
    BaseArchive::process(source);
    source.genePoolMask = source.genePoolMask & mask;
    mask = source.genePoolMask;
}

void FilteredOutputArchive::process(NeutralMeshPool& source) {
    if (isMasked(GenePoolMask::NeutralMeshes)) {
        return;
    }
    BaseArchive::process(source);
}

void FilteredOutputArchive::process(BlendShapePool& source) {
    if (isMasked(GenePoolMask::BlendShapes)) {
        return;
    }
    BaseArchive::process(source);
}

void FilteredOutputArchive::process(NeutralJointPool& source) {
    if (isMasked(GenePoolMask::NeutralJoints)) {
        return;
    }
    BaseArchive::process(source);
}

void FilteredOutputArchive::process(SkinWeightPool& source) {
    if (isMasked(GenePoolMask::SkinWeights)) {
        return;
    }
    BaseArchive::process(source);
}

void FilteredOutputArchive::process(JointBehaviorPool& source) {
    if (isMasked(GenePoolMask::JointBehavior)) {
        return;
    }
    BaseArchive::process(source);
}

}  // namespace gs4
