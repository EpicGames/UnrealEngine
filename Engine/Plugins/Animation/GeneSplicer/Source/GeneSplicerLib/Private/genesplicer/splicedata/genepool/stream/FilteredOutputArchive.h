// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/GenePoolMask.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"

#include <terse/archives/binary/OutputArchive.h>

#include <cstddef>
#include <cstdint>

namespace gs4 {

class FilteredOutputArchive final : public terse::ExtendableBinaryOutputArchive<FilteredOutputArchive,
                                                                                BoundedIOStream,
                                                                                std::uint64_t,
                                                                                std::uint64_t,
                                                                                terse::Endianness::Network> {
private:
    using BaseArchive = terse::ExtendableBinaryOutputArchive<FilteredOutputArchive,
                                                             BoundedIOStream,
                                                             std::uint64_t,
                                                             std::uint64_t,
                                                             terse::Endianness::Network>;
    friend Archive<FilteredOutputArchive>;

public:
    FilteredOutputArchive(BoundedIOStream* stream_, GenePoolMask mask_, MemoryResource* memRes_);

private:
    void process(GenePoolImpl::MetaData& source);
    void process(NeutralMeshPool& source);
    void process(BlendShapePool& source);
    void process(NeutralJointPool& source);
    void process(SkinWeightPool& source);
    void process(JointBehaviorPool& source);

    bool isMasked(GenePoolMask poolType);

    template<typename... Args>
    void process(Args&&... args) {
        BaseArchive::process(std::forward<Args>(args)...);
    }

private:
    BoundedIOStream* stream;
    MemoryResource* memRes;
    GenePoolMask mask;
};

}  // namespace gs4
