// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/rawgenes/BlendShapeRawGenes.h"

#include "genesplicer/TypeDefs.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/VariableWidthMatrix.h"
#include "genesplicer/types/Vec3.h"
#include "genesplicer/utils/Algorithm.h"

#include <cstddef>
#include <cstdint>
#include <iterator>

namespace gs4 {

BlendShapeRawGenes::BlendShapeRawGenes(MemoryResource* memRes_) :
    memRes(memRes_),
    blendShapeTargets{memRes_} {
}

void BlendShapeRawGenes::set(const Reader* dna) {
    blendShapeTargets.clear();
    std::uint16_t meshCount = dna->getMeshCount();
    for (std::uint16_t meshIdx = 0u; meshIdx < meshCount; meshIdx++) {
        const std::uint32_t vertexCount = dna->getVertexPositionCount(meshIdx);
        const std::uint16_t bsCount = dna->getBlendShapeTargetCount(meshIdx);
        blendShapeTargets.appendRow(bsCount, RawBlendShapeTarget{memRes});

        for (std::uint16_t bsIdx = 0; bsIdx < bsCount; bsIdx++) {
            auto& blendShapeTarget = blendShapeTargets[meshIdx][bsIdx];
            blendShapeTarget.deltas.resize(vertexCount);

            auto dnaIndices = dna->getBlendShapeTargetVertexIndices(meshIdx, bsIdx);
            ConstVec3VectorView dnaDenseView{dna->getBlendShapeTargetDeltaXs(meshIdx, bsIdx),
                                             dna->getBlendShapeTargetDeltaYs(meshIdx, bsIdx),
                                             dna->getBlendShapeTargetDeltaZs(meshIdx, bsIdx)};
            for (std::size_t i = 0u; i < dnaIndices.size(); i++) {
                std::uint32_t idx = dnaIndices[i];
                blendShapeTarget.deltas.xs[idx] = dnaDenseView.Xs[i];
                blendShapeTarget.deltas.ys[idx] = dnaDenseView.Ys[i];
                blendShapeTarget.deltas.zs[idx] = dnaDenseView.Zs[i];
            }
            blendShapeTarget.vertexIndices.assign(dnaIndices.begin(), dnaIndices.end());
            blendShapeTarget.blendShapeChannelIndex = dna->getBlendShapeChannelIndex(meshIdx, bsIdx);
        }
    }
}

void BlendShapeRawGenes::accustomize(ConstArrayView<const VariableWidthMatrix<std::uint32_t>> blendShapeIndicesOther) {
    if (blendShapeTargets.size() == 0u) {
        return;
    }
    for (std::uint16_t meshIdx = 0u; meshIdx < blendShapeIndicesOther.size(); meshIdx++) {
        std::size_t bsCount = blendShapeIndicesOther[meshIdx].rowCount();

        for (std::uint16_t bsIdx = 0u; bsIdx < bsCount; bsIdx++) {
            auto& blendShapeTarget = blendShapeTargets[meshIdx][bsIdx];
            auto& vertexIndices = blendShapeTarget.vertexIndices;
            auto otherVertexIndices = blendShapeIndicesOther[meshIdx][bsIdx];

            DynArray<std::uint32_t> resultingIndices{memRes};
            resultingIndices.resize(vertexIndices.size() + otherVertexIndices.size());
            auto onePastLastAdded = mergeIndices(otherVertexIndices,
                                                 ConstArrayView<std::uint32_t>{vertexIndices},
                                                 static_cast<std::uint32_t>(blendShapeTarget.deltas.size() - 1u),
                                                 resultingIndices.begin(),
                                                 memRes);
            auto resultingSize = static_cast<std::size_t>(std::distance(resultingIndices.begin(), onePastLastAdded));

            resultingIndices.resize(resultingSize);
            vertexIndices = std::move(resultingIndices);
        }
    }
}

const VariableWidthMatrix<RawBlendShapeTarget>& BlendShapeRawGenes::getBlendShapeTargets() const {
    return blendShapeTargets;
}

}  // namespace gs4
