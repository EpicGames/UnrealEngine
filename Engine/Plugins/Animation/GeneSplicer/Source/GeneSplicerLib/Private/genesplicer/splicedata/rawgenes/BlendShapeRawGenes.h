// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Aliases.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#include <cstdint>

namespace gs4 {

class BlendShapeRawGenes {
public:
    explicit BlendShapeRawGenes(MemoryResource* memRes);

    void set(const Reader* dna);
    void accustomize(ConstArrayView<VariableWidthMatrix<std::uint32_t>> blendShapeTargetVertexIndices);

    const VariableWidthMatrix<RawBlendShapeTarget>& getBlendShapeTargets() const;

private:
    MemoryResource* memRes;
    VariableWidthMatrix<RawBlendShapeTarget> blendShapeTargets;
};

}  // namespace gs4
