// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/rawgenes/JointBehaviorRawGenes.h"

#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Aliases.h"
#include "genesplicer/splicedata/rawgenes/JointGroupOutputIndicesMerger.h"
#include "genesplicer/splicedata/rawgenes/RawGenesUtils.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/VariableWidthMatrix.h"
#include "genesplicer/types/Vec3.h"

#include <cstddef>
#include <cstdint>
#include <iterator>

namespace gs4 {

JointBehaviorRawGenes::JointBehaviorRawGenes(MemoryResource* memRes_) :
    memRes(memRes_),
    jointGroups{memRes_},
    jointCount{0u} {
}

void JointBehaviorRawGenes::set(const Reader* dna) {
    std::uint16_t jointGroupCount = dna->getJointGroupCount();
    jointGroups.resize(jointGroupCount, RawJointGroup{memRes});
    jointCount = dna->getJointCount();

    for (std::uint16_t jntGrpIdx = 0; jntGrpIdx < jointGroupCount; jntGrpIdx++) {
        auto& jointGroup = jointGroups[jntGrpIdx];

        auto values = dna->getJointGroupValues(jntGrpIdx);
        jointGroup.values.assign(values.begin(), values.end());

        auto jointIndices = dna->getJointGroupJointIndices(jntGrpIdx);
        jointGroup.jointIndices.assign(jointIndices.begin(), jointIndices.end());

        auto outIndices = dna->getJointGroupOutputIndices(jntGrpIdx);
        jointGroup.outputIndices.assign(outIndices.begin(), outIndices.end());

        auto inputIndices = dna->getJointGroupInputIndices(jntGrpIdx);
        jointGroup.inputIndices.assign(inputIndices.begin(), inputIndices.end());

        auto lods = dna->getJointGroupLODs(jntGrpIdx);
        jointGroup.lods.assign(lods.begin(), lods.end());
    }
}

void JointBehaviorRawGenes::accustomizeJointGroup(ConstArrayView<std::uint16_t> outputIndicesOther,
                                                  ConstArrayView<std::uint16_t> lodsOther,
                                                  std::uint16_t jointGroupIndex) {
    RawJointGroup& jointGroup = jointGroups[jointGroupIndex];

    JointGroupOutputIndicesMerger merger{ConstArrayView<std::uint16_t>{jointGroup.jointIndices}, memRes};
    merger.add(jointGroup);
    merger.add(outputIndicesOther, lodsOther);

    RawJointGroup accustomedJointGroup{memRes};
    accustomedJointGroup.jointIndices.assign(jointGroup.jointIndices.begin(), jointGroup.jointIndices.end());
    accustomedJointGroup.inputIndices.assign(jointGroup.inputIndices.begin(), jointGroup.inputIndices.end());
    accustomedJointGroup.outputIndices.resize(jointGroup.outputIndices.size() + outputIndicesOther.size());
    accustomedJointGroup.lods.resize(jointGroup.lods.size());

    merger.merge(accustomedJointGroup.outputIndices.begin(), accustomedJointGroup.lods.end());

    accustomedJointGroup.outputIndices.resize(accustomedJointGroup.lods[0]);
    accustomedJointGroup.values.resize(accustomedJointGroup.lods[0] * accustomedJointGroup.inputIndices.size());

    copyJointGroupValues(jointGroup, accustomedJointGroup);
    jointGroup = std::move(accustomedJointGroup);
}

void JointBehaviorRawGenes::accustomize(const VariableWidthMatrix<std::uint16_t>& outputIndicesOther,
                                        const VariableWidthMatrix<std::uint16_t>& lodsOther) {
    if (jointGroups.size() == 0) {
        return;
    }

    for (std::uint16_t jntGrpIdx = 0; jntGrpIdx < outputIndicesOther.rowCount(); jntGrpIdx++) {
        accustomizeJointGroup(outputIndicesOther[jntGrpIdx], lodsOther[jntGrpIdx], jntGrpIdx);
    }
}

ConstArrayView<RawJointGroup> JointBehaviorRawGenes::getJointGroups() const {
    return {jointGroups.data(), jointGroups.size()};
}

}  // namespace gs4
