// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Aliases.h"
#include "genesplicer/splicedata/GenePool.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/VariableWidthMatrix.h"
#include "genesplicer/types/Vec3.h"

#include <cstdint>
#include <iterator>

namespace gs4 {

class JointBehaviorRawGenes {
public:
    explicit JointBehaviorRawGenes(MemoryResource* memRes);

    void set(const Reader* dna);
    void accustomize(const VariableWidthMatrix<std::uint16_t>& outputIndicesOther,
                     const VariableWidthMatrix<std::uint16_t>& lodsOther);
    void accustomizeJointGroup(ConstArrayView<std::uint16_t> outputIndicesOther,
                               ConstArrayView<std::uint16_t> lodsOther,
                               std::uint16_t jointGroupIndex);

    ConstArrayView<RawJointGroup> getJointGroups() const;

private:
    MemoryResource* memRes;
    Vector<RawJointGroup> jointGroups;
    std::uint16_t jointCount;
};

}  // namespace gs4
