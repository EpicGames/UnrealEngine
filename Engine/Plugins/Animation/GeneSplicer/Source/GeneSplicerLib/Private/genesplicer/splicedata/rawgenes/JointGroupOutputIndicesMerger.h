// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Aliases.h"
#include "genesplicer/splicedata/rawgenes/RawGenesUtils.h"

#include <cstdint>

namespace gs4 {

struct JointGroupOutputIndicesMerger {
public:
    JointGroupOutputIndicesMerger(ConstArrayView<std::uint16_t> jointIndices_, MemoryResource* memRes_) :
        memRes{memRes_},
        jointIndices{jointIndices_},
        outputIndices{memRes_},
        lods{memRes_} {
    }

    void add(const RawJointGroup& jointGroup) {
        add(ConstArrayView<std::uint16_t>{jointGroup.outputIndices}, ConstArrayView<std::uint16_t>{jointGroup.lods});
    }

    void add(ConstArrayView<std::uint16_t> outputIndices_, ConstArrayView<std::uint16_t> lods_) {
        outputIndices.push_back(outputIndices_);
        lods.push_back(lods_);
    }

    void reserve(std::uint16_t size) {
        outputIndices.reserve(size);
        lods.reserve(size);
    }

    template<typename TOutputIndicesOuputIter, typename ULODReverseOutputIter>
    TOutputIndicesOuputIter merge(TOutputIndicesOuputIter outputIndicesIter, ULODReverseOutputIter lodsOnePastLast) {
        Vector<ConstArrayView<std::uint16_t>> outputIndicesForLODVec{outputIndices.size(), {}, memRes};
        ArrayView<ConstArrayView<std::uint16_t>> outputIndicesForLOD{outputIndicesForLODVec};
        auto maxOutputIndex = static_cast<std::uint16_t>((*std::max_element(jointIndices.begin(), jointIndices.end()) + 1u) * 9u);

        auto onePastLastAdded = outputIndicesIter;
        for (std::size_t lodIdxPlusOne = lods[0].size(); lodIdxPlusOne > 0; --lodIdxPlusOne) {
            auto lodIdx = static_cast<std::uint16_t>(lodIdxPlusOne - 1u);

            for (std::uint16_t i = 0; i < outputIndices.size(); i++) {
                outputIndicesForLOD[i] = getOutputIndicesIntroducedByLOD(outputIndices[i], lods[i], lodIdx);
            }
            onePastLastAdded = mergeIndices({outputIndicesForLOD}, maxOutputIndex, onePastLastAdded, memRes);
            lodsOnePastLast--;
            *lodsOnePastLast = static_cast<std::uint16_t>(std::distance(outputIndicesIter, onePastLastAdded));
        }
        return onePastLastAdded;
    }

private:
    MemoryResource* memRes;
    ConstArrayView<std::uint16_t> jointIndices;
    Vector<ConstArrayView<std::uint16_t>> outputIndices;
    Vector<ConstArrayView<std::uint16_t>> lods;
};

}  // namespace gs4
