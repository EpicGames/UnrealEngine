// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/rawgenes/RawGenes.h"

#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Aliases.h"
#include "genesplicer/neutraljointsplicer/JointAttribute.h"
#include "genesplicer/splicedata/GenePool.h"
#include "genesplicer/splicedata/genepool/GenePoolInterface.h"
#include "genesplicer/splicedata/rawgenes/RawGenesUtils.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/PImplExtractor.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cstdint>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

RawGenes::RawGenes(MemoryResource* memRes_) :
    memRes{memRes_},
    neutralMeshes{memRes},
    vertexCountPerMesh{memRes},
    blendShapes{memRes},
    jointBehavior{memRes},
    neutralJoints{memRes},
    skinWeights{memRes} {
}

void RawGenes::set(const Reader* dna) {
    neutralMeshes = getNeutralMeshesFromDNA(dna, memRes);
    blendShapes.set(dna);
    jointBehavior.set(dna);
    neutralJoints = RawNeutralJoints{dna, memRes};
    auto getJointParentIndex = std::bind(&dna::DefinitionReader::getJointParentIndex, dna, std::placeholders::_1);
    toWorldSpace(getJointParentIndex, neutralJoints);
    skinWeights = getSkinWeightFromDNA(dna, memRes);
}

const RawVector3Vector& RawGenes::getNeutralJoints(JointAttribute jointAttribute) const {
    if (jointAttribute == JointAttribute::Translation) {
        return neutralJoints.translations;
    }
    return neutralJoints.rotations;
}

void RawGenes::accustomize(const GenePoolInterface* genePool) {
    blendShapes.accustomize(genePool->getBlendShapeTargetVertexIndices());
    jointBehavior.accustomize(genePool->getJointBehaviorOutputIndices(), genePool->getJointBehaviorLODs());
}

std::uint16_t RawGenes::getMeshCount() const {
    return static_cast<std::uint16_t>(neutralMeshes.size());
}

std::uint16_t RawGenes::getJointCount() const {
    return static_cast<std::uint16_t>(neutralJoints.translations.size());
}

std::uint32_t RawGenes::getVertexCount(std::uint16_t meshIndex) const {
    if (meshIndex < neutralMeshes.size()) {
        return static_cast<std::uint16_t>(neutralMeshes[meshIndex].size());
    }
    return 0u;
}

std::uint32_t RawGenes::getSkinWeightsCount(std::uint16_t meshIndex) const {
    if (meshIndex < skinWeights.size()) {
        return static_cast<std::uint16_t>(skinWeights[meshIndex].size());
    }
    return 0u;
}

ConstArrayView<RawVector3Vector> RawGenes::getNeutralMeshes() const {
    return {neutralMeshes.data(), neutralMeshes.size()};
}

const VariableWidthMatrix<RawBlendShapeTarget>& RawGenes::getBlendShapeTargets() const {
    return blendShapes.getBlendShapeTargets();
}

ConstArrayView<RawJointGroup> RawGenes::getJointGroups() const {
    return jointBehavior.getJointGroups();
}

ConstArrayView<Vector<RawVertexSkinWeights>> RawGenes::getSkinWeights() const {
    return {skinWeights.data(), skinWeights.size()};
}

}  // namespace gs4
