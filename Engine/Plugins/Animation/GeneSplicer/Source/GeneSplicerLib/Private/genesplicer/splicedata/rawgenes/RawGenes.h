// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Aliases.h"
#include "genesplicer/neutraljointsplicer/JointAttribute.h"
#include "genesplicer/splicedata/genepool/GenePoolInterface.h"
#include "genesplicer/splicedata/genepool/RawNeutralJoints.h"
#include "genesplicer/splicedata/rawgenes/BlendShapeRawGenes.h"
#include "genesplicer/splicedata/rawgenes/JointBehaviorRawGenes.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/VariableWidthMatrix.h"
#include "genesplicer/types/Vec3.h"

#include <cstdint>

namespace gs4 {

class RawGenes {
public:
    explicit RawGenes(MemoryResource* memRes);

    void set(const Reader* dna);

    std::uint16_t getMeshCount() const;
    std::uint16_t getJointCount() const;
    std::uint32_t getVertexCount(std::uint16_t meshIndex) const;
    std::uint32_t getSkinWeightsCount(std::uint16_t meshIndex) const;

    ConstArrayView<RawVector3Vector> getNeutralMeshes() const;
    const VariableWidthMatrix<RawBlendShapeTarget>& getBlendShapeTargets() const;
    ConstArrayView<RawJointGroup> getJointGroups() const;
    const RawVector3Vector& getNeutralJoints(JointAttribute jointAttribute) const;
    ConstArrayView<Vector<RawVertexSkinWeights>> getSkinWeights() const;

    void accustomize(const GenePoolInterface* genePool);

private:
    MemoryResource* memRes;
    Vector<RawVector3Vector> neutralMeshes;
    Vector<std::uint32_t> vertexCountPerMesh;
    BlendShapeRawGenes blendShapes;
    JointBehaviorRawGenes jointBehavior;
    RawNeutralJoints neutralJoints;
    Matrix<RawVertexSkinWeights> skinWeights;
};

}  // namespace gs4
