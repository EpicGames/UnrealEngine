// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Aliases.h"
#include "genesplicer/neutraljointsplicer/JointAttribute.h"
#include "genesplicer/neutraljointsplicer/JointAttributeSpecialization.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/utils/Algorithm.h"
#include "genesplicer/utils/IterTools.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cstdint>
#include <iterator>
#include <numeric>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

inline ConstArrayView<float> getJointValuesForOutputIndex(ConstArrayView<std::uint16_t> outputIndices,
                                                          ConstArrayView<float> values,
                                                          std::size_t inputCount,
                                                          std::uint16_t outputIndex) {
    auto outIndexIt = std::find(outputIndices.begin(), outputIndices.end(), outputIndex);
    std::size_t offset = 0u;
    std::size_t size = 0u;
    if (outIndexIt != outputIndices.end()) {
        offset = static_cast<std::size_t>(std::distance(outputIndices.begin(), outIndexIt)) * inputCount;
        size = inputCount;
    }
    return values.subview(offset, size);
}

inline ConstArrayView<float> getJointValuesForOutputIndex(const RawJointGroup& jointGroup, std::uint16_t outputIndex) {
    return getJointValuesForOutputIndex(ConstArrayView<std::uint16_t>{jointGroup.outputIndices},
                                        ConstArrayView<float>{jointGroup.values},
                                        jointGroup.inputIndices.size(),
                                        outputIndex);
}

inline ConstArrayView<float> getJointValuesForOutputIndex(const Reader* dna,
                                                          std::uint16_t jointGroupIndex,
                                                          std::uint16_t outputIndex) {
    return getJointValuesForOutputIndex(dna->getJointGroupOutputIndices(jointGroupIndex),
                                        dna->getJointGroupValues(jointGroupIndex),
                                        dna->getJointGroupInputIndices(jointGroupIndex).size(),
                                        outputIndex);
}

inline ConstArrayView<std::uint16_t> getOutputIndicesIntroducedByLOD(ConstArrayView<std::uint16_t> outputIndices,
                                                                     ConstArrayView<std::uint16_t> lods,
                                                                     std::uint16_t lodIndex) {
    std::size_t offset = 0u;
    if (lodIndex + 1u < lods.size()) {
        offset = lods[lodIndex + 1u];
    }
    return outputIndices.subview(offset, lods[lodIndex] - offset);
}

inline void copyJointGroupValues(const RawJointGroup& srcJointGroup, RawJointGroup& destJointGroup) {
    assert(srcJointGroup.inputIndices.size() == destJointGroup.inputIndices.size());
    auto insertIt = destJointGroup.values.begin();
    for (std::uint16_t outPos = 0u; outPos < destJointGroup.outputIndices.size(); outPos++) {
        const auto outputIndexValues = getJointValuesForOutputIndex(srcJointGroup, destJointGroup.outputIndices[outPos]);
        safeCopy(outputIndexValues.begin(), outputIndexValues.end(), insertIt, outputIndexValues.size());
        std::advance(insertIt, srcJointGroup.inputIndices.size());
    }
}

inline Vector<RawVector3Vector> getNeutralMeshesFromDNA(const Reader* dna, MemoryResource* memRes) {
    Vector<RawVector3Vector> neutralMeshes{memRes};
    const std::uint16_t meshCount = dna->getMeshCount();
    neutralMeshes.reserve(meshCount);
    for (std::uint16_t meshIdx = 0u; meshIdx < meshCount; meshIdx++) {
        auto xs = dna->getVertexPositionXs(meshIdx);
        auto ys = dna->getVertexPositionYs(meshIdx);
        auto zs = dna->getVertexPositionZs(meshIdx);
        neutralMeshes.emplace_back(xs, ys, zs, memRes);
    }
    return neutralMeshes;
}

template<JointAttribute JT>
inline RawVector3Vector getNeutralJointsFromDNA(const Reader* dna, MemoryResource* memRes) {
    auto values = getJointAttributeValues<JT>(dna);
    return {values.Xs, values.Ys, values.Zs, memRes};
}

inline Matrix<RawVertexSkinWeights> getSkinWeightFromDNA(const Reader* dna, MemoryResource* memRes) {
    Matrix<RawVertexSkinWeights> skinWeights{memRes};
    std::uint16_t meshCount = dna->getMeshCount();
    skinWeights.resize(meshCount);
    for (std::uint16_t meshIdx = 0; meshIdx < meshCount; meshIdx++) {
        std::uint32_t vertexCount = dna->getVertexPositionCount(meshIdx);
        skinWeights[meshIdx].resize(vertexCount, RawVertexSkinWeights{memRes});
        for (std::uint16_t vtxIdx = 0; vtxIdx < vertexCount; vtxIdx++) {
            RawVertexSkinWeights& vtxSkinWeights = skinWeights[meshIdx][vtxIdx];

            auto weights = dna->getSkinWeightsValues(meshIdx, vtxIdx);
            vtxSkinWeights.weights.assign(weights.begin(), weights.end());

            auto jointIndices = dna->getSkinWeightsJointIndices(meshIdx, vtxIdx);
            vtxSkinWeights.jointIndices.assign(jointIndices.begin(), jointIndices.end());
        }
    }
    return skinWeights;
}

}  // namespace gs4
