// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined(GS_AUTODETECT_SSE) && !defined(GS_BUILD_WITH_SSE)
    // Assume any x86 architecture we try to build on will support SSE
    #if defined(i386) || defined(__i386) || defined(__i386__) || defined(__IA32__) || defined(_M_IX86) || defined(__X86__) ||    \
        defined(_X86_) || defined(__THW_INTEL__) || defined(__I86__) || defined(__amd64__) || defined(__amd64) ||                \
        defined(__x86_64__) || defined(__x86_64) || defined(_M_X64) || defined(_M_AMD64)
        #define GS_BUILD_WITH_SSE 1
    #endif
#endif  // GS_AUTODETECT_SSE
