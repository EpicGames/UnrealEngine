// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/CalculationType.h"
#include "genesplicer/system/Detect.h"

#if defined(GS_BUILD_WITH_AVX) && !defined(TRIMD_ENABLE_AVX)
    #define TRIMD_ENABLE_AVX
#endif  // GS_BUILD_WITH_AVX

#if defined(GS_BUILD_WITH_SSE) && !defined(TRIMD_ENABLE_SSE)
    #define TRIMD_ENABLE_SSE
#endif  // GS_BUILD_WITH_SSE

#include <trimd/TRiMD.h>

namespace gs4 {

template<CalculationType CT>
struct GetTF128 {};

template<>
struct GetTF128<CalculationType::Scalar> {
    using type = trimd::scalar::F128;
};

template<>
struct GetTF128<CalculationType::SSE> {
    using type = trimd::F128;
};

template<>
struct GetTF128<CalculationType::AVX> {
    using type = trimd::F128;
};

template<CalculationType CT>
struct GetTF256 {};

template<>
struct GetTF256<CalculationType::Scalar> {
    using type = trimd::scalar::F256;
};

template<>
struct GetTF256<CalculationType::AVX> {
    using type = trimd::F256;
};

}  // namespace gs4
