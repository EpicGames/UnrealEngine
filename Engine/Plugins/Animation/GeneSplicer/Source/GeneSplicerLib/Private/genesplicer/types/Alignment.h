// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace gs4 {

enum class Alignment {
    Aligned,
    Unaligned
};

}  // namespace gs4
