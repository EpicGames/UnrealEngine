// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cassert>
#include <cmath>
#include <cstdint>
#include <iterator>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

namespace {

struct CompareFloat {

    bool operator()(float a, float b) const noexcept {
        return std::abs(a - b) < 0.0001f;
    }
};

}  // namespace

template<std::uint16_t BlockSize>
struct XYZBlock {
    float Xs[BlockSize];
    float Ys[BlockSize];
    float Zs[BlockSize];

    static constexpr std::uint16_t size() {
        return BlockSize;
    }

    static constexpr std::uint16_t totalSize() {
        return BlockSize * 3u;
    }

    bool operator==(const XYZBlock<BlockSize> other) const {
        return std::equal(std::begin(Xs), std::end(Xs), std::begin(other.Xs), CompareFloat{}) &&
               std::equal(std::begin(Ys), std::end(Ys), std::begin(other.Ys), CompareFloat{}) &&
               std::equal(std::begin(Zs), std::end(Zs), std::begin(other.Zs), CompareFloat{});
    }

    template<class Archive>
    void serialize(Archive& archive) {
        for (auto& x : Xs) {
            archive(x);
        }
        for (auto& y : Ys) {
            archive(y);
        }
        for (auto& z : Zs) {
            archive(z);
        }
    }
};

template<std::uint16_t BlockSize>
struct VBlock {
    float v[BlockSize];

    float& operator[](std::size_t index) {
        assert(index < BlockSize);
        return v[index];
    }

    const float& operator[](std::size_t index) const {
        assert(index < BlockSize);
        return v[index];
    }

    static constexpr std::uint16_t size() {
        return BlockSize;
    }

    bool operator==(const VBlock<BlockSize>& other) const {
        return std::equal(std::begin(v), std::end(v), std::begin(other.v), CompareFloat{});
    }

    template<class Archive>
    void serialize(Archive& archive) {
        for (auto& element : v) {
            archive(element);
        }
    }
};

}  // namespace gs4
