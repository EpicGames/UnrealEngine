// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/CalculationType.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/system/SIMD.h"
#include "genesplicer/types/Block.h"
#include "genesplicer/types/Matrix.h"
#include "genesplicer/types/Vec3.h"
#include "genesplicer/utils/IterTools.h"

#include <cstdint>

namespace gs4 {

template<std::uint16_t BlockSize>
using XYZTiledMatrix = AlignedMatrix2D<XYZBlock<BlockSize>>;

template<std::uint16_t BlockSize>
using TiledMatrix2D = Matrix2D<VBlock<BlockSize>, PolyAllocator<VBlock<BlockSize>, static_cast<std::uint16_t>(BlockSize * 4)>>;

inline std::size_t getPaddedCount(std::size_t count, std::uint16_t blockSize) {
    std::size_t remainder = count % blockSize;
    return count - remainder + (blockSize * static_cast<std::size_t>(1u && remainder));
}

inline std::size_t getBlockCount(std::size_t count) {
    constexpr std::uint16_t blockSize = XYZTiledMatrix<16u>::value_type::size();
    return count / blockSize + (static_cast<std::size_t>(1u && count % blockSize));
}

inline RawVector3Vector constructWithPadding(const RawVector3Vector& elements,
                                             MemoryResource* memRes,
                                             std::uint16_t blockSize = XYZTiledMatrix<16u>::value_type::size()) {
    RawVector3Vector paddedElements{memRes};
    paddedElements.resize(getPaddedCount(elements.size(), blockSize));
    safeCopy(elements.xs.begin(), elements.xs.end(), paddedElements.xs.begin(), paddedElements.xs.size());
    safeCopy(elements.ys.begin(), elements.ys.end(), paddedElements.ys.begin(), paddedElements.ys.size());
    safeCopy(elements.zs.begin(), elements.zs.end(), paddedElements.zs.begin(), paddedElements.zs.size());
    return paddedElements;
}

template<CalculationType CT>
struct BlockSplicer {

    static inline void splice(Matrix2DView<const XYZBlock<16u>> dnas,
                              Matrix2DView<const VBlock<16u>> weights,
                              ConstArrayView<std::uint16_t> dnaIndices,
                              Vec3VectorView result,
                              float scale) {
        using TF256 = typename GetTF256<CT>::type;
        float* destX = result.Xs.data();
        float* destY = result.Ys.data();
        float* destZ = result.Zs.data();
        const TF256 scale256(scale);

        for (std::uint32_t blockIdx = 0; blockIdx < weights.rowCount(); blockIdx++) {
            auto sumX0 = TF256::fromAlignedSource(destX);
            auto sumX1 = TF256::fromAlignedSource(destX + 8u);

            auto sumY0 = TF256::fromAlignedSource(destY);
            auto sumY1 = TF256::fromAlignedSource(destY + 8u);

            auto sumZ0 = TF256::fromAlignedSource(destZ);
            auto sumZ1 = TF256::fromAlignedSource(destZ + 8u);

            auto weightBlock = weights[blockIdx];
            auto dnaBlock = dnas[blockIdx];

            for (std::uint16_t dnaIdx : dnaIndices) {
                const auto& weight = weightBlock[dnaIdx];
                const auto& dna = dnaBlock[dnaIdx];

                const auto weight0 = scale256 * TF256::fromAlignedSource(weight.v);
                const auto weight2 = scale256 * TF256::fromAlignedSource(weight.v + 8u);

                sumX0 += TF256::fromAlignedSource(dna.Xs) * weight0;
                sumX1 += TF256::fromAlignedSource(dna.Xs + 8u) * weight2;

                sumY0 += TF256::fromAlignedSource(dna.Ys) * weight0;
                sumY1 += TF256::fromAlignedSource(dna.Ys + 8u) * weight2;

                sumZ0 += TF256::fromAlignedSource(dna.Zs) * weight0;
                sumZ1 += TF256::fromAlignedSource(dna.Zs + 8u) * weight2;
            }

            sumX0.alignedStore(destX);
            sumX1.alignedStore(destX + 8u);

            sumY0.alignedStore(destY);
            sumY1.alignedStore(destY + 8u);

            sumZ0.alignedStore(destZ);
            sumZ1.alignedStore(destZ + 8u);

            destX += 16u;
            destY += 16u;
            destZ += 16u;
        }
    }
};

template<>
struct BlockSplicer<CalculationType::SSE> {

    static inline void splice(Matrix2DView<const XYZBlock<16u>> dnas,
                              Matrix2DView<const VBlock<16u>> weights,
                              ConstArrayView<std::uint16_t> dnaIndices,
                              Vec3VectorView result,
                              float scale) {
        using TF128 = typename GetTF128<CalculationType::SSE>::type;
        float* destX = result.Xs.data();
        float* destY = result.Ys.data();
        float* destZ = result.Zs.data();
        const TF128 scale128(scale);

        for (std::uint32_t blockIdx = 0; blockIdx < weights.rowCount(); blockIdx++) {
            auto sumX0 = TF128::fromAlignedSource(destX);
            auto sumX1 = TF128::fromAlignedSource(destX + 4u);
            auto sumX2 = TF128::fromAlignedSource(destX + 8u);
            auto sumX3 = TF128::fromAlignedSource(destX + 12u);

            auto sumY0 = TF128::fromAlignedSource(destY);
            auto sumY1 = TF128::fromAlignedSource(destY + 4u);
            auto sumY2 = TF128::fromAlignedSource(destY + 8u);
            auto sumY3 = TF128::fromAlignedSource(destY + 12u);

            auto sumZ0 = TF128::fromAlignedSource(destZ);
            auto sumZ1 = TF128::fromAlignedSource(destZ + 4u);
            auto sumZ2 = TF128::fromAlignedSource(destZ + 8u);
            auto sumZ3 = TF128::fromAlignedSource(destZ + 12u);

            auto weightBlock = weights[blockIdx];
            auto dnaBlock = dnas[blockIdx];

            for (std::uint16_t dnaIdx : dnaIndices) {
                const auto& weight = weightBlock[dnaIdx];
                const auto& dna = dnaBlock[dnaIdx];

                const auto weight0 = scale128 * TF128::fromAlignedSource(weight.v);
                const auto weight1 = scale128 * TF128::fromAlignedSource(weight.v + 4u);
                const auto weight2 = scale128 * TF128::fromAlignedSource(weight.v + 8u);
                const auto weight3 = scale128 * TF128::fromAlignedSource(weight.v + 12u);

                sumX0 += TF128::fromAlignedSource(dna.Xs) * weight0;
                sumX1 += TF128::fromAlignedSource(dna.Xs + 4u) * weight1;
                sumX2 += TF128::fromAlignedSource(dna.Xs + 8u) * weight2;
                sumX3 += TF128::fromAlignedSource(dna.Xs + 12u) * weight3;

                sumY0 += TF128::fromAlignedSource(dna.Ys) * weight0;
                sumY1 += TF128::fromAlignedSource(dna.Ys + 4u) * weight1;
                sumY2 += TF128::fromAlignedSource(dna.Ys + 8u) * weight2;
                sumY3 += TF128::fromAlignedSource(dna.Ys + 12u) * weight3;

                sumZ0 += TF128::fromAlignedSource(dna.Zs) * weight0;
                sumZ1 += TF128::fromAlignedSource(dna.Zs + 4u) * weight1;
                sumZ2 += TF128::fromAlignedSource(dna.Zs + 8u) * weight2;
                sumZ3 += TF128::fromAlignedSource(dna.Zs + 12u) * weight3;
            }

            sumX0.alignedStore(destX);
            sumX1.alignedStore(destX + 4u);
            sumX2.alignedStore(destX + 8u);
            sumX3.alignedStore(destX + 12u);

            sumY0.alignedStore(destY);
            sumY1.alignedStore(destY + 4u);
            sumY2.alignedStore(destY + 8u);
            sumY3.alignedStore(destY + 12u);

            sumZ0.alignedStore(destZ);
            sumZ1.alignedStore(destZ + 4u);
            sumZ2.alignedStore(destZ + 8u);
            sumZ3.alignedStore(destZ + 12u);

            destX += 16u;
            destY += 16u;
            destZ += 16u;
        }
    }
};

}  // namespace gs4
