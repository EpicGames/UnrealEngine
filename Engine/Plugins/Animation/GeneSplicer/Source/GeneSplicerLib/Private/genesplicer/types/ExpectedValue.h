// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace gs4 {

template<typename T>
struct ExpectedValue {
    T expected;
    T got;

    explicit ExpectedValue(const T& value) :
        expected{value},
        got{} {
    }

    template<class Archive>
    void load(Archive& archive) {
        archive.label("value");
        archive(got);
    }

    template<class Archive>
    void save(Archive& archive) {
        archive.label("value");
        archive(expected);
        got = expected;
    }

    bool matches() const {
        return (expected == got);
    }
};

}  // namespace gs4
