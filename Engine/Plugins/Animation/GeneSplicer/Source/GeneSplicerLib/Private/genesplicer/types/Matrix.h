// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/Macros.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/Alignment.h"
#include "genesplicer/utils/IterTools.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cassert>
#include <cstddef>
#include <cstdint>
#include <cstring>
#include <functional>
#include <memory>
#include <type_traits>
#include <utility>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

template<class T, typename Allocator = PolyAllocator<T>>
class Matrix2D {
public:
    // *INDENT-OFF*
    using value_type = T;
    using const_value_type = const T;
    using index_type = std::size_t;
    using pointer = value_type*;
    using const_pointer = const value_type*;
    using reference = value_type&;
    using const_reference = const value_type&;
    using slice_type = ArrayView<value_type>;
    using const_slice_type = ConstArrayView<value_type>;
    using allocator_type = Allocator;
    // *INDENT-ON*

public:
    explicit Matrix2D(const allocator_type& alloc) :
        rows{},
        columns{},
        values{alloc} {
    }

    Matrix2D(index_type rows_, index_type columns_, const allocator_type& alloc) :
        rows{rows_},
        columns{columns_},
        values{rows * columns, value_type{}, alloc} {
        // This is not a delegating constructor because the compiler can optimize {} to use memset
    }

    Matrix2D(index_type rows_, index_type columns_, const_reference initial, const allocator_type& alloc) :
        rows{rows_},
        columns{columns_},
        values{rows_ * columns_, initial, alloc} {
    }

    Matrix2D(const Matrix2D& rhs, const allocator_type& alloc) :
        rows{rhs.rows},
        columns{rhs.columns},
        values{rhs.values.data(), rhs.values.size(), alloc} {
    }

    ~Matrix2D() = default;

    Matrix2D(const Matrix2D&) = default;
    Matrix2D& operator=(const Matrix2D&) = default;

    Matrix2D(Matrix2D&&) = default;
    Matrix2D& operator=(Matrix2D&&) = default;

    slice_type operator[](index_type row) {
        assert(row < rowCount());
        return {rowAt(row), columnCount()};
    }

    const_slice_type operator[](index_type row) const {
        assert(row < rowCount());
        return {rowAt(row), columnCount()};
    }

    index_type rowCount() const noexcept {
        return rows;
    }

    index_type columnCount() const noexcept {
        return columns;
    }

    std::size_t size() const noexcept {
        return values.size();
    }

    pointer data() noexcept {
        return values.data();
    }

    const_pointer data() const noexcept {
        return values.data();
    }

    allocator_type getAllocator() const noexcept {
        return values.get_allocator();
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(rows, columns, values);
    }

private:
    pointer rowAt(index_type index) {
        return const_cast<pointer>(const_cast<const Matrix2D*>(this)->rowAt(index));
    }

    const_pointer rowAt(index_type index) const {
        return values.data() + (index * columnCount());
    }

private:
    index_type rows;
    index_type columns;
    terse::DynArray<value_type, Allocator> values;
};

template<typename T>
using AlignedMatrix2D = Matrix2D<T, AlignedAllocator<T>>;

template<class T>
class Matrix2DView {

public:
    using value_type = T;
    using const_value_type = const T;
    using index_type = std::size_t;
    using pointer = value_type*;
    using const_pointer = const value_type*;
    using reference = value_type&;
    using const_reference = const value_type&;
    using slice_type = ArrayView<value_type>;
    using const_slice_type = ConstArrayView<value_type>;
    using size_type = std::size_t;

public:
    Matrix2DView(pointer ptr_, index_type rows_, index_type columns_) :
        ptr{ptr_},
        rows{rows_},
        columns{columns_} {
    }

    template<typename U, typename Allocator>
    Matrix2DView(const Matrix2D<U, Allocator>& matrix) :
        ptr{matrix.data()},
        rows{matrix.rowCount()},
        columns{matrix.columnCount()} {
    }

    template<typename U, typename Allocator>
    Matrix2DView(Matrix2D<U, Allocator>& matrix) :
        ptr{matrix.data()},
        rows{matrix.rowCount()},
        columns{matrix.columnCount()} {
    }

    ~Matrix2DView() = default;

    Matrix2DView(const Matrix2DView& other) :
        ptr{other.ptr},
        rows{other.rowCount()},
        columns{other.columnCount()} {
    }

    Matrix2DView& operator=(const Matrix2DView&) = default;

    Matrix2DView(Matrix2DView&&) = default;
    Matrix2DView& operator=(Matrix2DView&&) = default;

    index_type rowCount() const noexcept {
        return rows;
    }

    index_type columnCount() const noexcept {
        return columns;
    }

    std::size_t size() const noexcept {
        return rowCount() * columnCount();
    }

    slice_type operator[](index_type row) {
        assert(row < rowCount());
        return {rowAt(row), columnCount()};
    }

    const_slice_type operator[](index_type row) const {
        assert(row < rowCount());
        return {rowAt(row), columnCount()};
    }

    pointer data() noexcept {
        return ptr;
    }

    const_pointer data() const noexcept {
        return ptr;
    }

private:
    pointer rowAt(index_type index) {
        return const_cast<pointer>(const_cast<const Matrix2DView*>(this)->rowAt(index));
    }

    const_pointer rowAt(index_type index) const {
        return ptr + (index * columnCount());
    }

private:
    pointer ptr;
    index_type rows;
    index_type columns;
};

template<class T, typename Allocator = PolyAllocator<T>>
class Matrix3D {

public:
    using value_type = T;
    using const_value_type = const T;
    using index_type = std::size_t;
    using pointer = value_type*;
    using const_pointer = const value_type*;
    using reference = value_type&;
    using const_reference = const value_type&;
    using slice_type = Matrix2DView<value_type>;
    using const_slice_type = Matrix2DView<const value_type>;
    using size_type = std::size_t;
    using memory_resource_pointer = MemoryResource*;
    using allocator_type = Allocator;

public:
    explicit Matrix3D(const allocator_type& allocator_) :
        Matrix3D{0u, 0u, 0u, allocator_} {
    }

    Matrix3D(index_type zCount, index_type yCount, index_type xCount, allocator_type allocator_) :
        values{zCount * yCount * xCount, value_type{}, allocator_},
        z{zCount},
        y{yCount},
        x{xCount} {
    }

    Matrix3D(const Matrix3D& rhs, const allocator_type& allocator_) :
        values{rhs.values.data(), rhs.values.size(), allocator_},
        z{rhs.zCount()},
        y{rhs.yCount()},
        x{rhs.xCount()} {
    }

    Matrix3D(const Matrix3D& rhs) :
        values{rhs.values},
        z{rhs.zCount()},
        y{rhs.yCount()},
        x{rhs.xCount()} {
    }

    Matrix3D& operator=(const Matrix3D& rhs) {
        Matrix3D tmp{rhs};
        *this = std::move(tmp);
        return *this;
    }

    Matrix3D(Matrix3D&& rhs) = default;
    Matrix3D& operator=(Matrix3D&& rhs) = default;

    index_type zCount() const noexcept {
        return z;
    }

    index_type yCount() const noexcept {
        return y;
    }

    index_type xCount() const noexcept {
        return x;
    }

    std::size_t size() const noexcept {
        return z * y * x;
    }

    slice_type operator[](index_type zi) {
        assert(zi < zCount());
        return {xAt(zi), yCount(), xCount()};
    }

    const_slice_type operator[](index_type zi) const {
        assert(zi < zCount());
        return {xAt(zi), yCount(), xCount()};
    }

    pointer data() noexcept {
        return values.data();
    }

    const_pointer data() const noexcept {
        return values.data();
    }

    allocator_type getAllocator() const noexcept {
        return values.get_allocator();
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(values, z, y, x);
    }

private:
    pointer xAt(index_type zi) {
        return const_cast<pointer>(const_cast<const Matrix3D*>(this)->xAt(zi));
    }

    const_pointer xAt(index_type zi) const {
        return data() + (zi * yCount() * xCount());
    }

private:
    terse::DynArray<value_type, allocator_type> values;
    index_type z;
    index_type y;
    index_type x;
};

template<typename T>
using AlignedMatrix3D = Matrix3D<T, AlignedAllocator<T>>;

}  // namespace gs4
