// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/CalculationType.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/GenePool.h"

namespace gs4 {

template<class T, typename... Args>
ScopedPtr<T, FactoryDestroy<T>> makePImpl(Args&&... args) {
    return ScopedPtr<T, FactoryDestroy<T>>{FactoryCreate<T>{}(std::forward<Args>(args)...)};
}

template<class T>
struct PImplExtractor {

    using wrapper_type = T;
    using const_wrapper_type = const T;
    using wrapper_reference = wrapper_type&;
    using const_wrapper_reference = const wrapper_type&;
    using wrapper_ptr = wrapper_type*;
    using const_wrapper_ptr = const wrapper_type*;

    template<class TDestroyer>
    using wrapper_smart_ptr = ScopedPtr<wrapper_type, TDestroyer>;
    template<class TDestroyer>
    using const_wrapper_smart_ptr = const ScopedPtr<wrapper_type, TDestroyer>;

    using impl_type = typename T::Impl;
    using impl_ptr = impl_type*;
    using const_impl_ptr = const impl_type*;

    static const_impl_ptr get(const_wrapper_reference holder) {
        return holder.pImpl.get();
    }

    static impl_ptr get(wrapper_reference holder) {
        return const_cast<impl_ptr>(get(const_cast<const_wrapper_reference>(holder)));
    }

    static const_impl_ptr get(const_wrapper_ptr holder) {
        return holder->pImpl.get();
    }

    static impl_ptr get(wrapper_ptr holder) {
        return const_cast<impl_ptr>(get(const_cast<const_wrapper_ptr>(holder)));
    }

    template<class TDestroyer>
    static const_impl_ptr get(const_wrapper_smart_ptr<TDestroyer>& holder) {
        return holder.get()->pImpl.get();
    }

    template<class TDestroyer>
    static impl_ptr get(wrapper_smart_ptr<TDestroyer>& holder) {
        return holder.get()->pImpl.get();
    }
};

}  // namespace gs4
