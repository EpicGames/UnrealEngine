// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/types/ExpectedValue.h"

#include "genesplicer/TypeDefs.h"

namespace gs4 {

template<std::size_t Size>
struct Signature {
    using SignatureValueType = std::array<char, Size>;

    ExpectedValue<SignatureValueType> value;

    explicit Signature(SignatureValueType bytes) :
        value{bytes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("data");
        archive(value);
    }

    bool matches() const {
        return value.matches();
    }
};

}  // namespace gs4
