// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cstddef>
#include <functional>
#include <iterator>
#include <memory>
#include <type_traits>
#include <utility>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

template<class T, typename Allocator = PolyAllocator<T>>
class VariableWidthMatrix {
public:
    // *INDENT-OFF*
    using value_type = T;
    using const_value_type = const T;
    using index_type = std::size_t;
    using pointer = value_type*;
    using const_pointer = const value_type*;
    using reference = value_type&;
    using const_reference = const value_type&;
    using slice_type = ArrayView<value_type>;
    using const_slice_type = ConstArrayView<value_type>;
    using allocator_type = Allocator;
    // *INDENT-ON*
public:
    struct Iterator {
        using iterator_category = std::bidirectional_iterator_tag;
        using difference_type = std::ptrdiff_t;
        using value_type = T;
        using pointer = value_type*;
        using reference = value_type&;

        Iterator(const_pointer valuesPtr_, Vector<index_type>::const_iterator indicesIter_) :
            indicesIter{indicesIter_},
            valuesPtr{valuesPtr_} {
        }

        ConstArrayView<T> operator*() const {
            auto ptr = valuesPtr + (*indicesIter);
            auto sz = *(indicesIter + 1) - (*indicesIter);
            return {ptr, sz};
        }

        Iterator& operator++() {
            indicesIter++;
            return *this;
        }

        Iterator operator++(int) {
            Iterator tmp = *this;
            ++(*this);
            return tmp;
        }

        Iterator& operator--() {
            indicesIter--;
            return *this;
        }

        Iterator operator--(int) {
            Iterator tmp = *this;
            --(*this);
            return tmp;
        }

        friend bool operator==(const Iterator& a, const Iterator& b) {
            return a.indicesIter == b.indicesIter && a.valuesPtr == b.valuesPtr;
        }

        friend bool operator!=(const Iterator& a, const Iterator& b) {
            return !(a == b);
        }

    private:
        Vector<index_type>::const_iterator indicesIter;
        const_pointer valuesPtr;
    };

public:
    explicit VariableWidthMatrix(const allocator_type& allocator) :
        rowIndices{1u, static_cast<index_type>(0), allocator},
        values{allocator} {
    }

    VariableWidthMatrix(const VariableWidthMatrix& rhs, const allocator_type& alloc) :
        rowIndices{rhs.rowIndices, alloc},
        values{rhs.values, alloc} {
    }

    VariableWidthMatrix(VariableWidthMatrix&& rhs, const allocator_type& alloc) :
        rowIndices{rhs.rowIndices, alloc},
        values{std::move(rhs.values), alloc} {
    }

    VariableWidthMatrix(const VariableWidthMatrix&) = default;
    VariableWidthMatrix& operator=(const VariableWidthMatrix&) = default;

    VariableWidthMatrix(VariableWidthMatrix&&) = default;
    VariableWidthMatrix& operator=(VariableWidthMatrix&&) = default;

    ~VariableWidthMatrix() = default;

    slice_type operator[](index_type row) {
        assert(row < rowCount());
        return {rowAt(row), columnCount(row)};
    }

    const_slice_type operator[](index_type row) const {
        assert(row < rowCount());
        return {rowAt(row), columnCount(row)};
    }

    index_type rowCount() const noexcept {
        return rowIndices.size() - 1u;
    }

    index_type columnCount(index_type row) const {
        assert(row < rowCount());
        return rowIndices[row + 1] - rowIndices[row];
    }

    std::size_t size() const noexcept {
        return values.size();
    }

    pointer data() noexcept {
        return values.data();
    }

    const_pointer data() const noexcept {
        return values.data();
    }

    allocator_type getAllocator() const noexcept {
        return values.get_allocator();
    }

    void appendRow(const_slice_type row) {
        values.insert(values.end(), row.begin(), row.end());
        rowIndices.push_back(rowIndices[rowIndices.size() - 1u] + static_cast<index_type>(row.size()));
    }

    void appendRow(index_type columnCount) {
        values.insert(values.end(), columnCount, value_type{});
        rowIndices.push_back(rowIndices[rowIndices.size() - 1u] + columnCount);
    }

    void appendRow(index_type columnCount, const_reference value) {
        values.insert(values.end(), columnCount, value);
        rowIndices.push_back(rowIndices[rowIndices.size() - 1u] + columnCount);
    }

    void append(index_type row, const_reference element) {
        insert(row, columnCount(row), element);
    }

    void insert(index_type row, index_type column, const_reference element) {
        assert(row < rowCount());
        assert(column <= columnCount(row));
        values.insert(std::next(values.begin(), static_cast<int>(rowIndices[row] + column)), element);
        for (std::size_t i = row + 1u; i < rowIndices.size(); i++) {
            rowIndices[i]++;
        }
    }

    void reserve(index_type rowCount, index_type valueCount) {
        rowIndices.reserve(rowCount);
        values.reserve(valueCount);
    }

    void shrinkToFit() {
        rowIndices.shrink_to_fit();
        values.shrink_to_fit();
    }

    void clear() {
        rowIndices.resize(1u);
        values.clear();
    }

    Iterator begin() {
        return Iterator{values.data(), rowIndices.begin()};
    }

    const Iterator begin() const {
        return Iterator{values.data(), rowIndices.begin()};
    }

    Iterator end() {
        return Iterator{values.data(), std::prev(rowIndices.end())};
    }

    const Iterator end() const {
        return Iterator{values.data(), std::prev(rowIndices.end())};
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(rowIndices, values);
    }

private:
    pointer rowAt(index_type index) {
        return const_cast<pointer>(const_cast<const VariableWidthMatrix*>(this)->rowAt(index));
    }

    const_pointer rowAt(index_type index) const {
        return values.data() + rowIndices[index];
    }

private:
    Vector<index_type> rowIndices;
    Vector<value_type, Allocator> values;
};

template<typename T>
using AlignedVariableWidthMatrix = VariableWidthMatrix<T, AlignedAllocator<T>>;
}  // namespace gs4
