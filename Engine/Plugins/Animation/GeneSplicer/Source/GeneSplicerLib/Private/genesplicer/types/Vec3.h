// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Aliases.h"

namespace gs4 {

using Vec3 = Vector3;

struct ConstVec3VectorView {
    ConstArrayView<float> Xs;
    ConstArrayView<float> Ys;
    ConstArrayView<float> Zs;
};

struct Vec3VectorView {
    ArrayView<float> Xs;
    ArrayView<float> Ys;
    ArrayView<float> Zs;

    Vec3VectorView(RawVector3Vector& vec) :
        Xs{vec.xs},
        Ys{vec.ys},
        Zs{vec.zs} {
    }
};

}  // namespace gs4
