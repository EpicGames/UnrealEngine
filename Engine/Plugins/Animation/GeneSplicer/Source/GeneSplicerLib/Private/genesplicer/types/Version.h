// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/types/ExpectedValue.h"

#include <cstdint>

namespace gs4 {

struct ExpectedVersion {
    ExpectedValue<std::uint16_t> generation;
    ExpectedValue<std::uint16_t> version;

    ExpectedVersion(std::uint16_t generation_, std::uint16_t version_) :
        generation{generation_},
        version{version_} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(generation, version);
    }

    bool matches() const {
        return (generation.matches() && version.matches());
    }
};
}  // namespace gs4
