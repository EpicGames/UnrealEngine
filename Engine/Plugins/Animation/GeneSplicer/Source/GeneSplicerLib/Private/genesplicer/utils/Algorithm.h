// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"
#include "genesplicer/types/Aliases.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <iterator>
#include <type_traits>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

template<typename T, typename OutIter, typename std::enable_if<std::is_unsigned<T>::value, bool>::type = true>
OutIter mergeIndices(ConstArrayView<ConstArrayView<T>> multipleIndices,
                     T maximumIndex,
                     OutIter outputIter,
                     MemoryResource* memRes) {
    T maximumSize = static_cast<T>(maximumIndex + 1u);
    DynArray<char> contains{maximumSize, cFalse, memRes};
    const auto mark = [&contains](T index) { contains[index] = cTrue; };
    for (auto indices : multipleIndices) {
        std::for_each(indices.begin(), indices.end(), mark);
    }
    auto dest = outputIter;
    for (T i = 0; i < maximumSize; i++) {
        if (contains[i]) {
            *dest = i;
            ++dest;
        }
    }
    return dest;
}

template<typename T, typename OutIter, typename std::enable_if<std::is_unsigned<T>::value, bool>::type = true>
OutIter mergeIndices(ConstArrayView<T> indices,
                     ConstArrayView<T> otherIndices,
                     T maximumIndex,
                     OutIter outputIter,
                     MemoryResource* memRes) {
    std::array<ConstArrayView<T>, 2u> multipleIndices{indices, otherIndices};
    return mergeIndices({multipleIndices.data(), multipleIndices.size()}, maximumIndex, outputIter, memRes);
}

template<typename T>
static void inverseMapping(ConstArrayView<T> indices, ArrayView<T> inverseIndices) {
    assert(indices.size() == 0 || *std::max_element(indices.begin(), indices.end()) < inverseIndices.size());
    for (std::uint16_t i = 0; i < indices.size(); i++) {
        inverseIndices[indices[i]] = i;
    }
}

inline fmat4 extractTranslationMatrix(const fmat4& transformationMatrix) {
    auto t = fmat4::identity();
    t(3, 0) = transformationMatrix(3, 0);
    t(3, 1) = transformationMatrix(3, 1);
    t(3, 2) = transformationMatrix(3, 2);
    return t;
}

inline fmat4 extractRotationMatrix(const fmat4& transformationMatrix) {
    auto r = transformationMatrix;
    r(3, 0) = 0.0f;
    r(3, 1) = 0.0f;
    r(3, 2) = 0.0f;
    return r;
}

inline fvec3 extractTranslationVector(const fmat4& transformationMatrix) {
    return {transformationMatrix(3, 0), transformationMatrix(3, 1), transformationMatrix(3, 2)};
}

inline frad3 extractRotationVector(const fmat4& transformationMatrix) {
    frad3 angle{};
    const auto r = extractRotationMatrix(transformationMatrix);
    const auto r02 = r(0, 2);
    if (r02 < 1.0f) {
        if (r02 > -1.0f) {
            angle[0] = tdm::frad{std::atan2(r(1, 2), r(2, 2))};
            angle[1] = tdm::frad{std::asin(-r02)};
            angle[2] = tdm::frad{std::atan2(r(0, 1), r(0, 0))};
        } else {
            angle[0] = tdm::frad{std::atan2(-r(2, 1), r(1, 1))};
            angle[1] = tdm::frad{static_cast<float>(tdm::pi() / 2.0f)};
            angle[2] = tdm::frad{0};
        }
    } else {
        angle[0] = tdm::frad{-std::atan2(-r(2, 1), r(1, 1))};
        angle[1] = tdm::frad{static_cast<float>(-tdm::pi() / 2.0f)};
        angle[2] = tdm::frad{0};
    }
    return angle;
}

}  // namespace gs4
