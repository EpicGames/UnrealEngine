// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/Macros.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <iterator>
#if defined(_MSC_VER) && !defined(__clang__) && (_MSC_VER < 1938) && (_MSC_VER >= 1900) && (__cplusplus >= 202002L)
    #include <span>
#endif
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

template<class TIterator, typename TDistance>
TIterator advanced(TIterator source, TDistance distance) {
    std::advance(source, distance);
    return source;
}

template<class TIterator, class Predicate>
typename TIterator::difference_type advanceWhile(TIterator& it, const TIterator& end, Predicate pred) {
    const auto start = it;
    while (it != end && pred(*it)) {
        ++it;
    }
    return it - start;
}

template<class TInputIterator, class TOutputIterator>
inline void safeCopy(const TInputIterator& start, const TInputIterator& end, TOutputIterator destination, std::size_t size) {
#if defined(_MSC_VER) && !defined(__clang__) && (_MSC_VER < 1938)
    #if (_MSC_VER >= 1900) && (__cplusplus >= 202002L)
    std::copy(start, end, std::span{destination, size}.begin());
    #else
    std::copy(start, end, stdext::make_checked_array_iterator(destination, size));
    #endif
#else
    UNUSED(size);
    std::copy(start, end, destination);
#endif
}

}  // namespace gs4
