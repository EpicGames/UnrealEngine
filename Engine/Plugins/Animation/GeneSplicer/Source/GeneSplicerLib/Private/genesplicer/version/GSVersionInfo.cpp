// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/version/VersionInfo.h"

#include "genesplicer/version/Version.h"

#include <cstring>

namespace gs4 {

namespace {

constexpr int majorVersion = GS_MAJOR_VERSION;
constexpr int minorVersion = GS_MINOR_VERSION;
constexpr int patchVersion = GS_PATCH_VERSION;
constexpr const char* versionString = GS_VERSION_STRING;

}  // namespace

int VersionInfo::getMajorVersion() {
    return majorVersion;
}

int VersionInfo::getMinorVersion() {
    return minorVersion;
}

int VersionInfo::getPatchVersion() {
    return patchVersion;
}

StringView VersionInfo::getVersionString() {
    return {versionString, std::strlen(versionString)};
}

}  // namespace gs4
