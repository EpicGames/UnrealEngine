// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/RegionAffiliation.h"
#include "raf/TypeDefs.h"

namespace raf {

class BaseImpl {
protected:
    explicit BaseImpl(MemoryResource* memRes_) :
        memRes{memRes_},
        regionAffiliation{memRes} {
    }

    ~BaseImpl() = default;

    BaseImpl(const BaseImpl&) = delete;
    BaseImpl& operator=(const BaseImpl&) = delete;

    BaseImpl(BaseImpl&& rhs) = delete;
    BaseImpl& operator=(BaseImpl&&) = delete;

public:
    MemoryResource* getMemoryResource() {
        return memRes;
    }

protected:
    MemoryResource* memRes;
    RegionAffiliation regionAffiliation;
};

}  // namespace raf
