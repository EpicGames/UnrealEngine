// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/BaseImpl.h"
#include "raf/TypeDefs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstddef>
#include <limits>
#include <tuple>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace raf {

template<class TReaderBase>
class ReaderImpl : public TReaderBase, public virtual BaseImpl {
public:
    using ReaderInterface = TReaderBase;

public:
    explicit ReaderImpl(MemoryResource* memRes_);

    // JointRegionAffiliationReader methods
    std::uint16_t getJointCount() const override;
    ConstArrayView<std::uint16_t> getJointRegionIndices(std::uint16_t jointIndex) const override;
    ConstArrayView<float> getJointRegionAffiliation(std::uint16_t jointIndex) const override;

    // VertexRegionAffiliationReader methods
    std::uint16_t getMeshCount() const override;
    std::uint32_t getVertexCount(std::uint16_t meshIndex) const override;
    ConstArrayView<std::uint16_t> getVertexRegionIndices(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override;
    ConstArrayView<float> getVertexRegionAffiliation(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override;

    // RegionAffiliationReader methods
    std::uint16_t getRegionCount() const override;
    StringView getRegionName(std::uint16_t regionIndex) const override;
};

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4589)
#endif
template<class TReaderBase>
inline ReaderImpl<TReaderBase>::ReaderImpl(MemoryResource* memRes_) :
    BaseImpl{memRes_} {
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4505)
#endif
template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getJointCount() const {
    return static_cast<std::uint16_t>(regionAffiliation.jointRegions.size());
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getJointRegionIndices(std::uint16_t jointIndex) const {
    if (getJointCount() <= jointIndex) {
        return {};
    }
    return ConstArrayView<std::uint16_t>{regionAffiliation.jointRegions[jointIndex].regionIndices};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getJointRegionAffiliation(std::uint16_t jointIndex) const {
    if (getJointCount() <= jointIndex) {
        return {};
    }
    return ConstArrayView<float>{regionAffiliation.jointRegions[jointIndex].values};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getMeshCount() const {
    return static_cast<std::uint16_t>(regionAffiliation.vertexRegions.size());
}

template<class TReaderBase>
inline std::uint32_t ReaderImpl<TReaderBase>::getVertexCount(std::uint16_t meshIndex) const {
    if (getMeshCount() <= meshIndex) {
        return {};
    }
    return static_cast<std::uint32_t>(regionAffiliation.vertexRegions[meshIndex].size());
}

template<class TReaderBase>
inline ConstArrayView<std::uint16_t> ReaderImpl<TReaderBase>::getVertexRegionIndices(std::uint16_t meshIndex,
                                                                                     std::uint32_t vertexIndex) const {
    if ((getMeshCount() <= meshIndex) || (getVertexCount(meshIndex) <= vertexIndex)) {
        return {};
    }
    return ConstArrayView<std::uint16_t>{regionAffiliation.vertexRegions[meshIndex][vertexIndex].regionIndices};
}

template<class TReaderBase>
inline ConstArrayView<float> ReaderImpl<TReaderBase>::getVertexRegionAffiliation(std::uint16_t meshIndex,
                                                                                 std::uint32_t vertexIndex) const {
    if ((getMeshCount() <= meshIndex) || (getVertexCount(meshIndex) <= vertexIndex)) {
        return {};
    }
    return ConstArrayView<float>{regionAffiliation.vertexRegions[meshIndex][vertexIndex].values};
}

template<class TReaderBase>
inline std::uint16_t ReaderImpl<TReaderBase>::getRegionCount() const {
    return static_cast<std::uint16_t>(regionAffiliation.regionNames.size());
}

template<class TReaderBase>
inline StringView ReaderImpl<TReaderBase>::getRegionName(std::uint16_t regionIndex) const {
    if (regionIndex < regionAffiliation.regionNames.size()) {
        return StringView{regionAffiliation.regionNames[regionIndex]};
    }
    return {};
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif

}  // namespace raf
