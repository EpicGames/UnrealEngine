// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/TypeDefs.h"
#include "raf/types/SerializationTypes.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <cstdint>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace raf {

struct RawRegionAffiliation {
    Vector<std::uint16_t> regionIndices;
    Vector<float> values;

    explicit RawRegionAffiliation(MemoryResource* memRes) :
        regionIndices{memRes},
        values{memRes} {
    }

    template<typename TUInt16Container, typename TFloatContainer>
    RawRegionAffiliation(const TUInt16Container& regionIndices_, const TFloatContainer& values_, MemoryResource* memRes) :
        regionIndices{regionIndices_.size(), 0u, memRes},
        values{values_.size(), 0.0f, memRes} {
        assert(regionIndices_.size() == values_.size());
        std::copy(regionIndices_.cbegin(), regionIndices_.cend(), regionIndices.begin());
        std::copy(values_.cbegin(), values_.cend(), values.begin());
    }

    template<class Archive>
    void serialize(Archive& archive) {
        auto version = static_cast<Version*>(archive.getUserData());

        if (version->matches(1, 0)) {
            archive(values, regionIndices);
        } else if (version->matches(1, 1)) {
            archive(regionIndices, values);
        }
    }
};

struct RegionAffiliation {
    Signature<3> signature{{'R', 'A', 'F'}};
    Version version{1, 1};
    Matrix<RawRegionAffiliation> vertexRegions;
    Vector<RawRegionAffiliation> jointRegions;
    Vector<String<char>> regionNames;
    Signature<3> eof{{'F', 'A', 'R'}};

    explicit RegionAffiliation(MemoryResource* memRes) :
        vertexRegions{memRes},
        jointRegions{memRes},
        regionNames{memRes} {
    }

    template<class Archive>
    void load(Archive& archive) {
        archive(signature, version);
        if (!signature.matches()) {
            return;
        }

        auto oldUserData = archive.getUserData();
        archive.setUserData(&version);

        if (version.matches(1, 0)) {
            std::uint16_t regionCount = {};
            archive(vertexRegions, jointRegions, regionCount, eof);
            regionNames.resize(regionCount);
        } else if (version.matches(1, 1)) {
            archive(vertexRegions, jointRegions, regionNames, eof);
        }

        archive.setUserData(oldUserData);
    }

    template<class Archive>
    void save(Archive& archive) {
        archive(signature, version);

        auto oldUserData = archive.getUserData();
        archive.setUserData(&version);

        if (version.matches(1, 0)) {
            auto regionCount = static_cast<std::uint16_t>(regionNames.size());
            archive(vertexRegions, jointRegions, regionCount, eof);
        } else if (version.matches(1, 1)) {
            archive(vertexRegions, jointRegions, regionNames, eof);
        }

        archive.setUserData(oldUserData);
    }
};

}  // namespace raf
