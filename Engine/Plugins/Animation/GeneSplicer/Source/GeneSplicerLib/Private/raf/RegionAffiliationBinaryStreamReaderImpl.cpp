// Copyright Epic Games, Inc. All Rights Reserved.

#include "raf/RegionAffiliationBinaryStreamReaderImpl.h"

#include "raf/RegionAffiliation.h"
#include "raf/TypeDefs.h"

namespace raf {

#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wglobal-constructors"
sc::StatusProvider RegionAffiliationBinaryStreamReaderImpl::status{IOError, SignatureMismatchError, VersionMismatchError};
#pragma clang diagnostic pop

RegionAffiliationBinaryStreamReader::~RegionAffiliationBinaryStreamReader() = default;

RegionAffiliationBinaryStreamReader* RegionAffiliationBinaryStreamReader::create(BoundedIOStream* stream,
                                                                                 MemoryResource* memRes) {
    PolyAllocator<RegionAffiliationBinaryStreamReaderImpl> alloc{memRes};
    return alloc.newObject(stream, memRes);
}

void RegionAffiliationBinaryStreamReader::destroy(RegionAffiliationBinaryStreamReader* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto reader = static_cast<RegionAffiliationBinaryStreamReaderImpl*>(instance);
    PolyAllocator<RegionAffiliationBinaryStreamReaderImpl> alloc{reader->getMemoryResource()};
    alloc.deleteObject(reader);
}

RegionAffiliationBinaryStreamReaderImpl::RegionAffiliationBinaryStreamReaderImpl(BoundedIOStream* stream_,
                                                                                 MemoryResource* memRes_) :
    BaseImpl{memRes_},
    ReaderImpl{memRes_},
    stream{stream_} {
}

void RegionAffiliationBinaryStreamReaderImpl::read() {
    status.reset();
    trio::StreamScope scope{stream};
    if (!Status::isOk()) {
        status.set(IOError, Status::get().message);
        return;
    }
    terse::BinaryInputArchive<BoundedIOStream> archive{stream};
    archive >> regionAffiliation;

    if (!Status::isOk()) {
        return;
    }

    if (!regionAffiliation.signature.matches()) {
        status.set(SignatureMismatchError,
                   regionAffiliation.signature.value.expected.data(),
                   regionAffiliation.signature.value.got.data());
        return;
    }

    if (!regionAffiliation.version.supported()) {
        status.set(VersionMismatchError, regionAffiliation.version.generation, regionAffiliation.version.version);
        return;
    }
}

}  // namespace raf
