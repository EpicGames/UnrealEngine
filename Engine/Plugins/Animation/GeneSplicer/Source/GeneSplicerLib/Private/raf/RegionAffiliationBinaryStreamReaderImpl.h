// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/ReaderImpl.h"
#include "raf/RegionAffiliationBinaryStreamReader.h"
#include "raf/TypeDefs.h"

namespace raf {

class RegionAffiliationBinaryStreamReaderImpl : public ReaderImpl<RegionAffiliationBinaryStreamReader> {
public:
    RegionAffiliationBinaryStreamReaderImpl(BoundedIOStream* stream_, MemoryResource* memRes_);

    void read() override;

private:
    static sc::StatusProvider status;
    BoundedIOStream* stream;
};

}  // namespace raf
