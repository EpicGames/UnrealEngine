// Copyright Epic Games, Inc. All Rights Reserved.

#include "raf/RegionAffiliationBinaryStreamWriterImpl.h"

#include "raf/RegionAffiliation.h"
#include "raf/TypeDefs.h"

namespace raf {

#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wglobal-constructors"
sc::StatusProvider RegionAffiliationBinaryStreamWriterImpl::status{IOError};
#pragma clang diagnostic pop

RegionAffiliationBinaryStreamWriter::~RegionAffiliationBinaryStreamWriter() = default;

RegionAffiliationBinaryStreamWriter* RegionAffiliationBinaryStreamWriter::create(BoundedIOStream* stream,
                                                                                 MemoryResource* memRes) {
    PolyAllocator<RegionAffiliationBinaryStreamWriterImpl> alloc{memRes};
    return alloc.newObject(stream, memRes);
}

void RegionAffiliationBinaryStreamWriter::destroy(RegionAffiliationBinaryStreamWriter* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto writer = static_cast<RegionAffiliationBinaryStreamWriterImpl*>(instance);
    PolyAllocator<RegionAffiliationBinaryStreamWriterImpl> alloc{writer->getMemoryResource()};
    alloc.deleteObject(writer);
}

RegionAffiliationBinaryStreamWriterImpl::RegionAffiliationBinaryStreamWriterImpl(BoundedIOStream* stream_,
                                                                                 MemoryResource* memRes_) :
    BaseImpl{memRes_},
    WriterImpl{memRes_},
    stream{stream_} {
}

void RegionAffiliationBinaryStreamWriterImpl::write() {
    status.reset();
    trio::StreamScope scope{stream};
    if (!Status::isOk()) {
        status.set(IOError, Status::get().message);
        return;
    }
    terse::BinaryOutputArchive<BoundedIOStream> archive{stream};
    archive << regionAffiliation;
    archive.sync();
}

}  // namespace raf
