// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/RegionAffiliationBinaryStreamWriter.h"
#include "raf/TypeDefs.h"
#include "raf/WriterImpl.h"

namespace raf {

class RegionAffiliationBinaryStreamWriterImpl : public WriterImpl<RegionAffiliationBinaryStreamWriter> {
public:
    RegionAffiliationBinaryStreamWriterImpl(BoundedIOStream* stream_, MemoryResource* memRes_);

    void write() override;

private:
    static sc::StatusProvider status;

    BoundedIOStream* stream;
};

}  // namespace raf
