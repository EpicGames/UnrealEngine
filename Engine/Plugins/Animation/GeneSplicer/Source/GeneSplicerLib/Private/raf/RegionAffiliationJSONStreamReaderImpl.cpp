// Copyright Epic Games, Inc. All Rights Reserved.

#include "raf/RegionAffiliationJSONStreamReaderImpl.h"

#include "raf/RegionAffiliation.h"
#include "raf/TypeDefs.h"

namespace raf {

// Note: RegionAffiliationJSONStreamReaderImpl::status hasn't been initialized deliberately, as it uses the same error codes that
// were already registered in RegionAffiliationBinaryStreamReaderImpl, and since they are all registered in a single, global error
// code registry, this would trigger an assert there.

RegionAffiliationJSONStreamReader::~RegionAffiliationJSONStreamReader() = default;

RegionAffiliationJSONStreamReader* RegionAffiliationJSONStreamReader::create(BoundedIOStream* stream, MemoryResource* memRes) {
    PolyAllocator<RegionAffiliationJSONStreamReaderImpl> alloc{memRes};
    return alloc.newObject(stream, memRes);
}

void RegionAffiliationJSONStreamReader::destroy(RegionAffiliationJSONStreamReader* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto reader = static_cast<RegionAffiliationJSONStreamReaderImpl*>(instance);
    PolyAllocator<RegionAffiliationJSONStreamReaderImpl> alloc{reader->getMemoryResource()};
    alloc.deleteObject(reader);
}

RegionAffiliationJSONStreamReaderImpl::RegionAffiliationJSONStreamReaderImpl(BoundedIOStream* stream_, MemoryResource* memRes_) :
    BaseImpl{memRes_},
    ReaderImpl{memRes_},
    stream{stream_} {
}

void RegionAffiliationJSONStreamReaderImpl::read() {
    status.reset();
    trio::StreamScope scope{stream};
    if (!Status::isOk()) {
        status.set(IOError, Status::get().message);
        return;
    }
    terse::JSONInputArchive<BoundedIOStream> archive{stream};
    archive >> regionAffiliation;

    if (!Status::isOk()) {
        return;
    }

    if (!regionAffiliation.signature.matches()) {
        status.set(SignatureMismatchError,
                   regionAffiliation.signature.value.expected.data(),
                   regionAffiliation.signature.value.got.data());
        return;
    }

    if (!regionAffiliation.version.supported()) {
        status.set(VersionMismatchError, regionAffiliation.version.generation, regionAffiliation.version.version);
        return;
    }
}

}  // namespace raf
