// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/ReaderImpl.h"
#include "raf/RegionAffiliationJSONStreamReader.h"
#include "raf/TypeDefs.h"

namespace raf {

class RegionAffiliationJSONStreamReaderImpl : public ReaderImpl<RegionAffiliationJSONStreamReader> {
public:
    RegionAffiliationJSONStreamReaderImpl(BoundedIOStream* stream_, MemoryResource* memRes_);

    void read() override;

private:
    static sc::StatusProvider status;
    BoundedIOStream* stream;
};

}  // namespace raf
