// Copyright Epic Games, Inc. All Rights Reserved.

#include "raf/RegionAffiliationJSONStreamWriterImpl.h"

#include "raf/RegionAffiliation.h"
#include "raf/TypeDefs.h"

namespace raf {

// Note: RegionAffiliationJSONStreamWriterImpl::status hasn't been initialized deliberately, as it uses the same error codes that
// were already registered in RegionAffiliationBinaryStreamWriterImpl, and since they are all registered in a single, global error
// code registry, this would trigger an assert there.

RegionAffiliationJSONStreamWriter::~RegionAffiliationJSONStreamWriter() = default;

RegionAffiliationJSONStreamWriter* RegionAffiliationJSONStreamWriter::create(BoundedIOStream* stream,
                                                                             std::uint32_t indentWidth,
                                                                             MemoryResource* memRes) {
    PolyAllocator<RegionAffiliationJSONStreamWriterImpl> alloc{memRes};
    return alloc.newObject(stream, indentWidth, memRes);
}

void RegionAffiliationJSONStreamWriter::destroy(RegionAffiliationJSONStreamWriter* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto writer = static_cast<RegionAffiliationJSONStreamWriterImpl*>(instance);
    PolyAllocator<RegionAffiliationJSONStreamWriterImpl> alloc{writer->getMemoryResource()};
    alloc.deleteObject(writer);
}

RegionAffiliationJSONStreamWriterImpl::RegionAffiliationJSONStreamWriterImpl(BoundedIOStream* stream_,
                                                                             std::uint32_t indentWidth_,
                                                                             MemoryResource* memRes_) :
    BaseImpl{memRes_},
    WriterImpl{memRes_},
    stream{stream_},
    indentWidth{indentWidth_} {
}

void RegionAffiliationJSONStreamWriterImpl::write() {
    status.reset();
    trio::StreamScope scope{stream};
    if (!Status::isOk()) {
        status.set(IOError, Status::get().message);
        return;
    }
    terse::JSONOutputArchive<BoundedIOStream> archive{stream, indentWidth};
    archive << regionAffiliation;
    archive.sync();
}

}  // namespace raf
