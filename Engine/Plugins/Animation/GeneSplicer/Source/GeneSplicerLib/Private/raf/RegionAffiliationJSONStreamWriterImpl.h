// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/RegionAffiliationJSONStreamWriter.h"
#include "raf/TypeDefs.h"
#include "raf/WriterImpl.h"

namespace raf {

class RegionAffiliationJSONStreamWriterImpl : public WriterImpl<RegionAffiliationJSONStreamWriter> {
public:
    RegionAffiliationJSONStreamWriterImpl(BoundedIOStream* stream_, std::uint32_t indentWidth_, MemoryResource* memRes_);

    void write() override;

private:
    static sc::StatusProvider status;

    BoundedIOStream* stream;
    std::uint32_t indentWidth;
};

}  // namespace raf
