// Copyright Epic Games, Inc. All Rights Reserved.

#include "raf/RegionAffiliationStreamReader.h"

namespace raf {

VertexRegionAffiliationReader::~VertexRegionAffiliationReader() = default;
JointRegionAffiliationReader::~JointRegionAffiliationReader() = default;
RegionAffiliationReader::~RegionAffiliationReader() = default;
RegionAffiliationStreamReader::~RegionAffiliationStreamReader() = default;

const sc::StatusCode RegionAffiliationStreamReader::IOError{1007, "%s"};
const sc::StatusCode RegionAffiliationStreamReader::SignatureMismatchError{
    1008,
    "RegionAffiliation signature mismatched, expected %.3s, got %.3s"};
const sc::StatusCode RegionAffiliationStreamReader::VersionMismatchError{1009,
                                                                         "RegionAffiliation version mismatched, got %hu.%hu"};

}  // namespace raf
