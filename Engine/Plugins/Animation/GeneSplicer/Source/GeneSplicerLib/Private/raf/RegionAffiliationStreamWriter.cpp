// Copyright Epic Games, Inc. All Rights Reserved.

#include <raf/RegionAffiliationStreamWriter.h>

namespace raf {

VertexRegionAffiliationWriter::~VertexRegionAffiliationWriter() = default;
JointRegionAffiliationWriter::~JointRegionAffiliationWriter() = default;
RegionAffiliationWriter::~RegionAffiliationWriter() = default;
RegionAffiliationStreamWriter::~RegionAffiliationStreamWriter() = default;

const sc::StatusCode RegionAffiliationStreamWriter::IOError{1017, "%s"};

}  // namespace raf
