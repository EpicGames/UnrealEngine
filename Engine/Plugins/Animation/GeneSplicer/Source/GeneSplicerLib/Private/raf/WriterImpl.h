// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/BaseImpl.h"
#include "raf/RegionAffiliationReader.h"
#include "raf/TypeDefs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cstddef>
#include <cstring>
#include <tuple>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace raf {

template<class TContainer>
typename std::enable_if<std::is_constructible<typename TContainer::value_type, MemoryResource*>::value>::type
ensureHasSize(TContainer& target, std::size_t size) {
    target.reserve(size);
    while (target.size() < size) {
        target.push_back(typename TContainer::value_type(target.get_allocator().getMemoryResource()));
    }
}

template<class TContainer>
typename std::enable_if<!std::is_constructible<typename TContainer::value_type, MemoryResource*>::value>::type
ensureHasSize(TContainer& target, std::size_t size) {
    if (target.size() < size) {
        target.resize(size);
    }
}

template<class TContainer, typename U>
typename std::enable_if<std::is_integral<U>::value, typename TContainer::value_type&>::type getAt(TContainer& target, U index) {
    ensureHasSize(target, index + 1ul);
    return target[index];
}

template<class TContainer, typename TSize, typename TValue>
typename std::enable_if<std::is_integral<TSize>::value>::type setAt(TContainer& target, TSize index, const TValue& value) {
    getAt(target, index) = value;
}

template<class TWriterBase>
class WriterImpl : public TWriterBase, public virtual BaseImpl {
public:
    using WriterInterface = TWriterBase;

public:
    explicit WriterImpl(MemoryResource* memRes_);

    // JointRegionAffiliationWriter methods
    void setJointRegionIndices(std::uint16_t jointIndex, const std::uint16_t* regionIndices, std::uint16_t count) override;
    void setJointRegionAffiliation(std::uint16_t jointIndex, const float* regionAffiliationValues, std::uint16_t count) override;
    void clearJointAffiliations() override;
    void deleteJointAffiliation(std::uint16_t jointIndex) override;

    // VertexRegionAffiliationWriter methods
    void setVertexRegionIndices(std::uint16_t meshIndex,
                                std::uint32_t vertexIndex,
                                const std::uint16_t* regionIndices,
                                std::uint16_t count) override;
    void setVertexRegionAffiliation(std::uint16_t meshIndex,
                                    std::uint32_t vertexIndex,
                                    const float* regionAffiliationValues,
                                    std::uint16_t count) override;
    void clearVertexAffiliations() override;
    void clearVertexAffiliations(std::uint16_t meshIndex) override;
    void deleteVertexAffiliation(std::uint16_t meshIndex, std::uint32_t vertexIndex) override;

    // RegionAffiliationWriter methods
    void clearRegionNames() override;
    void setRegionName(std::uint16_t regionIndex, const char* regionName) override;
    void setFrom(const RegionAffiliationReader* source) override;
};

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4589)
#endif
template<class TWriterBase>
WriterImpl<TWriterBase>::WriterImpl(MemoryResource* memRes_) :
    BaseImpl{memRes_} {
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4505)
#endif
template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointRegionIndices(std::uint16_t jointIndex,
                                                           const std::uint16_t* regionIndices,
                                                           std::uint16_t count) {
    auto& jointRegion = getAt(regionAffiliation.jointRegions, jointIndex);
    jointRegion.regionIndices.assign(regionIndices, regionIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setJointRegionAffiliation(std::uint16_t jointIndex,
                                                               const float* regionAffiliationValues,
                                                               std::uint16_t count) {
    auto& jointRegion = getAt(regionAffiliation.jointRegions, jointIndex);
    jointRegion.values.assign(regionAffiliationValues, regionAffiliationValues + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearJointAffiliations() {
    regionAffiliation.jointRegions.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::deleteJointAffiliation(std::uint16_t jointIndex) {
    if (jointIndex < regionAffiliation.jointRegions.size()) {
        auto& joints = regionAffiliation.jointRegions;
        joints.erase(std::next(joints.begin(), jointIndex));
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setVertexRegionIndices(std::uint16_t meshIndex,
                                                            std::uint32_t vertexIndex,
                                                            const std::uint16_t* regionIndices,
                                                            std::uint16_t count) {
    auto& mesh = getAt(regionAffiliation.vertexRegions, meshIndex);
    auto& vertex = getAt(mesh, vertexIndex);
    vertex.regionIndices.assign(regionIndices, regionIndices + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setVertexRegionAffiliation(std::uint16_t meshIndex,
                                                                std::uint32_t vertexIndex,
                                                                const float* regionAffiliationValues,
                                                                std::uint16_t count) {
    auto& mesh = getAt(regionAffiliation.vertexRegions, meshIndex);
    auto& vertex = getAt(mesh, vertexIndex);
    vertex.values.assign(regionAffiliationValues, regionAffiliationValues + count);
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearVertexAffiliations() {
    regionAffiliation.vertexRegions.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearVertexAffiliations(std::uint16_t meshIndex) {
    if (meshIndex < regionAffiliation.vertexRegions.size()) {
        regionAffiliation.vertexRegions[meshIndex].clear();
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::deleteVertexAffiliation(std::uint16_t meshIndex, std::uint32_t vertexIndex) {
    if (meshIndex < regionAffiliation.vertexRegions.size()) {
        auto& mesh = regionAffiliation.vertexRegions[meshIndex];
        if (vertexIndex < mesh.size()) {
            mesh.erase(std::next(mesh.begin(), meshIndex));
        }
    }
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::clearRegionNames() {
    regionAffiliation.regionNames.clear();
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setRegionName(std::uint16_t regionIndex, const char* regionName) {
    getAt(regionAffiliation.regionNames, regionIndex) = regionName;
}

template<class TWriterBase>
inline void WriterImpl<TWriterBase>::setFrom(const RegionAffiliationReader* source) {
    clearRegionNames();
    for (std::uint16_t riPlusOne = source->getRegionCount(); riPlusOne > 0u; riPlusOne--) {
        const auto ri = static_cast<std::uint16_t>(riPlusOne - 1u);
        setRegionName(ri, source->getRegionName(ri));
    }
    clearJointAffiliations();
    for (std::uint16_t jiPlusOne = source->getJointCount(); jiPlusOne > 0; jiPlusOne--) {
        const auto ji = static_cast<std::uint16_t>(jiPlusOne - 1u);
        ConstArrayView<float> values = source->getJointRegionAffiliation(ji);
        ConstArrayView<std::uint16_t> indices = source->getJointRegionIndices(ji);
        setJointRegionAffiliation(ji, values.data(), static_cast<std::uint16_t>(values.size()));
        setJointRegionIndices(ji, indices.data(), static_cast<std::uint16_t>(indices.size()));
    }
    clearVertexAffiliations();
    for (std::uint16_t miPlusOne = source->getMeshCount(); miPlusOne > 0; miPlusOne--) {
        const auto mi = static_cast<std::uint16_t>(miPlusOne - 1u);
        for (std::uint32_t viPlusOne = source->getVertexCount(mi); viPlusOne > 0; viPlusOne--) {
            const auto vi = viPlusOne - 1u;
            ConstArrayView<float> values = source->getVertexRegionAffiliation(mi, vi);
            ConstArrayView<std::uint16_t> indices = source->getVertexRegionIndices(mi, vi);
            setVertexRegionAffiliation(mi, vi, values.data(), static_cast<std::uint16_t>(values.size()));
            setVertexRegionIndices(mi, vi, indices.data(), static_cast<std::uint16_t>(indices.size()));
        }
    }
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif

}  // namespace raf
