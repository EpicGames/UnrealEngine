// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365)
#endif
#include <array>
#include <cstdint>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace raf {

template<typename T>
struct ExpectedValue {
    T expected;
    T got;

    explicit ExpectedValue(const T& value) :
        expected{value},
        got{} {
    }

    template<class Archive>
    void load(Archive& archive) {
        archive(got);
    }

    template<class Archive>
    void save(Archive& archive) {
        archive(expected);
    }

    bool matches() const {
        return (expected == got);
    }
};

template<std::size_t Size>
struct Signature {
    using SignatureValueType = std::array<char, Size>;

    ExpectedValue<SignatureValueType> value;

    explicit Signature(SignatureValueType bytes) :
        value{bytes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(value);
    }

    bool matches() const {
        return value.matches();
    }
};

struct Version {
    std::uint16_t generation;
    std::uint16_t version;

    Version(std::uint16_t generation_, std::uint16_t version_) :
        generation{generation_},
        version{version_} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("generation");
        archive(generation);
        archive.label("version");
        archive(version);
    }

    bool supported() const {
        return (generation == static_cast<std::uint16_t>(1)) &&
               ((version == static_cast<std::uint16_t>(0)) || (version == static_cast<std::uint16_t>(1)));
    }

    bool matches(std::uint16_t generation_, std::uint16_t version_) const {
        return (generation == generation_) && (version == version_);
    }
};

}  // namespace raf
