// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef GENESPLICER_MODULE_DISCARD

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGeneSplicerLib, Log, All);

class FGeneSplicerLib : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

#endif  // GENESPLICER_MODULE_DISCARD
