// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/CalculationType.h"
#include "genesplicer/Defs.h"
#include "genesplicer/GeneSplicerDNAReader.h"
#include "genesplicer/splicedata/SpliceData.h"
#include "genesplicer/types/Aliases.h"

namespace gs4 {

/**
    @brief A stateless facility that wraps the splicing algorithms.
*/
class GeneSplicer final {
public:
    /**
        @brief Constructor
        @param calculationType
            Determines which algorithm implementation is used for splicing
    */
    GSAPI explicit GeneSplicer(CalculationType calculationType = CalculationType::SSE, MemoryResource* memRes = nullptr);

    GSAPI ~GeneSplicer();

    GeneSplicer(const GeneSplicer&) = delete;
    GeneSplicer& operator=(const GeneSplicer&) = delete;

    GSAPI GeneSplicer(GeneSplicer&& rhs);
    GSAPI GeneSplicer& operator=(GeneSplicer&& rhs);
    /**
        @brief Run all the individual splicers.
        @param spliceData
            Contains all the necessary input data that is used during splicing.
        @param output
            Output parameter - the DNA Reader that will contain the spliced DNA data.
    */
    GSAPI void splice(const SpliceData* spliceData, GeneSplicerDNAReader* output);
    /**
        @brief Run only the neutral mesh splicer.
        @param spliceData
            Contains all the necessary input data that is used during splicing.
        @param output
            Output parameter - the DNA Reader that will contain the spliced DNA data.
    */
    GSAPI void spliceNeutralMeshes(const SpliceData* spliceData, GeneSplicerDNAReader* output);
    /**
        @brief Run only the blend shape splicer.
        @param spliceData
            Contains all the necessary input data that is used during splicing.
        @param output
            Output parameter - the DNA Reader that will contain the spliced DNA data.
    */
    GSAPI void spliceBlendShapes(const SpliceData* spliceData, GeneSplicerDNAReader* output);
    /**
        @brief Run only the neutral joint splicer.
        @param spliceData
            Contains all the necessary input data that is used during splicing.
        @param output
            Output parameter - the DNA Reader that will contain the spliced DNA data.
    */
    GSAPI void spliceNeutralJoints(const SpliceData* spliceData, GeneSplicerDNAReader* output);
    /**
        @brief Run only the joint behavior splicer.
        @param spliceData
            Contains all the necessary input data that is used during splicing.
        @param output
            Output parameter - the DNA Reader that will contain the spliced DNA data.
    */
    GSAPI void spliceJointBehavior(const SpliceData* spliceData, GeneSplicerDNAReader* output);
    /**
        @brief Run only the skin weight splicer.
        @param spliceData
            Contains all the necessary input data that is used during splicing.
        @param output
            Output parameter - the DNA Reader that will contain the spliced DNA data.
    */
    GSAPI void spliceSkinWeights(const SpliceData* spliceData, GeneSplicerDNAReader* output);

private:
    class Impl;
    ScopedPtr<Impl, FactoryDestroy<Impl>> pImpl;
};

}  // namespace gs4
