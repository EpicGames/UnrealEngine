// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/Defs.h"
#include "genesplicer/types/Aliases.h"

namespace gs4 {

/**
    @brief A special purpose DNA Reader type which serves as the output parameter of GeneSplicer.
*/
class GSAPI GeneSplicerDNAReader : public Reader, public virtual Writer {
public:
    /**
        @brief Factory method for the creation of GeneSplicerDNAReader
        @param reader
            The original DNA reader from which GeneSplicerDNAReader is initialized.
        @note
            During initialization, all the static data (data that is not generated during splicing)
            is copied from the given source reader.
        @param memRes
            A custom memory resource to be used for allocations.
        @note
            If a custom memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static GeneSplicerDNAReader* create(const dna::Reader* reader, MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing a GeneSplicerDNAReader instance.
        @param instance
            Instance of GeneSplicerDNAReader to be freed.
        @see create
    */
    static void destroy(GeneSplicerDNAReader* instance);

protected:
    virtual ~GeneSplicerDNAReader();
};

}  // namespace gs4

namespace pma {

template<>
struct DefaultInstanceCreator<gs4::GeneSplicerDNAReader> {
    using type = FactoryCreate<gs4::GeneSplicerDNAReader>;
};

template<>
struct DefaultInstanceDestroyer<gs4::GeneSplicerDNAReader> {
    using type = FactoryDestroy<gs4::GeneSplicerDNAReader>;
};

}  // namespace pma
