// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/CalculationType.h"
#include "genesplicer/Defs.h"
#include "genesplicer/splicedata/GenePoolMask.h"
#include "genesplicer/types/Aliases.h"

#include <cstdint>

namespace gs4 {

/**
    @brief Encapsulates the input data that GeneSplicer uses during splicing.
*/
class GenePool final {
public:
    GSAPI static const StatusCode DNAMismatch;
    GSAPI static const StatusCode DNAsEmpty;

public:
    /**
        @brief Creates optimized structure for splicing that contains copy of all required data from DNAs
            that will be used in splicing.
        @param dnas
            The DNA Readers that will participate in splicing.
        @param deltaArchetype
            The deltaArchetype DNA reader.
        @note
            All required data from pointers is copied to internal data structures, does not take over
            ownership.
        @param dnaCount
            The number of DNA Readers, i.e. the length of the DNAs array.
        @param mask
            Used for loading only certain sections of GenePool.
        @note
            The number of DNAs set through this API, excluding deltaArchetype, directly impacts the number of splice weights
            that must be set through PoolSpliceParams::setSpliceWeights
        @see PoolSpliceParams::setSpliceWeights
    */
    GSAPI GenePool(const dna::Reader* deltaArchetype,
                   const dna::Reader** dnas,
                   std::uint16_t dnaCount,
                   GenePoolMask mask = GenePoolMask::All,
                   MemoryResource* memRes = nullptr);
    /**
        @brief Creates optimized structure for splicing that contains copy of all required data from DNAs
            that will be used in splicing.
        @note
            All required data from stream is copied to internal data structures, does not take over
            ownership.
        @param stream
            Source stream from which data is going to be read.
        @param mask
            Used for loading only certain sections of GenePool.
        @note
            The number of DNAs in stream, excluding deltaArchetype, directly impacts the number of splice weights
            that must be set through PoolSpliceParams::setSpliceWeights
        @see PoolSpliceParams::setSpliceWeights
    */
    GSAPI GenePool(BoundedIOStream* stream, GenePoolMask mask = GenePoolMask::All, MemoryResource* memRes = nullptr);

    /**
        @brief Writes out GenePool to a stream.
        @param stream
            Destination stream to which data is going to be written.
        @param mask
            Used for dumping only certain sections of GenePool.
   */
    GSAPI void dump(BoundedIOStream* stream, GenePoolMask mask = GenePoolMask::All);

    GSAPI ~GenePool();

    GenePool(const GenePool&) = delete;
    GenePool& operator=(const GenePool&) = delete;

    GSAPI GenePool(GenePool&& rhs);
    GSAPI GenePool& operator=(GenePool&& rhs);

    GSAPI std::uint16_t getDNACount() const;
    /**
        @param dnaIndex
            A position in the zero-indexed array of DNA names.
        @warning
            The index must be less than the value returned by getDNACount.
        @return View over the  name string.
        @see getDNACount
   */
    GSAPI StringView getDNAName(std::uint16_t dnaIndex) const;
    /**
        @param dnaIndex
            A position in the zero-indexed array of DNA genders.
        @warning
            The index must be less than the value returned by getDNACount.
        @see getDNACount
   */
    GSAPI dna::Gender getDNAGender(std::uint16_t dnaIndex) const;
    /**
        @param dnaIndex
            A position in the zero-indexed array of DNA ages.
        @warning
            The index must be less than the value returned by getDNACount.
        @see getDNACount
   */
    GSAPI std::uint16_t getDNAAge(std::uint16_t dnaIndex) const;

    GSAPI std::uint16_t getMeshCount() const;
    /**
        @brief Number of vertex positions in the entire mesh.
        @param meshIndex
            A position in the zero-indexed array of meshes.
        @warning
            The index must be less than the value returned by getMeshCount.
        @see getMeshCount
   */
    GSAPI std::uint32_t getVertexPositionCount(std::uint16_t meshIndex) const;
    /**
        @brief The vertex position of DNA.
        @param dnaIndex
           A position in the zero-indexed array of DNAs.
        @warning
           The index must be less than the value returned by getDNACount.
        @see getDNACount
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @see getMeshCount
        @param vertexIndex
            The index of the vertex position in the zero-indexed array of vertex positions.
        @warning
            vertexIndex must be less than the value returned by getVertexPositionCount.
        @see getVertexPositionCount
   */
    GSAPI Vector3 getDNAVertexPosition(std::uint16_t dnaIndex, std::uint16_t meshIndex, std::uint32_t vertexIndex) const;
    /**
        @brief The vertex position of archetype.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @see getMeshCount
        @param vertexIndex
            The index of the vertex position in the zero-indexed array of vertex positions.
        @warning
            vertexIndex must be less than the value returned by getVertexPositionCount.
        @see getVertexPositionCount
   */
    GSAPI Vector3 getArchetypeVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const;

    GSAPI std::uint16_t getJointCount() const;
    /**
        @brief Name of the requested joint.
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
        @warning
            The index must be less than the value returned by getJointCount.
        @see getJointCount
    */
    GSAPI StringView getJointName(std::uint16_t jointIndex) const;
    /**
        @brief Translation of DNA's joint in world space.
        @param dnaIndex
           A position in the zero-indexed array of DNAs.
        @warning
           The index must be less than the value returned by getDNACount.
        @see getDNACount
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
        @warning
            The index must be less than the value returned by getJointCount.
        @see getJointCount
   */
    GSAPI Vector3 getDNANeutralJointWorldTranslation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const;
    /**
        @brief Translation of archetype's joint in world space.
        @param jointIndex
            A name's position in the zero-indexed array of joint names.
        @warning
            The index must be less than the value returned by getJointCount.
     */
    GSAPI Vector3 getArchetypeNeutralJointWorldTranslation(std::uint16_t jointIndex) const;
    /**
        @brief Rotation of DNA's joint in world space.
        @param dnaIndex
           A position in the zero-indexed array of DNAs.
        @warning
           The index must be less than the value returned by getDNACount.
        @see getDNACount
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
        @warning
            The index must be less than the value returned by getJointCount.
        @see getJointCount
   */
    GSAPI Vector3 getDNANeutralJointWorldRotation(std::uint16_t dnaIndex, std::uint16_t jointIndex) const;
    /**
        @brief Rotation of archetype's joint in world space.
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
        @warning
            The index must be less than the value returned by getJointCount.
        @see getJointCount
    */
    GSAPI Vector3 getArchetypeNeutralJointWorldRotation(std::uint16_t jointIndex) const;

private:
    template<class T>
    friend struct PImplExtractor;

    class Impl;
    ScopedPtr<Impl, FactoryDestroy<Impl>> pImpl;
};

}  // namespace gs4
