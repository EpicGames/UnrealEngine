// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/Defs.h"

#include <cstdint>

namespace gs4 {

enum class GenePoolMask : std::uint32_t {
    NeutralMeshes = 1,
    BlendShapes = 2,
    SkinWeights = 4,
    NeutralJoints = 8,
    JointBehavior = 16,
    All = 31
};

GSAPI GenePoolMask operator~(GenePoolMask a);
GSAPI GenePoolMask operator|(GenePoolMask a, GenePoolMask b);
GSAPI GenePoolMask operator&(GenePoolMask a, GenePoolMask b);

}  // namespace gs4
