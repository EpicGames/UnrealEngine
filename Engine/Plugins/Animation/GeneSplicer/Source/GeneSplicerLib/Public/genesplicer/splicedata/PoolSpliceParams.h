// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/Defs.h"
#include "genesplicer/splicedata/GenePool.h"
#include "genesplicer/types/Aliases.h"

#include <cstdint>

namespace gs4 {
/**
    @brief Encapsulates the input data that GeneSplicer uses during splicing.
*/
class GSAPI PoolSpliceParams {
public:
    static const StatusCode GenePoolIncompatible;
    static const StatusCode WeightsInvalid;

public:
    virtual ~PoolSpliceParams();
    /**
        @brief Specify which subset of DNAs will participate in splicing.
        @note
            This may significantly reduce the amount of computation that needs to be performed.
        @param dnaIndices
            The indices of DNAs that will participate in splicing, while DNAs not present
            in this list will be skipped.
        @param count
            The number of DNA indices, i.e. the length of the dnaIndices array.
        @note
            The dna indices will be copied into PoolSpliceParams.
    */
    virtual void setDNAFilter(const std::uint16_t* dnaIndices, std::uint16_t count) = 0;
    /**
        @brief Specify which subset of meshes will participate in splicing.
        @note
            This may significantly reduce the amount of computation that needs to be performed.
        @param meshIndices
            The indices of meshes that will participate in splicing, while Meshes not present
            in this list will be skipped.
        @param count
            The number of Mesh indices, i.e. the length of the meshIndices array.
        @note
            The mesh indices will be copied into PoolSpliceParams.
    */
    virtual void setMeshFilter(const std::uint16_t* meshIndices, std::uint16_t count) = 0;
    /**
        @brief Clears dna and mesh filters previously set by setDNAFilter and SetMeshFilter
    */
    virtual void clearFilters() = 0;
    /**
        @brief Set weights for each region of DNAs starting from DNA at dnaStartIndex index spanning across
        successive DNAs until reaching the end of specified weights array, i.e. weights + count.
        @note
            All values will be copied.
        @param dnaStartIndex
            Index of the first dna in succession for which the weights apply.
        @param weights
            C style array of floats, containing weights of DNAs per all regions.
        @param count
            The number of weights, i.e. the length of the weights array.
        @warning
            The weight count must be equal to the region count multiplied by the number of DNAs whose
            weights are being set by the call.
        @details
            Initially when PoolSpliceParams is created weights are filled with zeros. For the sake of example
            lets say we have 2 Regions and 4 DNAs.
            Which translates to:
                        Region-0  Region-1
                DNA-0    0.0       0.0
                DNA-1    0.0       0.0
                DNA-2    0.0       0.0
                DNA-3    0.0       0.0
            after calling setSpiceWeights with dnaStartIndex = 1, weights=[0.1f, 0.9f, 0.4f, 05f], count = 4
            Weights will chagne for DNAs with indices 1 and 2:
                        Region-0  Region-1
                DNA-0    0.0       0.0
                DNA-1    0.1       0.9
                DNA-2    0.4       0.5
                DNA-3    0.0       0.0
    */
    virtual void setSpliceWeights(std::uint16_t dnaStartIndex, const float* weights, std::uint32_t count) = 0;
    /**
        @brief Sets scaling factor for splicing.
        @param scale
            Scale to be applied while splicing.
    */
    virtual void setScale(float scale) = 0;

    virtual std::uint16_t getDNACount() const = 0;
    virtual std::uint16_t getRegionCount() const = 0;
};

}  // namespace gs4
