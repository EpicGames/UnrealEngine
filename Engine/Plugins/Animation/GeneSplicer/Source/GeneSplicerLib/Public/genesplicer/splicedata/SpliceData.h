// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/Defs.h"
#include "genesplicer/splicedata/GenePool.h"
#include "genesplicer/splicedata/PoolSpliceParams.h"
#include "genesplicer/types/Aliases.h"

#include <cstdint>

namespace gs4 {

/**
    @brief Encapsulates the input data that GeneSplicer uses during splicing.
*/
class SpliceData final {
public:
    GSAPI explicit SpliceData(MemoryResource* memRes = nullptr);
    GSAPI ~SpliceData();

    SpliceData(const SpliceData&) = delete;
    SpliceData& operator=(const SpliceData&) = delete;

    GSAPI SpliceData(SpliceData&& rhs);
    GSAPI SpliceData& operator=(SpliceData&& rhs);
    /**
        @brief Registers gene pool for splicing.
        @param name
            Null terminated string used as key to later access gene pool parameters
        @param raf
            Region affiliations associated with given pool.
        @note
            Region affiliations will be copied into SpliceData and will be held as long as gene pool is not destroyed.
        @param genePool
            Gene pool that holds dnas to be spliced.
        @note
            Gene pool will not be copied, and the user is responsible to maintain it's lifecycle.
    */
    GSAPI void registerGenePool(const char* name, const RegionAffiliationReader* raf, const GenePool* genePool);
    /**
        @brief Unregisters gene pool previously registered
        @param name
            Null terminated string, desired name of gene pool to unregister.
    */
    GSAPI void unregisterGenePool(const char* name);
    /**
        @brief Access previously registered gene pools to set splicing parameters.
        @param name
            Null terminated string, desired name of gene pool to access.
        @note
            If name is not registered return value will be nullptr
    */
    GSAPI PoolSpliceParams* getPoolParams(const char* name);
    /**
        @brief Set the base Archetype DNA Reader.
        @note
            The baseArchetype DNA reader provides the neutral values that are used as base
            onto which deltas will be added.
        @param baseArchetype
            The baseArchetype DNA reader.
        @note
            All data reqired from pointer is copied to internal data structures, does not take over
            ownership.
    */
    GSAPI void setBaseArchetype(const dna::Reader* baseArchetype);

private:
    template<class T>
    friend struct PImplExtractor;

    class Impl;
    ScopedPtr<Impl, FactoryDestroy<Impl>> pImpl;
};

}  // namespace gs4
