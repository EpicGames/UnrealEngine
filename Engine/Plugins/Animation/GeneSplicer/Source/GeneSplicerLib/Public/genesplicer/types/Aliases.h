// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <dna/BinaryStreamReader.h>
#include <dna/BinaryStreamWriter.h>
#include <dna/Configuration.h>
#include <dna/types/Aliases.h>
#include <dna/types/Vector3.h>
#include <pma/MemoryResource.h>
#include <pma/ScopedPtr.h>
#include <pma/resources/AlignedMemoryResource.h>
#include <pma/resources/DefaultMemoryResource.h>
#include <raf/RegionAffiliationBinaryStreamReader.h>
#include <raf/RegionAffiliationBinaryStreamWriter.h>
#include <raf/RegionAffiliationJSONStreamReader.h>
#include <raf/RegionAffiliationJSONStreamWriter.h>
#include <status/Status.h>
#include <status/StatusCode.h>
#include <trio/Stream.h>
#include <trio/streams/FileStream.h>

namespace gs4 {

using dna::BinaryStreamReader;
using dna::BinaryStreamWriter;
using dna::DataLayer;
using dna::Reader;
using dna::StringView;
using dna::UnknownLayerPolicy;
using dna::Vector3;
using dna::Writer;
using raf::JointRegionAffiliationReader;
using raf::JointRegionAffiliationWriter;
using raf::RegionAffiliationBinaryStreamReader;
using raf::RegionAffiliationBinaryStreamWriter;
using raf::RegionAffiliationJSONStreamReader;
using raf::RegionAffiliationJSONStreamWriter;
using raf::RegionAffiliationReader;
using raf::RegionAffiliationStreamReader;
using raf::RegionAffiliationStreamWriter;
using raf::RegionAffiliationWriter;
using raf::VertexRegionAffiliationReader;
using raf::VertexRegionAffiliationWriter;
using sc::Status;
using sc::StatusCode;
using trio::BoundedIOStream;
using trio::FileStream;
using trio::MemoryMappedFileStream;
using trio::MemoryStream;

template<typename T>
using ArrayView = dna::ArrayView<T>;

template<typename T>
using ConstArrayView = dna::ConstArrayView<T>;

using namespace pma;

}  // namespace gs4
