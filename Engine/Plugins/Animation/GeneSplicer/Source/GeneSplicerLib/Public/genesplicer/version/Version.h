// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define GS_MAJOR_VERSION 10
#define GS_MINOR_VERSION 1
#define GS_PATCH_VERSION 3
#define GS_VERSION_STRING "10.1.3"
