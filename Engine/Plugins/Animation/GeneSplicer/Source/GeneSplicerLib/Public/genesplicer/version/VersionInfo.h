// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/Defs.h"
#include "genesplicer/types/Aliases.h"

namespace gs4 {

struct GSAPI VersionInfo {
    static int getMajorVersion();
    static int getMinorVersion();
    static int getPatchVersion();
    static StringView getVersionString();
};

}  // namespace gs4
