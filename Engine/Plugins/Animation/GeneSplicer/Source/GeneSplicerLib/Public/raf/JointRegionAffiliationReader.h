// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/Defs.h"
#include "raf/types/Aliases.h"

#include <cstdint>

namespace raf {

/**
    @brief Read-only accessors for joint region affiliation.
*/
class RAFAPI JointRegionAffiliationReader {

protected:
    virtual ~JointRegionAffiliationReader();

public:
    virtual std::uint16_t getJointCount() const = 0;
    /**
        @brief List of region indices for specified joint.
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
        @warning
            jointIndex must be less than the value returned by getJointCount.
    */
    virtual ConstArrayView<std::uint16_t> getJointRegionIndices(std::uint16_t jointIndex) const = 0;
    /**
        @brief List of joint-region affiliations (0.0f-1.0f) for specified joint.
        @param vertexIndex
            A joint's position in the zero-indexed array of joints.
        @warning
            jointIndex must be less than the value returned by getJointCount.
    */
    virtual ConstArrayView<float> getJointRegionAffiliation(std::uint16_t jointIndex) const = 0;
};

}  // namespace raf
