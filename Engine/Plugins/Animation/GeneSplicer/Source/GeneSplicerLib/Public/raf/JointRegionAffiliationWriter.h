// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/Defs.h"

#include <cstdint>

namespace raf {

/**
    @brief Write-only accessors for joint region affiliation.
*/
class RAFAPI JointRegionAffiliationWriter {
protected:
    virtual ~JointRegionAffiliationWriter();

public:
    /**
        @brief List of region indices.
        @param jointIndex
            A joint's position in the zero-indexed array of meshes.
        @param regionIndices
            The source address from which the region indices are to be copied.
        @param count
            The number of regions that affiliate with joint specified joint.
    */
    virtual void setJointRegionIndices(std::uint16_t jointIndex, const std::uint16_t* regionIndices, std::uint16_t count) = 0;
    /**
        @brief List of joint-region affiliations (0.0f-1.0f) for specified joint.
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
        @param regionAffiliationValues
            The source address from which the region affiliations are to be copied.
        @param count
            The number of regions that affiliate with joint specified joint.
    */
    virtual void setJointRegionAffiliation(std::uint16_t jointIndex,
                                           const float* regionAffiliationValues,
                                           std::uint16_t count) = 0;
    /**
        @brief Clears all joint region affiliations and indices .
     */
    virtual void clearJointAffiliations() = 0;
    /**
        @brief Deletes joint region affiliations and indices.
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
     */
    virtual void deleteJointAffiliation(std::uint16_t jointIndex) = 0;
};

}  // namespace raf
