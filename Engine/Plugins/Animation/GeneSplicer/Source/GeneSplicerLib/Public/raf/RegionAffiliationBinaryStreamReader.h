// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/Defs.h"
#include "raf/RegionAffiliationStreamReader.h"
#include "raf/types/Aliases.h"

namespace raf {

class RAFAPI RegionAffiliationBinaryStreamReader : public RegionAffiliationStreamReader {
protected:
    virtual ~RegionAffiliationBinaryStreamReader();

public:
    /**
        @brief Factory method for creation of RegionAffiliationBinaryStreamReader
        @param stream
            Stream from which data is going to be read.
        @param memRes
            A custom memory resource to be used for allocations.
        @note
            If a custom memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static RegionAffiliationBinaryStreamReader* create(BoundedIOStream* stream, MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing a RegionAffiliationBinaryStreamReader instance.
        @param instance
            Instance of RegionAffiliationBinaryStreamReader to be freed.
        @see create
    */
    static void destroy(RegionAffiliationBinaryStreamReader* instance);
};

}  // namespace raf

namespace pma {

template<>
struct DefaultInstanceCreator<raf::RegionAffiliationBinaryStreamReader> {
    using type = FactoryCreate<raf::RegionAffiliationBinaryStreamReader>;
};

template<>
struct DefaultInstanceDestroyer<raf::RegionAffiliationBinaryStreamReader> {
    using type = FactoryDestroy<raf::RegionAffiliationBinaryStreamReader>;
};

}  // namespace pma
