// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/Defs.h"
#include "raf/RegionAffiliationStreamWriter.h"
#include "raf/types/Aliases.h"

namespace raf {

class RAFAPI RegionAffiliationBinaryStreamWriter : public RegionAffiliationStreamWriter {
protected:
    virtual ~RegionAffiliationBinaryStreamWriter();

public:
    /**
        @brief Factory method for creation of RegionAffiliationBinaryStreamWriter
        @param stream
            Stream to which data is going to be written.
        @param memRes
            A custom memory resource to be used for allocations.
        @note
            If a custom memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static RegionAffiliationBinaryStreamWriter* create(BoundedIOStream* stream, MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing a RigDefinitionBinaryStreamWriter instance.
        @param instance
            Instance of RigDefinitionBinaryStreamWriter to be freed.
        @see create
    */
    static void destroy(RegionAffiliationBinaryStreamWriter* instance);
};

}  // namespace raf

namespace pma {

template<>
struct DefaultInstanceCreator<raf::RegionAffiliationBinaryStreamWriter> {
    using type = FactoryCreate<raf::RegionAffiliationBinaryStreamWriter>;
};

template<>
struct DefaultInstanceDestroyer<raf::RegionAffiliationBinaryStreamWriter> {
    using type = FactoryDestroy<raf::RegionAffiliationBinaryStreamWriter>;
};

}  // namespace pma
