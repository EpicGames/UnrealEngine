// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/Defs.h"
#include "raf/RegionAffiliationStreamReader.h"
#include "raf/types/Aliases.h"

namespace raf {

class RAFAPI RegionAffiliationJSONStreamReader : public RegionAffiliationStreamReader {
protected:
    virtual ~RegionAffiliationJSONStreamReader();

public:
    /**
        @brief Factory method for creation of RegionAffiliationJSONStreamReader
        @param stream
            Stream from which data is going to be read.
        @param memRes
            A custom memory resource to be used for allocations.
        @note
            If a custom memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static RegionAffiliationJSONStreamReader* create(BoundedIOStream* stream, MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing a RegionAffiliationJSONStreamReader instance.
        @param instance
            Instance of RegionAffiliationJSONStreamReader to be freed.
        @see create
    */
    static void destroy(RegionAffiliationJSONStreamReader* instance);
};

}  // namespace raf

namespace pma {

template<>
struct DefaultInstanceCreator<raf::RegionAffiliationJSONStreamReader> {
    using type = FactoryCreate<raf::RegionAffiliationJSONStreamReader>;
};

template<>
struct DefaultInstanceDestroyer<raf::RegionAffiliationJSONStreamReader> {
    using type = FactoryDestroy<raf::RegionAffiliationJSONStreamReader>;
};

}  // namespace pma
