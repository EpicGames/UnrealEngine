// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/Defs.h"
#include "raf/RegionAffiliationStreamWriter.h"
#include "raf/types/Aliases.h"

namespace raf {

class RAFAPI RegionAffiliationJSONStreamWriter : public RegionAffiliationStreamWriter {
protected:
    virtual ~RegionAffiliationJSONStreamWriter();

public:
    /**
        @brief Factory method for creation of RegionAffiliationJSONStreamWriter
        @param stream
            Stream to which data is going to be written.
        @param indentWidth
            Number of spaces to use for indentation.
        @param memRes
            A custom memory resource to be used for allocations.
        @note
            If a custom memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static RegionAffiliationJSONStreamWriter* create(BoundedIOStream* stream,
                                                     std::uint32_t indentWidth = 4u,
                                                     MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing a RegionAffiliationJSONStreamWriter instance.
        @param instance
            Instance of RegionAffiliationJSONStreamWriter to be freed.
        @see create
    */
    static void destroy(RegionAffiliationJSONStreamWriter* instance);
};

}  // namespace raf

namespace pma {

template<>
struct DefaultInstanceCreator<raf::RegionAffiliationJSONStreamWriter> {
    using type = FactoryCreate<raf::RegionAffiliationJSONStreamWriter>;
};

template<>
struct DefaultInstanceDestroyer<raf::RegionAffiliationJSONStreamWriter> {
    using type = FactoryDestroy<raf::RegionAffiliationJSONStreamWriter>;
};

}  // namespace pma
