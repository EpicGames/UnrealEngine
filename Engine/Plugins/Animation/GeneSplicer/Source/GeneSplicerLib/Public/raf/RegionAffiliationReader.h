// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/Defs.h"
#include "raf/JointRegionAffiliationReader.h"
#include "raf/VertexRegionAffiliationReader.h"

#include <cstdint>

namespace raf {

/**
    @brief Read-only accessors for vertex and joint region affiliation.
*/
class RAFAPI RegionAffiliationReader : public JointRegionAffiliationReader, public VertexRegionAffiliationReader {
protected:
    virtual ~RegionAffiliationReader();

public:
    virtual std::uint16_t getRegionCount() const = 0;
    virtual StringView getRegionName(std::uint16_t regionIndex) const = 0;
};

}  // namespace raf
