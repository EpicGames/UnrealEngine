// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/Defs.h"
#include "raf/RegionAffiliationReader.h"
#include "raf/types/Aliases.h"

namespace raf {

class RAFAPI RegionAffiliationStreamReader : public RegionAffiliationReader {
public:
    static const StatusCode IOError;
    static const StatusCode SignatureMismatchError;
    static const StatusCode VersionMismatchError;

protected:
    virtual ~RegionAffiliationStreamReader();

public:
    virtual void read() = 0;
};

}  // namespace raf
