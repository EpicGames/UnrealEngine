// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/Defs.h"
#include "raf/RegionAffiliationWriter.h"
#include "raf/types/Aliases.h"

namespace raf {

class RAFAPI RegionAffiliationStreamWriter : public RegionAffiliationWriter {
public:
    static const StatusCode IOError;

protected:
    virtual ~RegionAffiliationStreamWriter();

public:
    virtual void write() = 0;
};

}  // namespace raf
