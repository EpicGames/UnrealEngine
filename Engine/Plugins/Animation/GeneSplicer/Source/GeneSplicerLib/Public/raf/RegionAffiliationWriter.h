// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/Defs.h"
#include "raf/JointRegionAffiliationWriter.h"
#include "raf/VertexRegionAffiliationWriter.h"

#include <cstdint>

namespace raf {

class RegionAffiliationReader;

/**
    @brief Write-only accessors for vertex and joint region affiliation.
*/
class RAFAPI RegionAffiliationWriter : public JointRegionAffiliationWriter, public VertexRegionAffiliationWriter {
protected:
    virtual ~RegionAffiliationWriter();

public:
    virtual void clearRegionNames() = 0;
    virtual void setRegionName(std::uint16_t regionIndex, const char* regionName) = 0;
    virtual void setFrom(const RegionAffiliationReader* source) = 0;
};

}  // namespace raf
