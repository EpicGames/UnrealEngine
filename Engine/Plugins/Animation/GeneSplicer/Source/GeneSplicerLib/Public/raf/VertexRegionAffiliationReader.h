// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/Defs.h"
#include "raf/types/Aliases.h"

#include <cstdint>

namespace raf {

/**
    @brief Read-only accessors for vertex and joint region affiliation.
*/
class RAFAPI VertexRegionAffiliationReader {
protected:
    virtual ~VertexRegionAffiliationReader();

public:
    virtual std::uint16_t getMeshCount() const = 0;
    /**
        @brief Number of vertices in the entire mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual std::uint32_t getVertexCount(std::uint16_t meshIndex) const = 0;
    /**
        @brief List of region indices for specified vertex.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param vertexIndex
            A vertex's position in the zero-indexed array of vertices.
        @warning
            vertexIndex must be less than the value returned by getVertexCount.
    */
    virtual raf::ConstArrayView<std::uint16_t> getVertexRegionIndices(std::uint16_t meshIndex,
                                                                      std::uint32_t vertexIndex) const = 0;
    /**
        @brief List of vertex-region affiliations (0.0f-1.0f) for specified vertex.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param vertexIndex
            A vertex's position in the zero-indexed array of vertices.
        @warning
            vertexIndex must be less than the value returned by getVertexCount.
    */
    virtual raf::ConstArrayView<float> getVertexRegionAffiliation(std::uint16_t meshIndex, std::uint32_t vertexIndex) const = 0;
};

}  // namespace raf
