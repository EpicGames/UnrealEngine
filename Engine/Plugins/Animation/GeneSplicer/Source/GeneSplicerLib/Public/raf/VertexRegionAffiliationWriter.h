// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raf/Defs.h"

#include <cstdint>

namespace raf {

/**
    @brief Write-only accessors for vertex and joint region affiliation.
*/
class RAFAPI VertexRegionAffiliationWriter {
protected:
    virtual ~VertexRegionAffiliationWriter();

public:
    /**
        @brief List of region indices.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param vertexIndex
            A vertex's position in the zero-indexed array of meshes.
        @param regionIndices
            The source address from which the region indices are to be copied.
        @param count
            The number of regions that affiliate with vertexIndex.
        @note
            The mesh storage will be implicitly resized (if needed) to provide
            storage for the number of meshes that is inferred from the specified index.
    */
    virtual void setVertexRegionIndices(std::uint16_t meshIndex,
                                        std::uint32_t vertexIndex,
                                        const std::uint16_t* regionIndices,
                                        std::uint16_t count) = 0;
    /**
        @brief List of vertex region affiliations.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param vertexIndex
            A vertex's position in the zero-indexed array of meshes.
        @param regionAffiliationValues
            The source address from which the region affiliations are to be copied.
        @param count
            The number of regions that affiliate with vertexIndex.
        @note
            The mesh storage will be implicitly resized (if needed) to provide
            storage for the number of meshes that is inferred from the specified index.
    */
    virtual void setVertexRegionAffiliation(std::uint16_t meshIndex,
                                            std::uint32_t vertexIndex,
                                            const float* regionAffiliationValues,
                                            std::uint16_t count) = 0;
    /**
       @brief Clears all vertex region affiliations and indices.
   */
    virtual void clearVertexAffiliations() = 0;
    /**
        @brief Clears all region affiliations and indices of vertices in specified mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
    */
    virtual void clearVertexAffiliations(std::uint16_t meshIndex) = 0;
    /**
        @brief Deletes vertex region affiliations and indices.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param vertexIndex
            A vertex's position in the zero-indexed array of meshes.
     */
    virtual void deleteVertexAffiliation(std::uint16_t meshIndex, std::uint32_t vertexIndex) = 0;
};

}  // namespace raf
