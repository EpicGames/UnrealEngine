// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <arrayview/ArrayView.h>
#include <arrayview/StringView.h>
#include <pma/MemoryResource.h>
#include <pma/ScopedPtr.h>
#include <status/Status.h>
#include <status/StatusCode.h>
#include <trio/Stream.h>
#include <trio/streams/FileStream.h>

namespace raf {

using sc::Status;
using sc::StatusCode;
using trio::BoundedIOStream;
using trio::FileStream;

using namespace av;
using namespace pma;

}  // namespace raf
