// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define RAF_MAJOR_VERSION 5
#define RAF_MINOR_VERSION 2
#define RAF_PATCH_VERSION 5
#define RAF_VERSION_STRING "5.2.5"
