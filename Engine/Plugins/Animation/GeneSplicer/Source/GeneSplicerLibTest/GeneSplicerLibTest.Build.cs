// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class GeneSplicerLibTest : ModuleRules
	{
		public GeneSplicerLibTest(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseUnity = false; // A windows include is preprocessing some method names causing compile failures.

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				if (Target.Architecture != UnrealArch.Arm64)
				{
					PrivateDefinitions.Add("GS_BUILD_WITH_SSE=1");
				}
				PublicDefinitions.Add("GTEST_OS_WINDOWS=1");
			}

			string GeneSplicerLibPath = Path.GetFullPath(Path.Combine(ModuleDirectory, "../GeneSplicerLib"));

			if (Target.LinkType == TargetLinkType.Monolithic)
			{
				PublicDependencyModuleNames.Add("GeneSplicerLib");
				// Only tests (this module) should have access to Lib module's Private directory, so that's why Private was not made Internal.
				CppCompileWarningSettings.ModuleIncludePrivateWarningLevel = WarningLevel.Off;
				PrivateIncludePaths.Add(Path.Combine(GeneSplicerLibPath, "Private"));
			}
			else
			{
				PrivateDefinitions.Add("GENESPLICER_MODULE_DISCARD");
				ConditionalAddModuleDirectory(new DirectoryReference(GeneSplicerLibPath));
			}

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"GoogleTest",
					"RigLogicLib"
				}
			);
		}
	}
}