// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeneSplicerLibTest.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FGeneSplicerLibTest"

DEFINE_LOG_CATEGORY(LogGeneSplicerLibTest);

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FGeneSplicerLibTest, GeneSplicerLibTest)
