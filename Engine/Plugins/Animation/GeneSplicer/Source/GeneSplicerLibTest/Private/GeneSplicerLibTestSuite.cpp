// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "GeneSplicerLibTest.h"
#include "gtest/gtest.h"
#include "Misc/AutomationTest.h"

class FGeneSplicerLibTestPrinter
	: public ::testing::EmptyTestEventListener
{
	virtual void OnTestStart(const ::testing::TestInfo& InTestInfo) override
	{
		UE_LOGF(LogGeneSplicerLibTest, Verbose, "Test %ls.%ls Starting", *FString(InTestInfo.test_suite_name()), *FString(InTestInfo.name()));
	}

	virtual void OnTestPartResult(const ::testing::TestPartResult& InTestPartResult) override
	{
		if (InTestPartResult.failed())
		{
			UE_LOGF(LogGeneSplicerLibTest, Error, "FAILED in %ls:%d\n%ls", *FString(InTestPartResult.file_name()), InTestPartResult.line_number(), *FString(InTestPartResult.summary()));
		}
		else
		{
			UE_LOGF(LogGeneSplicerLibTest, Verbose, "Succeeded in %ls:%d\n%ls", *FString(InTestPartResult.file_name()), InTestPartResult.line_number(), *FString(InTestPartResult.summary()));
		}
	}

	virtual void OnTestEnd(const ::testing::TestInfo& InTestInfo) override
	{
		UE_LOGF(LogGeneSplicerLibTest, Verbose, "Test %ls.%ls Ending", *FString(InTestInfo.test_suite_name()), *FString(InTestInfo.name()));
	}
};

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FGeneSplicerLibTestSuite, "GeneSplicerLib", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

void FGeneSplicerLibTestSuite::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	::testing::InitGoogleTest();

	const ::testing::UnitTest* Instance = ::testing::UnitTest::GetInstance();
	for (int32 TestSuiteIndex = 0; TestSuiteIndex < Instance->total_test_suite_count(); ++TestSuiteIndex)
	{
		const ::testing::TestSuite* TestSuite = Instance->GetTestSuite(TestSuiteIndex);
		const FString TestCaseName = FString::Format(TEXT("{0}"), { TestSuite->name() });
		OutBeautifiedNames.Add(TestCaseName);
	}

	OutTestCommands = OutBeautifiedNames;	
}

bool FGeneSplicerLibTestSuite::RunTest(const FString& Parameters)
{
	::testing::TestEventListeners& Listeners = ::testing::UnitTest::GetInstance()->listeners();

	FGeneSplicerLibTestPrinter TestPrinter;
	Listeners.Append(&TestPrinter);

	const FString Filter = FString::Format(TEXT("{0}*"), { Parameters });
	const TArray<ANSICHAR> TestFilter{ TCHAR_TO_ANSI(*Filter), Filter.Len() + 1 };
	::testing::GTEST_FLAG(filter) = TestFilter.GetData();
	return (RUN_ALL_TESTS() == 0);
}

#endif  // WITH_DEV_AUTOMATION_TESTS