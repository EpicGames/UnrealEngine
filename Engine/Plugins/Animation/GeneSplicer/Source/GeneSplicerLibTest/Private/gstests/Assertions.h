// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Defs.h"
#include "gstests/Fixtures.h"

#include "genesplicer/GeneSplicerDNAReaderImpl.h"
#include "genesplicer/dna/Aliases.h"
#include "genesplicer/splicedata/genepool/BlendShapeDeltas.h"
#include "genesplicer/splicedata/genepool/SingleJointBehavior.h"
#include "genesplicer/splicedata/rawgenes/RawGenes.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/BlockStorage.h"
#include "genesplicer/types/VariableWidthMatrix.h"
#include "genesplicer/types/Vec3.h"

namespace gs4 {

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 6326)
#endif

inline void assertVec3Near(Vec3 result, Vec3 expected, float threshold) {
    ASSERT_NEAR(result.x, expected.x, threshold);
    ASSERT_NEAR(result.y, expected.y, threshold);
    ASSERT_NEAR(result.z, expected.z, threshold);
}

inline void assertNeutralMeshes(Reader* output, Reader* expected, float threshold) {
    ASSERT_EQ(output->getMeshCount(), expected->getMeshCount());
    for (std::uint16_t meshIdx = 0u; meshIdx < expected->getMeshCount(); ++meshIdx) {
        ASSERT_EQ(output->getVertexPositionCount(meshIdx), expected->getVertexPositionCount(meshIdx));
        for (std::uint32_t vertexIdx = 0u; vertexIdx < expected->getVertexPositionCount(meshIdx); ++vertexIdx) {
            Vec3 resultPosition = output->getVertexPosition(meshIdx, vertexIdx);
            Vec3 expectedPosition = expected->getVertexPosition(meshIdx, vertexIdx);
            assertVec3Near(resultPosition, expectedPosition, threshold);
        }
    }
}

inline void assertNeutralMeshes(ConstArrayView<RawVector3Vector> neutralMeshes, Reader* expected) {
    std::uint16_t expectedMeshCount = expected->getMeshCount();
    auto actualMeshCount = static_cast<std::uint16_t>(neutralMeshes.size());
    ASSERT_EQ(expectedMeshCount, actualMeshCount);
    for (std::uint16_t meshIdx = 0u; meshIdx < actualMeshCount; meshIdx++) {
        auto expectedVerticesXs = expected->getVertexPositionXs(meshIdx);
        const auto& actualVerticesXs = neutralMeshes[meshIdx].xs;
        ASSERT_ELEMENTS_AND_SIZE_EQ(expectedVerticesXs, actualVerticesXs);

        auto expectedVerticesYs = expected->getVertexPositionYs(meshIdx);
        const auto& actualVerticesYs = neutralMeshes[meshIdx].ys;
        ASSERT_ELEMENTS_AND_SIZE_EQ(expectedVerticesYs, actualVerticesYs);

        auto expectedVerticesZs = expected->getVertexPositionZs(meshIdx);
        const auto& actualVerticesZs = neutralMeshes[meshIdx].zs;
        ASSERT_ELEMENTS_AND_SIZE_EQ(expectedVerticesZs, actualVerticesZs);
    }
}

inline void assertNeutralMeshPoolData(ConstArrayView<XYZTiledMatrix<16u>> actualData,
                                      const Vector<Matrix<XYZBlock<16u>>>& expectedData) {
    ASSERT_EQ(actualData.size(), expectedData.size());
    for (std::uint16_t meshIndex = 0; meshIndex < expectedData.size(); meshIndex++) {
        const auto& mesh = actualData[meshIndex];
        const auto& expectedMesh = expectedData[meshIndex];
        std::size_t blockCount = mesh.rowCount();
        ASSERT_EQ(blockCount, expectedMesh.size());

        for (std::uint32_t blockIdx = 0u; blockIdx < blockCount; blockIdx++) {
            for (std::size_t dnaIdx = 0u; dnaIdx < mesh[blockIdx].size(); dnaIdx++) {
                ASSERT_EQ(mesh[blockIdx][dnaIdx], expectedMesh[blockIdx][dnaIdx]);
            }
        }
    }
}

inline void assertBlendShapeTargets(Reader* output, Reader* expected, float threshold) {
    ASSERT_EQ(output->getMeshCount(), expected->getMeshCount());
    for (std::uint16_t meshIdx = 0u; meshIdx < expected->getMeshCount(); ++meshIdx) {
        ASSERT_EQ(output->getBlendShapeTargetCount(meshIdx), expected->getBlendShapeTargetCount(meshIdx));
        for (std::uint16_t bsIdx = 0u; bsIdx < expected->getBlendShapeTargetCount(meshIdx); ++bsIdx) {
            ASSERT_EQ(output->getBlendShapeTargetDeltaCount(meshIdx, bsIdx),
                      expected->getBlendShapeTargetDeltaCount(meshIdx, bsIdx));

            auto resultIndices = output->getBlendShapeTargetVertexIndices(meshIdx, bsIdx);
            auto expectedIndices = expected->getBlendShapeTargetVertexIndices(meshIdx, bsIdx);
            ASSERT_EQ(resultIndices, expectedIndices);

            for (std::uint32_t deltaIdx = 0u; deltaIdx < expected->getBlendShapeTargetDeltaCount(meshIdx, bsIdx); ++deltaIdx) {
                Vec3 resultDelta = output->getBlendShapeTargetDelta(meshIdx, bsIdx, deltaIdx);
                Vec3 expectedDelta = expected->getBlendShapeTargetDelta(meshIdx, bsIdx, deltaIdx);
                assertVec3Near(resultDelta, expectedDelta, threshold);
            }
        }
    }
}

inline void assertBlendShapeTargets(const VariableWidthMatrix<RawBlendShapeTarget>& blendShapeTargets, Reader* expected) {
    std::uint16_t expectedMeshCount = expected->getMeshCount();

    ASSERT_EQ(blendShapeTargets.rowCount(), expectedMeshCount);
    for (std::uint16_t meshIndex = 0u; meshIndex < expectedMeshCount; meshIndex++) {
        auto mesh = blendShapeTargets[meshIndex];

        std::uint16_t expectedBlendShapeCount = expected->getBlendShapeTargetCount(meshIndex);
        ASSERT_EQ(mesh.size(), expectedBlendShapeCount);

        for (std::uint16_t bsIdx = 0u; bsIdx < expectedBlendShapeCount; bsIdx++) {
            auto expectedIndices = expected->getBlendShapeTargetVertexIndices(meshIndex, bsIdx);
            ASSERT_ELEMENTS_AND_SIZE_EQ(mesh[bsIdx].vertexIndices, expectedIndices);
            auto expectedXs = expected->getBlendShapeTargetDeltaXs(meshIndex, bsIdx);
            auto expectedYs = expected->getBlendShapeTargetDeltaYs(meshIndex, bsIdx);
            auto expectedZs = expected->getBlendShapeTargetDeltaZs(meshIndex, bsIdx);

            const auto& actualDeltas = mesh[bsIdx].deltas;
            ASSERT_ELEMENTS_AND_SIZE_EQ(actualDeltas.xs, expectedXs);
            ASSERT_ELEMENTS_AND_SIZE_EQ(actualDeltas.ys, expectedYs);
            ASSERT_ELEMENTS_AND_SIZE_EQ(actualDeltas.zs, expectedZs);
        }
    }
}

inline void assertBlendShapePoolVertexIndices(ConstArrayView<VariableWidthMatrix<std::uint32_t>> indices,
                                              const Vector<Matrix<std::uint32_t>>& expectedIndices) {
    ASSERT_EQ(indices.size(), expectedIndices.size());
    for (std::uint16_t meshIndex = 0u; meshIndex < indices.size(); meshIndex++) {
        const auto& mesh = indices[meshIndex];
        const auto& expectedMesh = expectedIndices[meshIndex];
        auto blendShapeCount = static_cast<std::uint16_t>(mesh.rowCount());
        ASSERT_EQ(blendShapeCount, expectedMesh.size());

        for (std::uint16_t bsIdx = 0u; bsIdx < blendShapeCount; bsIdx++) {
            ASSERT_EQ(mesh[bsIdx], expectedMesh[bsIdx]);
        }
    }
}

template<std::uint16_t BlockSize>
inline void assertBlendShapePoolDeltas(const VariableWidthMatrix<AlignedVariableWidthMatrix<XYZBlock<BlockSize>>>& deltas,
                                       const Matrix<Matrix<XYZBlock<BlockSize>>>& expectedDeltas) {
    ASSERT_EQ(deltas.rowCount(), expectedDeltas.size());
    for (std::uint16_t meshIndex = 0; meshIndex < expectedDeltas.size(); meshIndex++) {
        auto mesh = deltas[meshIndex];
        const auto& expectedMesh = expectedDeltas[meshIndex];
        ASSERT_EQ(mesh.size(), expectedMesh.size());

        for (std::uint16_t bsIdx = 0u; bsIdx < mesh.size(); bsIdx++) {
            const auto& blendShape = mesh[bsIdx];
            const auto& expectedBlendShape = expectedMesh[bsIdx];
            auto blockCount = static_cast<std::uint16_t>(blendShape.rowCount());

            for (std::uint32_t blockIdx = 0u; blockIdx < blockCount; blockIdx++) {
                for (std::size_t dnaIdx = 0u; dnaIdx < blendShape[blockIdx].size(); dnaIdx++) {
                    const auto& dnaValues = blendShape[blockIdx][dnaIdx];
                    const auto& expectedDNAValues = expectedBlendShape[blockIdx][dnaIdx];
                    ASSERT_EQ(dnaValues, expectedDNAValues);
                }
            }
        }
    }
}

template<std::uint16_t BlockSize>
inline void assertBlendShapePoolArchDeltas(const Vector<AlignedVariableWidthMatrix<XYZBlock<BlockSize>>>& deltas,
                                           const Vector<Matrix<XYZBlock<BlockSize>>>& expectedDeltas) {
    ASSERT_EQ(deltas.size(), expectedDeltas.size());

    for (std::uint16_t meshIndex = 0; meshIndex < expectedDeltas.size(); meshIndex++) {
        auto mesh = deltas[meshIndex];
        const auto& expectedMesh = expectedDeltas[meshIndex];
        ASSERT_EQ(mesh.rowCount(), expectedMesh.size());

        for (std::uint16_t bsIdx = 0u; bsIdx < mesh.rowCount(); bsIdx++) {
            const auto& blendShape = mesh[bsIdx];
            const auto& expectedBlendShape = expectedMesh[bsIdx];
            auto blockCount = static_cast<std::uint16_t>(blendShape.size());

            for (std::uint32_t blockIdx = 0u; blockIdx < blockCount; blockIdx++) {
                const auto& archValues = blendShape[blockIdx];
                const auto& expectedDNAValues = expectedBlendShape[blockIdx];
                ASSERT_EQ(archValues, expectedDNAValues);
            }
        }
    }
}

inline void assertBlendShapePoolDNAIndices(const VariableWidthMatrix<VariableWidthMatrix<std::uint16_t>>& dnaIndices,
                                           const Matrix<Matrix<std::uint16_t>>& expectedDNAIndices) {
    ASSERT_EQ(dnaIndices.rowCount(), expectedDNAIndices.size());

    for (std::uint16_t meshIndex = 0; meshIndex < expectedDNAIndices.size(); meshIndex++) {
        auto mesh = dnaIndices[meshIndex];
        const auto& expectedMesh = expectedDNAIndices[meshIndex];
        ASSERT_EQ(mesh.size(), expectedMesh.size());

        for (std::uint16_t bsIdx = 0u; bsIdx < mesh.size(); bsIdx++) {
            const auto& blendShape = mesh[bsIdx];
            const auto& expectedBlendShape = expectedMesh[bsIdx];
            auto blockCount = static_cast<std::uint16_t>(blendShape.rowCount());

            for (std::uint32_t blockIdx = 0u; blockIdx < blockCount; blockIdx++) {
                for (std::size_t dnaIdx = 0u; dnaIdx < blendShape[blockIdx].size(); dnaIdx++) {
                    const auto& indices = blendShape[blockIdx][dnaIdx];
                    const auto& expectedIndices = expectedBlendShape[blockIdx][dnaIdx];
                    ASSERT_EQ(indices, expectedIndices);
                }
            }
        }
    }
}

inline void assertNeutralJointTranslations(Reader* output, Reader* expected, float threshold) {
    ASSERT_EQ(output->getJointCount(), expected->getJointCount());
    for (std::uint16_t jointIdx = 0u; jointIdx < expected->getJointCount(); ++jointIdx) {
        Vec3 resultOffset = output->getNeutralJointTranslation(jointIdx);
        Vec3 expectedOffset = expected->getNeutralJointTranslation(jointIdx);
        assertVec3Near(resultOffset, expectedOffset, threshold);
    }
}

inline void assertNeutralJointTranslation(const RawVector3Vector& neutralJoints, Reader* expected) {
    auto expectedTranslationXs = expected->getNeutralJointTranslationXs();
    ASSERT_ELEMENTS_AND_SIZE_EQ(neutralJoints.xs, expectedTranslationXs);

    auto expectedTranslationYs = expected->getNeutralJointTranslationYs();
    ASSERT_ELEMENTS_AND_SIZE_EQ(neutralJoints.ys, expectedTranslationYs);

    auto expectedTranslationZs = expected->getNeutralJointTranslationZs();
    ASSERT_ELEMENTS_AND_SIZE_EQ(neutralJoints.zs, expectedTranslationZs);
}

inline void assertNeutralJoints(const RawVector3Vector& actual, Vector<Vector3> expected, float threshold) {
    ASSERT_EQ(actual.size(), expected.size());
    for (std::size_t i = 0; i < actual.size(); i++) {
        ASSERT_NEAR(actual.xs[i], expected[i].x, threshold);
        ASSERT_NEAR(actual.ys[i], expected[i].y, threshold);
        ASSERT_NEAR(actual.zs[i], expected[i].z, threshold);
    }
}

inline void assertNeutralJointPool(const XYZTiledMatrix<16u>& actual, const Matrix<XYZBlock<16u>>& expected) {
    std::size_t blockCount = actual.rowCount();
    ASSERT_EQ(blockCount, expected.size());

    for (std::uint32_t blockIdx = 0u; blockIdx < blockCount; blockIdx++) {
        for (std::size_t dnaIdx = 0u; dnaIdx < expected[blockIdx].size(); dnaIdx++) {
            const auto& actualBlock = actual[blockIdx][dnaIdx];
            const auto& expectedBlock = expected[blockIdx][dnaIdx];
            for (std::size_t i = 0; i < 16; i++) {
                ASSERT_NEAR(actualBlock.Xs[i], expectedBlock.Xs[i], 0.001);
                ASSERT_NEAR(actualBlock.Ys[i], expectedBlock.Ys[i], 0.001);
                ASSERT_NEAR(actualBlock.Zs[i], expectedBlock.Zs[i], 0.001);
            }
        }
    }
}

inline void assertNeutralJointRotations(Reader* output, Reader* expected, float threshold) {
    ASSERT_EQ(output->getJointCount(), expected->getJointCount());
    for (std::uint16_t jointIdx = 0u; jointIdx < expected->getJointCount(); ++jointIdx) {
        Vec3 resultOffset = output->getNeutralJointRotation(jointIdx);
        Vec3 expectedOffset = expected->getNeutralJointRotation(jointIdx);
        assertVec3Near(resultOffset, expectedOffset, threshold);
    }
}

inline void assertNeutralJointRotation(const RawVector3Vector& neutralJoints, Reader* expected) {
    auto expectedRotationXs = expected->getNeutralJointRotationXs();
    ASSERT_ELEMENTS_AND_SIZE_EQ(neutralJoints.xs, expectedRotationXs);

    auto expectedRotationYs = expected->getNeutralJointRotationYs();
    ASSERT_ELEMENTS_AND_SIZE_EQ(neutralJoints.ys, expectedRotationYs);

    auto expectedRotationZs = expected->getNeutralJointRotationZs();
    ASSERT_ELEMENTS_AND_SIZE_EQ(neutralJoints.zs, expectedRotationZs);
}

inline void assertJointBehavior(Reader* output, Reader* expected, float threshold) {
    ASSERT_EQ(output->getJointGroupCount(), expected->getJointGroupCount());
    for (std::uint16_t jointGroupIdx = 0u; jointGroupIdx < expected->getJointGroupCount(); ++jointGroupIdx) {
        auto resultOutputIndices = output->getJointGroupOutputIndices(jointGroupIdx);
        auto expectedOutputIndices = expected->getJointGroupOutputIndices(jointGroupIdx);
        ASSERT_EQ(resultOutputIndices, expectedOutputIndices);

        auto resultValues = output->getJointGroupValues(jointGroupIdx);
        auto expectedValues = expected->getJointGroupValues(jointGroupIdx);
        ASSERT_ELEMENTS_NEAR(resultValues, expectedValues, expectedValues.size(), threshold);

        auto resultLODs = output->getJointGroupLODs(jointGroupIdx);
        auto expectedLODs = expected->getJointGroupLODs(jointGroupIdx);
        ASSERT_EQ(resultLODs, expectedLODs);
    }
}

inline void assertJointBehavior(ConstArrayView<RawJointGroup> jointGroups, Reader* expected) {
    std::uint16_t expectedJointGroupCount = expected->getJointGroupCount();

    ASSERT_EQ(jointGroups.size(), expectedJointGroupCount);
    for (std::uint16_t jointGroupIdx = 0u; jointGroupIdx < expectedJointGroupCount; jointGroupIdx++) {

        const auto& actualJointGroup = jointGroups[jointGroupIdx];

        auto expectedInputIndices = expected->getJointGroupInputIndices(jointGroupIdx);
        ASSERT_ELEMENTS_AND_SIZE_EQ(expectedInputIndices, actualJointGroup.inputIndices);

        auto expectedOutputIndices = expected->getJointGroupOutputIndices(jointGroupIdx);
        ASSERT_ELEMENTS_AND_SIZE_EQ(expectedOutputIndices, actualJointGroup.outputIndices);

        auto expectedLODs = expected->getJointGroupLODs(jointGroupIdx);
        ASSERT_ELEMENTS_AND_SIZE_EQ(expectedLODs, actualJointGroup.lods);

        auto expectedValues = expected->getJointGroupValues(jointGroupIdx);
        ASSERT_ELEMENTS_AND_SIZE_EQ(expectedValues, actualJointGroup.values);
    }
}

inline void assertJointBehaviorPoolIndices(const VariableWidthMatrix<std::uint16_t>& indices,
                                           const Matrix<std::uint16_t>& expectedIndices) {
    auto jointGroupCount = static_cast<std::uint16_t>(expectedIndices.size());
    ASSERT_EQ(indices.rowCount(), jointGroupCount);
    for (std::uint16_t jntGrpIdx = 0u; jntGrpIdx < jointGroupCount; jntGrpIdx++) {
        ASSERT_EQ(indices[jntGrpIdx], expectedIndices[jntGrpIdx]);
    }
}

inline void assertJointBehaviorValues(ConstArrayView<SingleJointBehavior> actualJoints,
                                      const Vector<canonical::JBJoint>& expectedJoints) {
    for (std::uint16_t jntOffset = 0u; jntOffset < expectedJoints.size(); jntOffset++) {
        const auto& joint = actualJoints[jntOffset];
        const auto& expectedJoint = expectedJoints[jntOffset];
        auto blockValues = joint.getValues();
        ASSERT_EQ(blockValues.size(), expectedJoint.blockValues.size());

        for (std::size_t outPos = 0; outPos < blockValues.size(); outPos++) {
            const auto& outPosValues = blockValues[outPos];
            const auto& expectedOutPosValues = expectedJoint.blockValues[outPos];
            ASSERT_EQ(outPosValues.rowCount(), expectedOutPosValues.size());

            for (std::size_t vBlockIdx = 0u; vBlockIdx < outPosValues.rowCount(); vBlockIdx++) {
                for (std::size_t dnaIdx = 0u; dnaIdx < outPosValues[vBlockIdx].size(); dnaIdx++) {
                    ASSERT_EQ(outPosValues[vBlockIdx][dnaIdx], expectedOutPosValues[vBlockIdx][dnaIdx]);
                }
            }
        }
    }
}

inline void assertSkinWeights(Reader* output, Reader* expected, float threshold) {
    ASSERT_EQ(output->getMeshCount(), expected->getMeshCount());
    for (std::uint16_t meshIdx = 0; meshIdx < expected->getMeshCount(); ++meshIdx) {
        ASSERT_EQ(output->getVertexPositionCount(meshIdx), expected->getVertexPositionCount(meshIdx));
        for (std::uint32_t vertexIdx = 0; vertexIdx < expected->getVertexPositionCount(meshIdx); ++vertexIdx) {
            auto resultWeights = output->getSkinWeightsValues(meshIdx, vertexIdx);
            auto expectedWeights = expected->getSkinWeightsValues(meshIdx, vertexIdx);
            ASSERT_ELEMENTS_NEAR(resultWeights, expectedWeights, expectedWeights.size(), threshold);

            auto resultJointIndices = output->getSkinWeightsJointIndices(meshIdx, vertexIdx);
            auto expectedJointIndices = expected->getSkinWeightsJointIndices(meshIdx, vertexIdx);
            ASSERT_EQ(resultJointIndices, expectedJointIndices);
        }
    }
}

inline void assertSkinWeights(ConstArrayView<Vector<RawVertexSkinWeights>> skinWeights, Reader* expected) {
    std::uint16_t expectedMeshCount = expected->getMeshCount();
    auto actualMeshCount = static_cast<std::uint16_t>(skinWeights.size());
    ASSERT_EQ(actualMeshCount, expectedMeshCount);

    for (std::uint16_t meshIdx = 0u; meshIdx < actualMeshCount; meshIdx++) {
        std::uint32_t expectedSkinWeightCount = expected->getSkinWeightsCount(meshIdx);
        auto actualSkinWeightCount = static_cast<std::uint32_t>(skinWeights[meshIdx].size());
        ASSERT_EQ(actualSkinWeightCount, expectedSkinWeightCount);

        for (std::uint32_t vertexIdx = 0u; vertexIdx < expectedSkinWeightCount; vertexIdx++) {
            auto expectedValues = expected->getSkinWeightsValues(meshIdx, vertexIdx);
            const auto& actualValues = skinWeights[meshIdx][vertexIdx].weights;
            ASSERT_ELEMENTS_AND_SIZE_EQ(actualValues, expectedValues);

            auto expectedJointIndices = expected->getSkinWeightsJointIndices(meshIdx, vertexIdx);
            const auto& actualJointIndices = skinWeights[meshIdx][vertexIdx].jointIndices;
            ASSERT_ELEMENTS_AND_SIZE_EQ(actualJointIndices, expectedJointIndices);
        }
    }
}

inline void assertSkinWeightPoolJointIndices(ConstArrayView<VariableWidthMatrix<std::uint16_t>> indices,
                                             const Vector<Matrix<std::uint16_t>>& expectedIndices) {
    ASSERT_EQ(indices.size(), expectedIndices.size());

    for (std::uint32_t meshIndex = 0u; meshIndex < indices.size(); meshIndex++) {
        const auto& mesh = indices[meshIndex];
        const auto& expectedMesh = expectedIndices[meshIndex];
        const auto vertexCount = mesh.rowCount();
        ASSERT_EQ(vertexCount, expectedMesh.size());

        for (std::uint16_t vtxIdx = 0u; vtxIdx < vertexCount; vtxIdx++) {
            ASSERT_EQ(mesh[vtxIdx], expectedMesh[vtxIdx]);
        }
    }
}

inline void assertSkinWeightPoolValues(const VariableWidthMatrix<TiledMatrix2D<16u>>& weights,
                                       const Matrix<Matrix<VBlock<16u>>>& expectedWeights) {
    ASSERT_EQ(weights.rowCount(), expectedWeights.size());

    for (std::uint16_t meshIndex = 0; meshIndex < weights.rowCount(); meshIndex++) {
        auto mesh = weights[meshIndex];
        const auto& expectedMesh = expectedWeights[meshIndex];
        auto blockCount = static_cast<std::uint32_t>(mesh.size());
        ASSERT_EQ(blockCount, expectedMesh.size());

        for (std::uint32_t blockIdx = 0u; blockIdx < blockCount; blockIdx++) {
            const auto& dnaBlock = mesh[blockIdx];
            const auto& expectedDNABlock = expectedMesh[blockIdx];

            for (std::size_t dnaIdx = 0u; dnaIdx < expectedDNABlock.size(); dnaIdx++) {
                auto dnaValues = dnaBlock[dnaIdx];
                const auto& expectedDNAValues = expectedDNABlock[dnaIdx];
                ASSERT_EQ(dnaValues.size(), expectedDNAValues.size());
                auto jntPosCount = static_cast<std::uint16_t>(dnaValues.size());

                for (std::uint32_t jntPos = 0u; jntPos < jntPosCount; jntPos++) {
                    ASSERT_EQ(dnaValues[jntPos], expectedDNAValues[jntPos]);
                }
            }
        }
    }
}

inline void assertRawGenes(const RawGenes& rawGenes, Reader* expected) {
    ASSERT_EQ(expected->getMeshCount(), rawGenes.getMeshCount());
    for (std::uint16_t meshIdx = 0u; meshIdx < rawGenes.getMeshCount(); meshIdx++) {
        ASSERT_EQ(expected->getVertexPositionCount(meshIdx), rawGenes.getVertexCount(meshIdx));
        ASSERT_EQ(expected->getSkinWeightsCount(meshIdx), rawGenes.getSkinWeightsCount(meshIdx));
    }
    ASSERT_EQ(expected->getJointCount(), rawGenes.getJointCount());
    assertNeutralMeshes(rawGenes.getNeutralMeshes(), expected);
    assertBlendShapeTargets(rawGenes.getBlendShapeTargets(), expected);
    assertSkinWeights(rawGenes.getSkinWeights(), expected);
    assertNeutralJoints(rawGenes.getNeutralJoints(JointAttribute::Translation),
                        canonical::expectedRawGenesNeutralJointTranslations,
                        0.0001f);
    assertNeutralJoints(rawGenes.getNeutralJoints(JointAttribute::Rotation),
                        canonical::expectedRawGenesNeutralJointRotations,
                        0.0001f);
    assertJointBehavior(rawGenes.getJointGroups(), expected);
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif

}  // namespace gs4
