// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "gstests/Fixtures.h"
#include "gstests/MockedReader.h"

#include "genesplicer/TypeDefs.h"

#include <cstdint>

#if defined(__clang__) || defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wunused-parameter"
#endif
#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4100)
#endif
namespace gs4 {

class FixtureReader : public dna::MockedReader {
public:
    enum {
        archetype = 2u,
        expected = 3u
    };

public:
    static FixtureReader* create(std::uint16_t dnaIndex) {
        return new FixtureReader(dnaIndex);
    }

    static void destroy(FixtureReader* instance) {
        delete instance;
    }

    explicit FixtureReader(std::uint16_t dnaIndex_) :
        dnaIndex{dnaIndex_} {
    }

    StringView getName() const override {
        return {"Character", 9ul};
    }

    std::uint16_t getPSDCount() const override {
        return canonical::psdCount;
    }

    std::uint16_t getMeshCount() const override {
        return canonical::meshCount;
    }

    std::uint16_t getJointCount() const override {
        return canonical::jointCount;
    }

    StringView getMeshName(std::uint16_t meshIndex) const override {
        return {"testMesh", 8ul};
    }

    StringView getJointName(std::uint16_t jointIndex) const override {
        return {"testJoint", 9ul};
    }

    StringView getBlendShapeChannelName(std::uint16_t channelIndex) const override {
        return {"testBlendShape", 14ul};
    }

    Vector3 getNeutralJointTranslation(std::uint16_t index) const override {
        return {canonical::neutralJointTranslations[dnaIndex][0ul][index],
                canonical::neutralJointTranslations[dnaIndex][1ul][index],
                canonical::neutralJointTranslations[dnaIndex][2ul][index]};
    }

    ConstArrayView<float> getNeutralJointTranslationXs() const override {
        return ConstArrayView<float>{canonical::neutralJointTranslations[dnaIndex][0ul]};
    }

    ConstArrayView<float> getNeutralJointTranslationYs() const override {
        return ConstArrayView<float>{canonical::neutralJointTranslations[dnaIndex][1ul]};
    }

    ConstArrayView<float> getNeutralJointTranslationZs() const override {
        return ConstArrayView<float>{canonical::neutralJointTranslations[dnaIndex][2ul]};
    }

    Vector3 getNeutralJointRotation(std::uint16_t index) const override {
        return {canonical::neutralJointRotations[dnaIndex][0ul][index],
                canonical::neutralJointRotations[dnaIndex][1ul][index],
                canonical::neutralJointRotations[dnaIndex][2ul][index]};
    }

    ConstArrayView<float> getNeutralJointRotationXs() const override {
        return ConstArrayView<float>{canonical::neutralJointRotations[dnaIndex][0ul]};
    }

    ConstArrayView<float> getNeutralJointRotationYs() const override {
        return ConstArrayView<float>{canonical::neutralJointRotations[dnaIndex][1ul]};
    }

    ConstArrayView<float> getNeutralJointRotationZs() const override {
        return ConstArrayView<float>{canonical::neutralJointRotations[dnaIndex][2ul]};
    }

    std::uint32_t getVertexPositionCount(std::uint16_t meshIndex) const override {
        return static_cast<std::uint32_t>(canonical::neutralMeshes[dnaIndex][meshIndex][0].size());
    }

    dna::Position getVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override {
        return {canonical::neutralMeshes[dnaIndex][meshIndex][0ul][vertexIndex],
                canonical::neutralMeshes[dnaIndex][meshIndex][1ul][vertexIndex],
                canonical::neutralMeshes[dnaIndex][meshIndex][2ul][vertexIndex]};
    }

    ConstArrayView<float> getVertexPositionXs(std::uint16_t meshIndex) const override {
        return ConstArrayView<float>{canonical::neutralMeshes[dnaIndex][meshIndex][0ul]};
    }

    ConstArrayView<float> getVertexPositionYs(std::uint16_t meshIndex) const override {
        return ConstArrayView<float>{canonical::neutralMeshes[dnaIndex][meshIndex][1ul]};
    }

    ConstArrayView<float> getVertexPositionZs(std::uint16_t meshIndex) const override {
        return ConstArrayView<float>{canonical::neutralMeshes[dnaIndex][meshIndex][2ul]};
    }

    std::uint32_t getVertexNormalCount(std::uint16_t meshIndex) const override {
        return static_cast<std::uint32_t>(canonical::neutralMeshes[dnaIndex][meshIndex][0].size());
    }

    dna::Normal getVertexNormal(std::uint16_t meshIndex, std::uint32_t normalIndex) const override {
        return {canonical::neutralMeshes[dnaIndex][meshIndex][0ul][normalIndex],
                canonical::neutralMeshes[dnaIndex][meshIndex][1ul][normalIndex],
                canonical::neutralMeshes[dnaIndex][meshIndex][2ul][normalIndex]};
    }

    ConstArrayView<float> getVertexNormalXs(std::uint16_t meshIndex) const override {
        return ConstArrayView<float>{canonical::neutralMeshes[dnaIndex][meshIndex][0ul]};
    }

    ConstArrayView<float> getVertexNormalYs(std::uint16_t meshIndex) const override {
        return ConstArrayView<float>{canonical::neutralMeshes[dnaIndex][meshIndex][1ul]};
    }

    ConstArrayView<float> getVertexNormalZs(std::uint16_t meshIndex) const override {
        return ConstArrayView<float>{canonical::neutralMeshes[dnaIndex][meshIndex][2ul]};
    }

    ConstArrayView<float> getSkinWeightsValues(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override {
        return ConstArrayView<float>{canonical::skinWeightsValues[dnaIndex][meshIndex][vertexIndex]};
    }

    ConstArrayView<std::uint16_t> getSkinWeightsJointIndices(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override {
        return ConstArrayView<std::uint16_t>{canonical::skinWeightsJointIndices[dnaIndex][meshIndex][vertexIndex]};
    }

    std::uint32_t getSkinWeightsCount(std::uint16_t meshIndex) const override {
        return static_cast<std::uint32_t>(canonical::skinWeightsValues[dnaIndex][meshIndex].size());
    }

    std::uint16_t getMaximumInfluencePerVertex(std::uint16_t meshIndex) const override {
        return 2ul;
    }

    std::uint16_t getJointGroupCount() const override {
        return canonical::regionCount;
    }

    ConstArrayView<std::uint16_t> getJointGroupLODs(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{canonical::jointGroupLODs[dnaIndex][jointGroupIndex]};
    }

    ConstArrayView<std::uint16_t> getJointGroupInputIndices(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{canonical::jointGroupInputIndices[jointGroupIndex]};
    }

    ConstArrayView<std::uint16_t> getJointGroupOutputIndices(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{canonical::jointGroupOutputIndices[dnaIndex][jointGroupIndex]};
    }

    ConstArrayView<float> getJointGroupValues(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<float>{canonical::jointGroupValues[dnaIndex][jointGroupIndex]};
    }

    ConstArrayView<std::uint16_t> getJointGroupJointIndices(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{canonical::jointGroupJointIndices[dnaIndex][jointGroupIndex]};
    }

    std::uint16_t getBlendShapeChannelCount() const override {
        return canonical::blendShapeCount;
    }

    std::uint16_t getBlendShapeTargetCount(std::uint16_t meshIndex) const override {
        return canonical::blendShapeCount;
    }

    std::uint32_t getBlendShapeTargetDeltaCount(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override {
        return static_cast<std::uint32_t>(canonical::blendShapeTargetDeltas[dnaIndex][blendShapeTargetIndex][0ul].size());
    }

    dna::Delta getBlendShapeTargetDelta(std::uint16_t meshIndex,
                                        std::uint16_t blendShapeTargetIndex,
                                        std::uint32_t deltaIndex) const override {
        const auto& deltas = canonical::blendShapeTargetDeltas[dnaIndex][blendShapeTargetIndex];
        return {deltas[0ul][deltaIndex], deltas[1ul][deltaIndex], deltas[2ul][deltaIndex]};
    }

    ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t meshIndex,
                                                     std::uint16_t blendShapeTargetIndex) const override {
        return ConstArrayView<float>{canonical::blendShapeTargetDeltas[dnaIndex][blendShapeTargetIndex][0ul]};
    }

    ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t meshIndex,
                                                     std::uint16_t blendShapeTargetIndex) const override {
        return ConstArrayView<float>{canonical::blendShapeTargetDeltas[dnaIndex][blendShapeTargetIndex][1ul]};
    }

    ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t meshIndex,
                                                     std::uint16_t blendShapeTargetIndex) const override {
        return ConstArrayView<float>{canonical::blendShapeTargetDeltas[dnaIndex][blendShapeTargetIndex][2ul]};
    }

    ConstArrayView<std::uint32_t> getBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                                   std::uint16_t blendShapeTargetIndex) const override {
        return ConstArrayView<std::uint32_t>{canonical::blendShapeTargetVertexIndices[dnaIndex][blendShapeTargetIndex]};
    }

    std::uint16_t getMLTypeCount() const override {
        return {};
    }

    std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const override {
        return {};
    }

    std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const override {
        return {};
    }

    dna::MachineLearnedBehaviorOperationType getMLOperationType(std::uint16_t mlTypeIndex,
                                                                std::uint16_t mlOperationSetIndex,
                                                                std::uint16_t mlOperationIndex) const override {
        return {};
    }

    ConstArrayView<std::uint32_t> getMLOperationParameters(std::uint16_t mlTypeIndex,
                                                           std::uint16_t mlOperationSetIndex,
                                                           std::uint16_t mlOperationIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLOperationDependencyOperationSetIndices(std::uint16_t mlTypeIndex,
                                                                              std::uint16_t mlOperationSetIndex,
                                                                              std::uint16_t mlOperationIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLOperationDependencyOperationIndices(std::uint16_t mlTypeIndex,
                                                                           std::uint16_t mlOperationSetIndex,
                                                                           std::uint16_t mlOperationIndex) const override {
        return {};
    }

    std::uint16_t getMLOperationIndexListCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLOperationIndicesForLOD(std::uint16_t mlTypeIndex,
                                                              std::uint16_t mlOperationSetIndex,
                                                              std::uint16_t lod) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLJointsInputIndices() const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLJointsOutputIndices() const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLJointsParameterKeys() const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLJointsParameterValues() const override {
        return {};
    }

private:
    std::uint16_t dnaIndex;
};

}  // namespace gs4
#if defined(__clang__) || defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
