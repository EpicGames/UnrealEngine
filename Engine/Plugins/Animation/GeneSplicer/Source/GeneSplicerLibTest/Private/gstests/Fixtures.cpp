// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Fixtures.h"

#include "genesplicer/TypeDefs.h"
#include "genesplicer/dna/Aliases.h"
#include "genesplicer/types/Block.h"

#include <cstddef>
#include <cstdint>

#ifdef _MSC_VER
    #pragma warning(disable : 4503)
#endif

namespace gs4 {

namespace canonical {
// *INDENT-OFF*
const Matrix<Matrix<float>> neutralMeshes = {{// DNA-0
                                              {
                                                  // Mesh-0
                                                  // V0    V1    V2    V3    V4    V5    V6    V7    V8    V9    V10   V11   V12
                                                  // V13   V14   V15   V16   V17   V18   V19   V20
                                                  {1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f,
                                                   1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f},  // Xs
                                                  {2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f,
                                                   0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f},  // Ys
                                                  {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
                                                   0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // Zs
                                              },
                                              {
                                                  // Mesh-1
                                                  // V0    V1    V2    V3    V4    V5    V6    V7    V8    V9    V10   V11   V12
                                                  // V13   V14   V15   V16   V17   V18   V19   V20
                                                  {1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f,
                                                   1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f},  // Xs
                                                  {2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f,
                                                   0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f},  // Ys
                                                  {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
                                                   0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // Zs
                                              }},
                                             {// DNA-1
                                              {
                                                  // Mesh-0
                                                  // V0    V1    V2    V3    V4    V5    V6    V7    V8    V9    V10   V11   V12
                                                  // V13   V14   V15   V16   V17   V18   V19   V20
                                                  {0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f,
                                                   2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f},  // Xs
                                                  {1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f,
                                                   0.0f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f},  // Ys
                                                  {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
                                                   0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // Zs
                                              },
                                              {
                                                  // Mesh-1
                                                  // V0    V1    V2    V3    V4    V5    V6    V7    V8    V9    V10   V11   V12
                                                  // V13   V14   V15   V16   V17   V18   V19   V20
                                                  {0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f,
                                                   2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 2.0f},  // Xs
                                                  {1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f,
                                                   0.0f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f},  // Ys
                                                  {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
                                                   0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // Zs
                                              }},
                                             {// DNA arch
                                              {
                                                  // Mesh-0
                                                  // V0    V1    V2    V3    V4    V5    V6    V7    V8    V9    V10   V11   V12
                                                  // V13   V14   V15   V16   V17   V18   V19   V20
                                                  {0.5f, 0.0f, 1.5f, 0.5f, 0.0f, 1.5f, 0.5f, 0.0f, 1.5f, 0.5f, 0.0f,
                                                   1.5f, 0.5f, 0.0f, 1.5f, 0.5f, 0.0f, 1.5f, 0.5f, 0.0f, 1.5f},  // Xs
                                                  {1.5f, 0.0f, 0.0f, 1.5f, 0.0f, 0.0f, 1.5f, 0.0f, 0.0f, 1.5f, 0.0f,
                                                   0.0f, 1.5f, 0.0f, 0.0f, 1.5f, 0.0f, 0.0f, 1.5f, 0.0f, 0.0f},  // Ys
                                                  {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
                                                   0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // Zs
                                              },
                                              {
                                                  // Mesh-1
                                                  // V0    V1    V2    V3    V4    V5    V6    V7    V8    V9    V10   V11   V12
                                                  // V13   V14   V15   V16   V17   V18   V19   V20
                                                  {0.5f, 0.0f, 1.5f, 0.5f, 0.0f, 1.5f, 0.5f, 0.0f, 1.5f, 0.5f, 0.0f,
                                                   1.5f, 0.5f, 0.0f, 1.5f, 0.5f, 0.0f, 1.5f, 0.5f, 0.0f, 1.5f},  // Xs
                                                  {1.5f, 0.0f, 0.0f, 1.5f, 0.0f, 0.0f, 1.5f, 0.0f, 0.0f, 1.5f, 0.0f,
                                                   0.0f, 1.5f, 0.0f, 0.0f, 1.5f, 0.0f, 0.0f, 1.5f, 0.0f, 0.0f},  // Ys
                                                  {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
                                                   0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // Zs
                                              }},
                                             {// DNA expected
                                              {
                                                  // Mesh-0
                                                  // V0    V1    V2    V3     V4    V5    V6     V7    V8    V9    V10   V11 V12
                                                  // V13   V14    V15   V16   V17    V18    V19   V20
                                                  {0.46f, 0.0f,  1.3f, 0.46f, 0.0f,  1.3f, 0.46f, 0.0f,  1.3f, 0.46f, 0.0f,
                                                   1.3f,  0.46f, 0.0f, 1.3f,  0.46f, 0.0f, 1.3f,  0.46f, 0.0f, 1.3f},  // Xs
                                                  {1.46f, 0.0f,  0.0f, 1.46f, 0.0f,  0.0f, 1.46f, 0.0f,  0.0f, 1.46f, 0.0f,
                                                   0.0f,  1.46f, 0.0f, 0.0f,  1.46f, 0.0f, 0.0f,  1.46f, 0.0f, 0.0f},  // Ys
                                                  {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
                                                   0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // Zs
                                              },
                                              {
                                                  // Mesh-1
                                                  // V0    V1    V2    V3     V4    V5    V6     V7    V8    V9    V10   V11 V12
                                                  // V13   V14    V15   V16   V17    V18    V19   V20
                                                  {0.46f, 0.0f,  1.3f, 0.46f, 0.0f,  1.3f, 0.46f, 0.0f,  1.3f, 0.46f, 0.0f,
                                                   1.3f,  0.46f, 0.0f, 1.3f,  0.46f, 0.0f, 1.3f,  0.46f, 0.0f, 1.3f},  // Xs
                                                  {1.46f, 0.0f,  0.0f, 1.46f, 0.0f,  0.0f, 1.46f, 0.0f,  0.0f, 1.46f, 0.0f,
                                                   0.0f,  1.46f, 0.0f, 0.0f,  1.46f, 0.0f, 0.0f,  1.46f, 0.0f, 0.0f},  // Ys
                                                  {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
                                                   0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // Zs
                                              }}};

const Matrix<Matrix<float>> skinWeightsValues = {{// DNA-0
                                                  {
                                                      // Mesh-0
                                                      {1.0f},        // V0 {J0}
                                                      {1.0f},        // V1 {J1}
                                                      {1.0f},        // V2 {J2}
                                                      {0.4f, 0.6f},  // V3 {J0, J1}
                                                      {0.2f, 0.8f},  // V4 {J0, J2}
                                                      {0.1f, 0.9f},  // V5 {J0, J1}
                                                      {1.0f},        // V6 {J0}
                                                      {1.0f},        // V7 {J1}
                                                      {1.0f},        // V8 {J2}
                                                      {1.0f},        // V9 {J0}
                                                      {1.0f},        // V10 {J1}
                                                      {1.0f},        // V11 {J2}
                                                      {1.0f},        // V12 {J0}
                                                      {1.0f},        // V13 {J1}
                                                      {1.0f},        // V14 {J2}
                                                      {1.0f},        // V15 {J0}
                                                      {1.0f},        // V16 {J1}
                                                      {1.0f},        // V17 {J2}
                                                      {1.0f},        // V18 {J0}
                                                      {1.0f},        // V19 {J1}
                                                      {1.0f}         // V20 {J2}
                                                  },
                                                  {
                                                      // Mesh-1
                                                      {1.0f},        // V0 {J0}
                                                      {1.0f},        // V1 {J1}
                                                      {1.0f},        // V2 {J2}
                                                      {0.4f, 0.6f},  // V3 {J0, J1}
                                                      {0.2f, 0.8f},  // V4 {J0, J2}
                                                      {0.1f, 0.9f},  // V5 {J0, J1}
                                                      {1.0f},        // V6 {J0}
                                                      {1.0f},        // V7 {J1}
                                                      {1.0f},        // V8 {J2}
                                                      {1.0f},        // V9 {J0}
                                                      {1.0f},        // V10 {J1}
                                                      {1.0f},        // V11 {J2}
                                                      {1.0f},        // V12 {J0}
                                                      {1.0f},        // V13 {J1}
                                                      {1.0f},        // V14 {J2}
                                                      {1.0f},        // V15 {J0}
                                                      {1.0f},        // V16 {J1}
                                                      {1.0f},        // V17 {J2}
                                                      {1.0f},        // V18 {J0}
                                                      {1.0f},        // V19 {J1}
                                                      {1.0f}         // V20 {J2}
                                                  }},
                                                 {// DNA-1
                                                  {
                                                      // Mesh-0
                                                      {1.0f},        // V0 {J2}
                                                      {1.0f},        // V1 {J1}
                                                      {1.0f},        // V2 {J0}
                                                      {0.5f, 0.5f},  // V3 {J1, J2}
                                                      {0.2f, 0.8f},  // V4 {J0, J1}
                                                      {0.8f, 0.2f},  // V5 {J1, J2}
                                                      {0.4f, 0.6f},  // V6 {J0, J2}
                                                      {1.0f},        // V7 {J0}
                                                      {1.0f},        // V8 {J1}
                                                      {1.0f},        // V9 {J2}
                                                      {1.0f},        // V10 {J0}
                                                      {1.0f},        // V11 {J1}
                                                      {1.0f},        // V12 {J2}
                                                      {1.0f},        // V13 {J0}
                                                      {1.0f},        // V14 {J1}
                                                      {1.0f},        // V15 {J2}
                                                      {1.0f},        // V16 {J0}
                                                      {1.0f},        // V17 {J1}
                                                      {1.0f},        // V18 {J2}
                                                      {1.0f},        // V19 {J0}
                                                      {1.0f}         // V20 {J1}
                                                  },
                                                  {
                                                      // Mesh-1
                                                      {1.0f},        // V0 {J2}
                                                      {1.0f},        // V1 {J1}
                                                      {1.0f},        // V2 {J0}
                                                      {0.5f, 0.5f},  // V3 {J1, J2}
                                                      {0.2f, 0.8f},  // V4 {J0, J1}
                                                      {0.8f, 0.2f},  // V5 {J1, J2}
                                                      {0.4f, 0.6f},  // V6 {J0, J2}
                                                      {1.0f},        // V7 {J0}
                                                      {1.0f},        // V8 {J1}
                                                      {1.0f},        // V9 {J2}
                                                      {1.0f},        // V10 {J0}
                                                      {1.0f},        // V11 {J1}
                                                      {1.0f},        // V12 {J2}
                                                      {1.0f},        // V13 {J0}
                                                      {1.0f},        // V14 {J1}
                                                      {1.0f},        // V15 {J2}
                                                      {1.0f},        // V16 {J0}
                                                      {1.0f},        // V17 {J1}
                                                      {1.0f},        // V18 {J2}
                                                      {1.0f},        // V19 {J0}
                                                      {1.0f}         // V20 {J1}
                                                  }},
                                                 {// DNA arch
                                                  {
                                                      // Mesh-0
                                                      {0.5f, 0.5f},        // V0
                                                      {1.0f},              // V1
                                                      {0.5f, 0.5f},        // V2
                                                      {0.6875f, 0.3125f},  // V3
                                                      {0.667f, 0.333f},    // V4
                                                      {0.833f, 0.167f},    // V5
                                                      {0.7f, 0.3f},        // V6
                                                      {0.5f, 0.5f},        // V7
                                                      {0.5f, 0.5f},        // V8
                                                      {0.5f, 0.5f},        // V9
                                                      {0.5f, 0.5f},        // V10
                                                      {0.5f, 0.5f},        // V11
                                                      {0.5f, 0.5f},        // V12
                                                      {0.5f, 0.5f},        // V13
                                                      {0.5f, 0.5f},        // V14
                                                      {0.5f, 0.5f},        // V15
                                                      {0.5f, 0.5f},        // V16
                                                      {0.5f, 0.5f},        // V17
                                                      {0.5f, 0.5f},        // V18
                                                      {0.5f, 0.5f},        // V19
                                                      {0.5f, 0.5f}         // V20
                                                  },
                                                  {
                                                      // Mesh-1
                                                      {0.5f, 0.5f},        // V0
                                                      {1.0f},              // V1
                                                      {0.5f, 0.5f},        // V2
                                                      {0.6875f, 0.3125f},  // V3
                                                      {0.667f, 0.333f},    // V4
                                                      {0.833f, 0.167f},    // V5
                                                      {0.7f, 0.3f},        // V6
                                                      {0.5f, 0.5f},        // V7
                                                      {0.5f, 0.5f},        // V8
                                                      {0.5f, 0.5f},        // V9
                                                      {0.5f, 0.5f},        // V10
                                                      {0.5f, 0.5f},        // V11
                                                      {0.5f, 0.5f},        // V12
                                                      {0.5f, 0.5f},        // V13
                                                      {0.5f, 0.5f},        // V14
                                                      {0.5f, 0.5f},        // V15
                                                      {0.5f, 0.5f},        // V16
                                                      {0.5f, 0.5f},        // V17
                                                      {0.5f, 0.5f},        // V18
                                                      {0.5f, 0.5f},        // V19
                                                      {0.5f, 0.5f}         // V20
                                                  }},
                                                 {// DNA expected
                                                  {
                                                      // Mesh-0
                                                      {0.471830985915493f, 0.528169014084507f},  // V0 {J0, J2}
                                                      {1.0f},                                    // V1 {J1}
                                                      {0.16667f, 0.83333f},                      // V2 {J0, J2}
                                                      {0.6745f, 0.3255f},                        // V3 {J1, J2}
                                                      {0.625f, 0.375f},                          // V4{J1, J2}
                                                      {0.913793103448276f, 0.086206896551724f},  // V5 {J1, J0}
                                                      {0.683098591549296f, 0.316901408450704f},  // V6 {J0, j2}
                                                      {0.625f, 0.375f},                          // V7 {J0, J1}
                                                      {0.16667f, 0.83333f},                      // V8 {J1, J2}
                                                      {0.471830985915493f, 0.528169014084507f},  // V9 {J0, J2}
                                                      {0.625f, 0.375f},                          // V10 {J0, J1}
                                                      {0.16667f, 0.83333f},                      // V11 {J1, J2}
                                                      {0.471830985915493f, 0.528169014084507f},  // V12 {J0, J2}
                                                      {0.625f, 0.375f},                          // V13 {J0, J1}
                                                      {0.16667f, 0.83333f},                      // V14 {J1, J2}
                                                      {0.471830985915493f, 0.528169014084507f},  // V15 {J0, J2}
                                                      {0.625f, 0.375f},                          // V16 {J0, J1}
                                                      {0.16667f, 0.83333f},                      // V17 {J1, J2}
                                                      {0.471830985915493f, 0.528169014084507f},  // V18 {J0, J2}
                                                      {0.625f, 0.375f},                          // V19 {J0, J1}
                                                      {0.16667f, 0.83333f},                      // V20 {J1, J2}
                                                  },
                                                  {
                                                      // Mesh-1
                                                      {0.471830985915493f, 0.528169014084507f},  // V0 {J0, J2}
                                                      {1.0f},                                    // V1 {J1}
                                                      {0.16667f, 0.83333f},                      // V2 {J0, J2}
                                                      {0.6745f, 0.3255f},                        // V3 {J1, J2}
                                                      {0.625f, 0.375f},                          // V4{J1, J2}
                                                      {0.913793103448276f, 0.086206896551724f},  // V5 {J1, J0}
                                                      {0.683098591549296f, 0.316901408450704f},  // V6 {J0, j2}
                                                      {0.625f, 0.375f},                          // V7 {J0, J1}
                                                      {0.16667f, 0.83333f},                      // V8 {J1, J2}
                                                      {0.471830985915493f, 0.528169014084507f},  // V9 {J0, J2}
                                                      {0.625f, 0.375f},                          // V10 {J0, J1}
                                                      {0.16667f, 0.83333f},                      // V11 {J1, J2}
                                                      {0.471830985915493f, 0.528169014084507f},  // V12 {J0, J2}
                                                      {0.625f, 0.375f},                          // V13 {J0, J1}
                                                      {0.16667f, 0.83333f},                      // V14 {J1, J2}
                                                      {0.471830985915493f, 0.528169014084507f},  // V15 {J0, J2}
                                                      {0.625f, 0.375f},                          // V16 {J0, J1}
                                                      {0.16667f, 0.83333f},                      // V17 {J1, J2}
                                                      {0.471830985915493f, 0.528169014084507f},  // V18 {J0, J2}
                                                      {0.625f, 0.375f},                          // V19 {J0, J1}
                                                      {0.16667f, 0.83333f},                      // V20 {J1, J2}
                                                  }}};

const Matrix<Matrix<std::uint16_t>> skinWeightsJointIndices = {{// DNA-0
                                                                {
                                                                    // Mesh-0
                                                                    {0},     // V0
                                                                    {1},     // V1
                                                                    {2},     // V2
                                                                    {0, 1},  // V3
                                                                    {0, 2},  // V4
                                                                    {0, 1},  // V5
                                                                    {0},     // V6
                                                                    {1},     // V7
                                                                    {2},     // V8
                                                                    {0},     // V9
                                                                    {1},     // V10
                                                                    {2},     // V11
                                                                    {0},     // V12
                                                                    {1},     // V13
                                                                    {2},     // V14
                                                                    {0},     // V15
                                                                    {1},     // V16
                                                                    {2},     // V17
                                                                    {0},     // V18
                                                                    {1},     // V19
                                                                    {2}      // V20
                                                                },
                                                                {
                                                                    // Mesh-1
                                                                    {0},     // V0
                                                                    {1},     // V1
                                                                    {2},     // V2
                                                                    {0, 1},  // V3
                                                                    {0, 2},  // V4
                                                                    {0, 1},  // V5
                                                                    {0},     // V6
                                                                    {1},     // V7
                                                                    {2},     // V8
                                                                    {0},     // V9
                                                                    {1},     // V10
                                                                    {2},     // V11
                                                                    {0},     // V12
                                                                    {1},     // V13
                                                                    {2},     // V14
                                                                    {0},     // V15
                                                                    {1},     // V16
                                                                    {2},     // V17
                                                                    {0},     // V18
                                                                    {1},     // V19
                                                                    {2}      // V20
                                                                }},
                                                               {// DNA-1
                                                                {
                                                                    // Mesh-0
                                                                    {2},     // V0
                                                                    {1},     // V1
                                                                    {0},     // V2
                                                                    {1, 2},  // V3
                                                                    {0, 1},  // V4
                                                                    {1, 2},  // V5
                                                                    {0, 2},  // V6
                                                                    {0},     // V7
                                                                    {1},     // V8
                                                                    {2},     // V9
                                                                    {0},     // V10
                                                                    {1},     // V11
                                                                    {2},     // V12
                                                                    {0},     // V13
                                                                    {1},     // V14
                                                                    {2},     // V15
                                                                    {0},     // V16
                                                                    {1},     // V17
                                                                    {2},     // V18
                                                                    {0},     // V19
                                                                    {1}      // V20
                                                                },
                                                                {
                                                                    // Mesh-1
                                                                    {2},     // V0
                                                                    {1},     // V1
                                                                    {0},     // V2
                                                                    {1, 2},  // V3
                                                                    {0, 1},  // V4
                                                                    {1, 2},  // V5
                                                                    {0, 2},  // V6
                                                                    {0},     // V7
                                                                    {1},     // V8
                                                                    {2},     // V9
                                                                    {0},     // V10
                                                                    {1},     // V11
                                                                    {2},     // V12
                                                                    {0},     // V13
                                                                    {1},     // V14
                                                                    {2},     // V15
                                                                    {0},     // V16
                                                                    {1},     // V17
                                                                    {2},     // V18
                                                                    {0},     // V19
                                                                    {1}      // V20
                                                                }},
                                                               {// DNA arch
                                                                {
                                                                    // Mesh-0
                                                                    {0, 2},  // V0
                                                                    {1},     // V1
                                                                    {0, 2},  // V2
                                                                    {1, 2},  // V3
                                                                    {0, 2},  // V4
                                                                    {1, 2},  // V5
                                                                    {0, 2},  // V6
                                                                    {0, 1},  // V7
                                                                    {1, 2},  // V8
                                                                    {0, 2},  // V9
                                                                    {0, 1},  // V10
                                                                    {1, 2},  // V11
                                                                    {0, 2},  // V12
                                                                    {0, 1},  // V13
                                                                    {1, 2},  // V14
                                                                    {0, 2},  // V15
                                                                    {0, 1},  // V16
                                                                    {1, 2},  // V17
                                                                    {0, 2},  // V18
                                                                    {0, 1},  // V19
                                                                    {1, 2}   // V20
                                                                },
                                                                {
                                                                    // Mesh-1
                                                                    {0, 2},  // V0
                                                                    {1},     // V1
                                                                    {0, 2},  // V2
                                                                    {1, 2},  // V3
                                                                    {0, 2},  // V4
                                                                    {1, 2},  // V5
                                                                    {0, 2},  // V6
                                                                    {0, 1},  // V7
                                                                    {1, 2},  // V8
                                                                    {0, 2},  // V9
                                                                    {0, 1},  // V10
                                                                    {1, 2},  // V11
                                                                    {0, 2},  // V12
                                                                    {0, 1},  // V13
                                                                    {1, 2},  // V14
                                                                    {0, 2},  // V15
                                                                    {0, 1},  // V16
                                                                    {1, 2},  // V17
                                                                    {0, 2},  // V18
                                                                    {0, 1},  // V19
                                                                    {1, 2}   // V20
                                                                }},
                                                               {// DNA expected
                                                                {
                                                                    // Mesh-0
                                                                    {0, 2},  // V0
                                                                    {1},     // V1
                                                                    {0, 2},  // V2
                                                                    {1, 2},  // V3
                                                                    {1, 2},  // V4
                                                                    {1, 0},  // V5
                                                                    {0, 2},  // V6
                                                                    {0, 1},  // V7
                                                                    {1, 2},  // V8
                                                                    {0, 2},  // V9
                                                                    {0, 1},  // V10
                                                                    {1, 2},  // V11
                                                                    {0, 2},  // V12
                                                                    {0, 1},  // V13
                                                                    {1, 2},  // V14
                                                                    {0, 2},  // V15
                                                                    {0, 1},  // V16
                                                                    {1, 2},  // V17
                                                                    {0, 2},  // V18
                                                                    {0, 1},  // V19
                                                                    {1, 2}   // V20
                                                                },
                                                                {
                                                                    // Mesh-1
                                                                    {0, 2},  // V0
                                                                    {1},     // V1
                                                                    {0, 2},  // V2
                                                                    {1, 2},  // V3
                                                                    {1, 2},  // V4
                                                                    {1, 0},  // V5
                                                                    {0, 2},  // V6
                                                                    {0, 1},  // V7
                                                                    {1, 2},  // V8
                                                                    {0, 2},  // V9
                                                                    {0, 1},  // V10
                                                                    {1, 2},  // V11
                                                                    {0, 2},  // V12
                                                                    {0, 1},  // V13
                                                                    {1, 2},  // V14
                                                                    {0, 2},  // V15
                                                                    {0, 1},  // V16
                                                                    {1, 2},  // V17
                                                                    {0, 2},  // V18
                                                                    {0, 1},  // V19
                                                                    {1, 2}   // V20
                                                                }}};

const Vector<Matrix<float>> neutralJointTranslations = {{
                                                            // DNA-0
                                                            // J0    J1    J2
                                                            {1.0f, 0.484f, 0.091f},  // Xs
                                                            {1.0f, 0.502f, 0.215f},  // Ys
                                                            {1.0f, 0.513f, 0.293f}   // Zs
                                                        },
                                                        {
                                                            // DNA-1
                                                            // J0    J1    J2
                                                            {0.6f, 0.989f, 0.307f},  // Xs
                                                            {0.6f, 1.001f, 0.173f},  // Ys
                                                            {0.6f, 1.01f, 0.126f}    // Zs
                                                        },
                                                        {
                                                            // DNA arch
                                                            // J0    J1    J2
                                                            {0.8f, 0.734f, 0.196f},  // Xs
                                                            {0.8f, 0.752f, 0.2f},    // Ys
                                                            {0.8f, 0.763f, 0.203f}   // Zs
                                                        },
                                                        {
                                                            // DNA expected
                                                            // J0    J1     J2
                                                            {0.8f, 0.7550f, 0.1528f},  // Xs
                                                            {0.8f, 0.7715f, 0.1713f},  // Ys
                                                            {0.8f, 0.7833f, 0.1925f},  // Zs
                                                        }};

const Vector<Matrix<float>> neutralJointRotations = {{
                                                         // DNA-0
                                                         // J0    J1    J2
                                                         {10.0f, 5.0f, 10.0f},  // Xs
                                                         {10.0f, 5.0f, 20.0f},  // Ys
                                                         {10.0f, 5.0f, 30.0f}   // Zs
                                                     },
                                                     {
                                                         // DNA-1
                                                         // J0    J1    J2
                                                         {6.0f, 10.0f, 30.0f},  // Xs
                                                         {6.0f, 10.0f, 20.0f},  // Ys
                                                         {6.0f, 10.0f, 10.0f}   // Zs
                                                     },
                                                     {
                                                         // DNA arch
                                                         // J0    J1    J2
                                                         {8.0f, 7.5f, 20.0f},  // Xs
                                                         {8.0f, 7.5f, 20.0f},  // Ys
                                                         {8.0f, 7.5f, 20.0f}   // Zs
                                                     },
                                                     {
                                                         // DNA expected
                                                         // J0    J1     J2
                                                         {8.0f, 7.5464f, 17.7917f},  // Xs
                                                         {8.0f, 7.8324f, 19.7221f},  // Ys
                                                         {8.0f, 7.5475f, 21.1838f},  // Zs
                                                     }};

const Vector<Matrix<std::uint16_t>> jointGroupJointIndices = {{
                                                                  // DNA-0
                                                                  {0, 1},  // JointGroup-0
                                                                  {2}      // JointGroup-1
                                                              },
                                                              {
                                                                  // DNA-1
                                                                  {0, 1},  // JointGroup-0
                                                                  {2}      // JointGroup-1
                                                              },
                                                              {
                                                                  // DNA arch
                                                                  {0, 1},  // JointGroup-0
                                                                  {2}      // JointGroup-1
                                                              },
                                                              {
                                                                  // DNA expected
                                                                  {0, 1},  // JointGroup-0
                                                                  {2}      // JointGroup-1
                                                              }};

const Vector<Matrix<std::uint16_t>> jointGroupOutputIndices = {{
                                                                   // DNA-0
                                                                   {1, 3, 7},    // JointGroup-0
                                                                   {19, 20, 21}  // JointGroup-1
                                                               },
                                                               {
                                                                   // DNA-1
                                                                   {0, 1, 2, 6, 11, 12, 13},  // JointGroup-0
                                                                   {21, 22, 23}               // JointGroup-1
                                                               },
                                                               {
                                                                   // DNA arch
                                                                   {1, 3, 7, 10, 12},    // JointGroup-0
                                                                   {19, 20, 21, 22, 23}  // JointGroup-1
                                                               },
                                                               {
                                                                   // DNA expected
                                                                   {0, 1, 2, 3, 6, 7, 10, 11, 12, 13},  // JointGroup-0
                                                                   {19, 20, 21, 22, 23}                 // JointGroup-1
                                                               },
                                                               {
                                                                   // Combined DNA0 DNA1
                                                                   {0, 1, 2, 3, 6, 7, 11, 12, 13},  // JointGroup-0
                                                                   {19, 20, 21, 22, 23}             // JointGroup-1
                                                               }};

const Matrix<std::uint16_t> jointGroupInputIndices = {
    {1, 2},  // JointGroup-0
    {4, 5}   // JointGroup-1
};

const Vector<Matrix<float>> jointGroupValues = {{// DNA-0
                                                 {
                                                     // Joint group-0
                                                     0.1f,
                                                     0.1f,  // O1
                                                     0.3f,
                                                     0.3f,  // O3
                                                     0.7f,
                                                     0.7f  // O7
                                                 },
                                                 {
                                                     // Joint group-1
                                                     0.1f,
                                                     0.1f,  // O19
                                                     0.5f,
                                                     0.5f,  // O20
                                                     1.0f,
                                                     1.0f  // O21
                                                 }},
                                                {// DNA-1
                                                 {
                                                     // Joint group-0
                                                     0.05f,
                                                     0.05f,  // O0
                                                     0.1f,
                                                     0.1f,  // O1
                                                     0.2f,
                                                     0.2f,  // O2
                                                     0.6f,
                                                     0.6f,  // O6
                                                     1.1f,
                                                     1.1f,  // O11
                                                     1.2f,
                                                     1.2f,  // O12
                                                     1.3f,
                                                     1.3f  // O13
                                                 },
                                                 {
                                                     // Joint group-1
                                                     0.1f,
                                                     0.1f,  // O21
                                                     0.2f,
                                                     0.2f,  // O22
                                                     0.3f,
                                                     0.3f  // O23
                                                 }},
                                                {// DNA arch
                                                 {
                                                     // Joint group-0
                                                     0.05f,
                                                     0.05f,  // O1
                                                     0.15f,
                                                     0.15f,  // O3
                                                     0.6f,
                                                     0.6f,  // O7
                                                     0.5f,
                                                     0.5f,  // O10
                                                     0.1f,
                                                     0.1f  // O12
                                                 },
                                                 {
                                                     // Joint group-1
                                                     0.05f,
                                                     0.05f,  // O19
                                                     0.25f,
                                                     0.25f,  // O20
                                                     0.55f,
                                                     0.55f,  // O21
                                                     0.1f,
                                                     0.1f,  // O22
                                                     0.15f,
                                                     0.15f  // O23
                                                 }},
                                                {// DNA expected
                                                 {
                                                     // Joint group-0
                                                     0.0f,  0.0f,   // O0
                                                     0.05f, 0.05f,  // O1
                                                     0.0f,  0.0f,   // O2
                                                     0.15f, 0.15f,  // O3
                                                     0.0f,  0.0f,   // O6
                                                     0.6f,  0.6f,   // O7
                                                     -0.3f, -0.3f,  // O10
                                                     1.1f,  1.1f,   // O11
                                                     1.14f, 1.14f,  // O12
                                                     1.3f,  1.3f,   // O13
                                                 },
                                                 {
                                                     // Joint group-1
                                                     0.058f,
                                                     0.058f,  // O19
                                                     0.29f,
                                                     0.29f,  // O20
                                                     0.622f,
                                                     0.622f,  // O21
                                                     0.084f,
                                                     0.084f,  // O22
                                                     0.126f,
                                                     0.126f  // O23
                                                 }}};

const Vector<AlignedVector<float>> accustomedArchetypeJointGroupValues = {{
                                                                              // Joint group-0
                                                                              0.0f,  0.0f,   // O0
                                                                              0.05f, 0.05f,  // O1
                                                                              0.0f,  0.0f,   // O2
                                                                              0.15f, 0.15f,  // O3
                                                                              0.0f,  0.0f,   // O6
                                                                              0.6f,  0.6f,   // O7
                                                                              0.5f,  0.5f,   // O10
                                                                              0.0f,  0.0f,   // O11
                                                                              0.1f,  0.1f,   // O12
                                                                              0.0f,  0.0f,   // O13
                                                                          },
                                                                          {
                                                                              // Joint group-1
                                                                              0.05f,
                                                                              0.05f,  // O19
                                                                              0.25f,
                                                                              0.25f,  // O20
                                                                              0.55f,
                                                                              0.55f,  // O21
                                                                              0.1f,
                                                                              0.1f,  // O22
                                                                              0.15f,
                                                                              0.15f  // O23
                                                                          }};

const Vector<Matrix<std::uint16_t>> jointGroupLODs = {{
                                                          // DNA0
                                                          {3, 3, 3},  // Joint group-0
                                                          {3, 3, 3}   // Joint group-1
                                                      },
                                                      {
                                                          // DNA1
                                                          {7, 4, 4},  // Joint group-0
                                                          {3, 3, 3}   // Joint group-1
                                                      },
                                                      {
                                                          // DNA arch
                                                          {5, 3, 3},  // Joint group-0
                                                          {5, 5, 5}   // Joint group-1
                                                      },
                                                      {
                                                          // DNA expected
                                                          {10, 6, 6},  // Joint group-0
                                                          {5, 5, 5}    // Joint group-1
                                                      },
                                                      {
                                                          // Combined DNA0 DNA1
                                                          {9, 6, 6},  // JointGroup-0
                                                          {5, 5, 5}   // JointGroup-1
                                                      }};

const Vector<Matrix<std::uint32_t>> blendShapeTargetVertexIndices = {
    {
        // DNA-0
        {0, 2, 3, 5, 6, 8, 9, 11, 12, 14, 15, 17, 18, 20},                      // BlendShape-0
        {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 20}  // BlendShape-1
    },
    {
        // DNA-1
        {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20},  // BlendShape-0
        {0, 2, 3, 5, 6, 8, 9, 11, 12, 14, 15, 17, 18, 19, 20}                        // BlendShape-1
    },
    {
        // DNA arch
        {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20},  // BlendShape-0
        {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 20}       // BlendShape-1
    },
    {
        // DNA expected
        {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20},  // BlendShape-0
        {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20}   // BlendShape-1
    }};

const Matrix<Matrix<float>> blendShapeTargetDeltas = {
    {// DNA-0
     {
         // BlendShape-0
         // V0     V2     V3    V5    V6    V8    V9     V11    V12    V14    V15   V17    V18   V20
         {0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f},  // Xs
         {0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f},  // Ys
         {0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f, 0.73f, 0.1f}   // Zs
     },
     {
         // BlendShape-1
         // V0     V1     V2     V3      V4     V5     V6     V7    V8      V9    V10    V11    V12    V13    V14    V15    V16
         // V17    V18     V20
         {-0.73f, 1.0f,  -0.1f,  -0.73f, 1.0f,  -0.1f,  -0.73f, 1.0f,  -0.1f,  -0.73f,
          1.0f,   -0.1f, -0.73f, 1.0f,   -0.1f, -0.73f, 1.0f,   -0.1f, -0.73f, -0.1f},  // Xs
         {-0.73f, 1.0f,  -0.1f,  -0.73f, 1.0f,  -0.1f,  -0.73f, 1.0f,  -0.1f,  -0.73f,
          1.0f,   -0.1f, -0.73f, 1.0f,   -0.1f, -0.73f, 1.0f,   -0.1f, -0.73f, -0.1f},  // Ys
         {-0.73f, 1.0f,  -0.1f,  -0.73f, 1.0f,  -0.1f,  -0.73f, 1.0f,  -0.1f,  -0.73f,
          1.0f,   -0.1f, -0.73f, 1.0f,   -0.1f, -0.73f, 1.0f,   -0.1f, -0.73f, -0.1f}  // Zs
     }},
    {// DNA-1
     {
         // BlendShape-0
         // V0     V1     V2     V3     V4     V5     V6     V7     V8     V9     V10    V11     V12    V13    V14     V15   V16
         // V17     V18   V19    V20
         {
             -0.57f, 1.0f,   -0.5f, -0.57f, 1.0f,   -0.5f, -0.57f, 1.0f,   -0.5f, -0.57f, 1.0f,
             -0.5f,  -0.57f, 1.0f,  -0.5f,  -0.57f, 1.0f,  -0.5f,  -0.57f, 1.0f,  -0.5f,
         },  // Xs
         {
             -0.57f, 1.0f,   -0.5f, -0.57f, 1.0f,   -0.5f, -0.57f, 1.0f,   -0.5f, -0.57f, 1.0f,
             -0.5f,  -0.57f, 1.0f,  -0.5f,  -0.57f, 1.0f,  -0.5f,  -0.57f, 1.0f,  -0.5f,
         },  // Ys
         {
             -0.57f, 1.0f,   -0.5f, -0.57f, 1.0f,   -0.5f, -0.57f, 1.0f,   -0.5f, -0.57f, 1.0f,
             -0.5f,  -0.57f, 1.0f,  -0.5f,  -0.57f, 1.0f,  -0.5f,  -0.57f, 1.0f,  -0.5f,
         }  // Zs

     },
     {
         // BlendShape-1
         // V0     V2     V3    V5    V6    V8    V9     V11    V12    V14    V15   V17    V18   v19   V20
         {0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.5f},  // Xs
         {0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.5f},  // Ys
         {0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.57f, 0.5f, 0.5f}   // Zs
     }},
    {// DNA arch
     {
         // BlendShape-0
         // V0     V1     V2    V3    V4      V5     V6    V7     V8    V9     V10    V11   V12    V13    V14   V15   V16    V17
         // V18   V19    V20
         {0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,
          -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f},  // Xs
         {0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,
          -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f},  // Ys
         {0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,
          -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f}  // Zs
     },
     {
         // BlendShape-1
         // V0     V1      V2    V3      V4   V5     V6     V7    V8    V9     V10    V11   V12    V13    V14  V15     V16   V17
         // V18    V20
         {-0.08f, 0.5f, 0.2f,   -0.08f, 0.5f, 0.2f,   -0.08f, 0.5f, 0.2f,   -0.08f,
          0.5f,   0.2f, -0.08f, 0.5f,   0.2f, -0.08f, 0.5f,   0.2f, -0.08f, 0.2f},  // Xs
         {-0.08f, 0.5f, 0.2f,   -0.08f, 0.5f, 0.2f,   -0.08f, 0.5f, 0.2f,   -0.08f,
          0.5f,   0.2f, -0.08f, 0.5f,   0.2f, -0.08f, 0.5f,   0.2f, -0.08f, 0.2f},  // Ys
         {-0.08f, 0.5f, 0.2f,   -0.08f, 0.5f, 0.2f,   -0.08f, 0.5f, 0.2f,   -0.08f,
          0.5f,   0.2f, -0.08f, 0.5f,   0.2f, -0.08f, 0.5f,   0.2f, -0.08f, 0.2f}  // Zs
     }},
    {// DNA expected
     {
         // BlendShape-0
         // V0     V1      V2    V3       V4   V5     V6    V7       V8    V9      V10   V11   V12     V13    V14    V15   V 16
         // V17    V18    V19    V20
         {0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,
          -0.08f, 0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,  -0.08f},  // Xs
         {0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,
          -0.08f, 0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,  -0.08f},  // Ys
         {0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,
          -0.08f, 0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,  -0.08f, 0.028f, 0.62f,  -0.08f}  // Zs
     },
     {
         // BlendShape-1
         // V0        V1     V2    V3        V4     V5     V6       V7    V8      V9      V10     V11   V12      V13    V14    V15
         // V16    V17    V18   , v19   V20
         {-0.028f, 0.38f,   0.08f, -0.028f, 0.38f,   0.08f, -0.028f, 0.38f,   0.08f, -0.028f, 0.38f,
          0.08f,   -0.028f, 0.38f, 0.08f,   -0.028f, 0.38f, 0.08f,   -0.028f, 0.30f, 0.08f},  // Xs
         {-0.028f, 0.38f,   0.08f, -0.028f, 0.38f,   0.08f, -0.028f, 0.38f,   0.08f, -0.028f, 0.38f,
          0.08f,   -0.028f, 0.38f, 0.08f,   -0.028f, 0.38f, 0.08f,   -0.028f, 0.30f, 0.08f},  // Ys
         {-0.028f, 0.38f,   0.08f, -0.028f, 0.38f,   0.08f, -0.028f, 0.38f,   0.08f, -0.028f, 0.38f,
          0.08f,   -0.028f, 0.38f, 0.08f,   -0.028f, 0.38f, 0.08f,   -0.028f, 0.30f, 0.08f}  // Zs
     }}};

const Vector<Matrix<XYZBlock<16u>>> expectedNeutralMeshPoolValues = {
    { // Mesh 0
     {// Pool0
      {
          // DNA0
          {0.5f, 0.0f, -0.5f, 0.5f, 0.0f, -0.5f, 0.5f, 0.0f, -0.5f, 0.5f, 0.0f, -0.5f, 0.5f, 0.0f, -0.5f, 0.5f},  // Xs
          {0.5f, 0.0f, 0.0f, 0.5f, 0.0f, 0.0f, 0.5f, 0.0f, 0.0f, 0.5f, 0.0f, 0.0f, 0.5f, 0.0f, 0.0f, 0.5f},       // Ys
          {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}        // Zs
      },
      {
          // DNA1
          {-0.5f, 0.0f, 0.5f, -0.5f, 0.0f, 0.5f, -0.5f, 0.0f, 0.5f, -0.5f, 0.0f, 0.5f, -0.5f, 0.0f, 0.5f, -0.5f},  // Xs
          {-0.5f, 0.0f, 0.0f, -0.5f, 0.0f, 0.0f, -0.5f, 0.0f, 0.0f, -0.5f, 0.0f, 0.0f, -0.5f, 0.0f, 0.0f, -0.5f},  // Ys
          {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}         // Zs
      }},
     {// Pool1
      {
          // DNA0
          {0.0f, -0.5f, 0.5f, 0.0f, -0.5f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Xs
          {0.0f, 0.0f, 0.5f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},    // Ys
          {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}     // Zs
      },
      {
          // DNA1
          {0.0f, 0.5f, -0.5f, 0.0f, 0.5f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Xs
          {0.0f, 0.0f, -0.5f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Ys
          {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}    // Zs
      }}},
    { // Mesh 1
     {// Pool0
      {
          // DNA0
          {0.5f, 0.0f, -0.5f, 0.5f, 0.0f, -0.5f, 0.5f, 0.0f, -0.5f, 0.5f, 0.0f, -0.5f, 0.5f, 0.0f, -0.5f, 0.5f},  // Xs
          {0.5f, 0.0f, 0.0f, 0.5f, 0.0f, 0.0f, 0.5f, 0.0f, 0.0f, 0.5f, 0.0f, 0.0f, 0.5f, 0.0f, 0.0f, 0.5f},       // Ys
          {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}        // Zs
      },
      {
          // DNA1
          {-0.5f, 0.0f, 0.5f, -0.5f, 0.0f, 0.5f, -0.5f, 0.0f, 0.5f, -0.5f, 0.0f, 0.5f, -0.5f, 0.0f, 0.5f, -0.5f},  // Xs
          {-0.5f, 0.0f, 0.0f, -0.5f, 0.0f, 0.0f, -0.5f, 0.0f, 0.0f, -0.5f, 0.0f, 0.0f, -0.5f, 0.0f, 0.0f, -0.5f},  // Ys
          {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}         // Zs
      }},
     {// Pool1
      {
          // DNA0
          {0.0f, -0.5f, 0.5f, 0.0f, -0.5f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Xs
          {0.0f, 0.0f, 0.5f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},    // Ys
          {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}     // Zs
      },
      {
          // DNA1
          {0.0f, 0.5f, -0.5f, 0.0f, 0.5f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Xs
          {0.0f, 0.0f, -0.5f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Ys
          {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}    // Zs
      }}}};

const Matrix<XYZBlock<16u>> expectedNeutralJointPoolTranslations = {
    {// Pool0
     {
         // DNA0
         {0.2f, -0.0491f, -0.122f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Xs
         {0.2f, -0.0503f, -0.082f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Ys
         {0.2f, -0.0504f, 0.0627f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}   // Zs
     },
     {
         // DNA1
         {-0.2f, 0.0509f, 0.1351f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Xs
         {-0.2f, 0.0498f, 0.0731f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Ys
         {-0.2f, 0.0499f, -0.0584f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // Zs
     }}};

const Matrix<XYZBlock<16u>> expectedNeutralJointPoolRotations = {
    {// Pool0
     {
         // DNA0
         {0.0349f, -0.0117f, -0.1781f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Xs
         {0.0349f, -0.0051f, -0.0617f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Ys
         {0.0349f, -0.0128f, 0.1510f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}    // Zs
     },
     {
         // DNA1
         {-0.0349f, 0.0081f, 0.1642f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Xs
         {-0.0349f, 0.0087f, 0.0678f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // Ys
         {-0.0349f, 0.0094f, -0.1592f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // Zs
     }}};

const Vector<Vector<std::size_t>> expectedBlendShapePoolBucketOffsets = {{// Mesh 0
                                                                          {0u, 6u, 12u}},
                                                                         {// Mesh 1
                                                                          {12u, 18u, 24u}}};

const Vector<std::uint32_t> expectedBlendShapePoolBucketVertexIndices = {0u, 4u, 8u, 12u, 16u, 20u, 0u, 4u, 8u, 12u, 16u, 20u,
                                                                         0u, 4u, 8u, 12u, 16u, 20u, 0u, 4u, 8u, 12u, 16u, 20u};

const Vector<std::size_t> expectedBlendShapePoolBucketDNABlockOffsets{
    0u, 2u, 4u, 6u, 8u, 10u, 12u, 14u, 16u, 18u, 20u, 22u, 24u, 26u, 28u, 30u, 32u, 34u, 36u, 38u, 40u, 42u, 44u, 46u, 48u};

const Vector<XYZBlock<4u>> expectedBlendShapePoolDNADeltas = {
    // Mesh 0
    // BlendShape 0
    // Bucket 0
    {// DNA 0
     // v0   v1     v2    v3
     {0.73f, 0.0f, 0.1f, 0.73f},
     {0.73f, 0.0f, 0.1f, 0.73f},
     {0.73f, 0.0f, 0.1f, 0.73f}},
    {// DNA 1
     // v0      v1   v2      v3
     {-0.57f, 1.0f, -0.5f, -0.57f},
     {-0.57f, 1.0f, -0.5f, -0.57f},
     {-0.57f, 1.0f, -0.5f, -0.57f}},
    // Bucket 1
    {// DNA 0
     // v4   v5     v6    v7
     {0.0f, 0.1f, 0.73f, 0.0f},
     {0.0f, 0.1f, 0.73f, 0.0f},
     {0.0f, 0.1f, 0.73f, 0.0f}},
    {// DNA 1
     // v4    v5     v6     v7
     {1.0f, -0.5f, -0.57f, 1.0f},
     {1.0f, -0.5f, -0.57f, 1.0f},
     {1.0f, -0.5f, -0.57f, 1.0f}},
    // Bucket 2
    {// DNA 0
     // v8   v9    v10   v11
     {0.1f, 0.73f, 0.0f, 0.1f},
     {0.1f, 0.73f, 0.0f, 0.1f},
     {0.1f, 0.73f, 0.0f, 0.1f}},
    {// DNA 1
     // v8   v9     v10    v11
     {-0.5f, -0.57f, 1.0f, -0.5f},
     {-0.5f, -0.57f, 1.0f, -0.5f},
     {-0.5f, -0.57f, 1.0f, -0.5f}},
    // Bucket 3
    {// DNA 0
     // v12   v13   v14   v15
     {0.73f, 0.0f, 0.1f, 0.73f},
     {0.73f, 0.0f, 0.1f, 0.73f},
     {0.73f, 0.0f, 0.1f, 0.73f}},
    {// DNA 1
     // v12   v13    v14    v15
     {-0.57f, 1.0f, -0.5f, -0.57f},
     {-0.57f, 1.0f, -0.5f, -0.57f},
     {-0.57f, 1.0f, -0.5f, -0.57f}},
    // Bucket 4
    {// DNA 0
     // v16   v17   v18   v19
     {0.0f, 0.1f, 0.73f, 0.0f},
     {0.0f, 0.1f, 0.73f, 0.0f},
     {0.0f, 0.1f, 0.73f, 0.0f}},
    {// DNA 1
     // v16   v17    v18    v19
     {1.0f, -0.5f, -0.57f, 1.0f},
     {1.0f, -0.5f, -0.57f, 1.0f},
     {1.0f, -0.5f, -0.57f, 1.0f}},
    // Bucket 5
    {// DNA 0
     // v20   p   p      p
     {0.1f, 0.0f, 0.0f, 0.0f},
     {0.1f, 0.0f, 0.0f, 0.0f},
     {0.1f, 0.0f, 0.0f, 0.0f}},
    {// DNA 1
     // v20   p      p     p
     {-0.5f, 0.0f, 0.0f, 0.0f},
     {-0.5f, 0.0f, 0.0f, 0.0f},
     {-0.5f, 0.0f, 0.0f, 0.0f}},
    // BlendShape 1
    // Bucket 0
    {// DNA 0
     // v0   v1     v2        v3
     {-0.73f, 1.0f, -0.1f, -0.73f},
     {-0.73f, 1.0f, -0.1f, -0.73f},
     {-0.73f, 1.0f, -0.1f, -0.73f}},
    {// DNA 1
     // v0      v1   v2     v3
     {0.57f, 0.0f, 0.5f, 0.57f},
     {0.57f, 0.0f, 0.5f, 0.57f},
     {0.57f, 0.0f, 0.5f, 0.57f}},
    // Bucket 1
    {// DNA 0
     // v4   v5     v6     v7
     {1.0f, -0.1f, -0.73f, 1.0f},
     {1.0f, -0.1f, -0.73f, 1.0f},
     {1.0f, -0.1f, -0.73f, 1.0f}},
    {// DNA 1
     // v4    v5     v6    v7
     {0.0f, 0.5f, 0.57f, 0.0f},
     {0.0f, 0.5f, 0.57f, 0.0f},
     {0.0f, 0.5f, 0.57f, 0.0f}},
    // Bucket 2
    {// DNA 0
     // v8   v9    v10   v11
     {-0.1f, -0.73f, 1.0f, -0.1f},
     {-0.1f, -0.73f, 1.0f, -0.1f},
     {-0.1f, -0.73f, 1.0f, -0.1f}},
    {// DNA 1
     // v8   v9     v10    v11
     {0.5f, 0.57f, 0.0f, 0.5f},
     {0.5f, 0.57f, 0.0f, 0.5f},
     {0.5f, 0.57f, 0.0f, 0.5f}},
    // Bucket 3
    {// DNA 0
     // v12   v13   v14     v15
     {-0.73f, 1.0f, -0.1f, -0.73f},
     {-0.73f, 1.0f, -0.1f, -0.73f},
     {-0.73f, 1.0f, -0.1f, -0.73f}},
    {// DNA 1
     // v12   v13   v14   v15
     {0.57f, 0.0f, 0.5f, 0.57f},
     {0.57f, 0.0f, 0.5f, 0.57f},
     {0.57f, 0.0f, 0.5f, 0.57f}},
    // Bucket 4
    {// DNA 0
     // v16   v17   v18   v19
     {1.0f, -0.1f, -0.73f, 0.0f},
     {1.0f, -0.1f, -0.73f, 0.0f},
     {1.0f, -0.1f, -0.73f, 0.0f}},
    {// DNA 1
     // v16   v17    v18    v19
     {0.0f, 0.5f, 0.57f, 0.5f},
     {0.0f, 0.5f, 0.57f, 0.5f},
     {0.0f, 0.5f, 0.57f, 0.5f}},
    // Bucket 5
    {// DNA 0
     // v20   p   p      p
     {-0.1f, 0.0f, 0.0f, 0.0f},
     {-0.1f, 0.0f, 0.0f, 0.0f},
     {-0.1f, 0.0f, 0.0f, 0.0f}},
    {// DNA 1
     // v20   p      p     p
     {0.5f, 0.0f, 0.0f, 0.0f},
     {0.5f, 0.0f, 0.0f, 0.0f},
     {0.5f, 0.0f, 0.0f, 0.0f}},
    // Mesh 1
    // BlendShape 0
    // Bucket 0
    {// DNA 0
     // v0   v1     v2    v3
     {0.73f, 0.0f, 0.1f, 0.73f},
     {0.73f, 0.0f, 0.1f, 0.73f},
     {0.73f, 0.0f, 0.1f, 0.73f}},
    {// DNA 1
     // v0      v1   v2      v3
     {-0.57f, 1.0f, -0.5f, -0.57f},
     {-0.57f, 1.0f, -0.5f, -0.57f},
     {-0.57f, 1.0f, -0.5f, -0.57f}},
    // Bucket 1
    {// DNA 0
     // v4   v5     v6    v7
     {0.0f, 0.1f, 0.73f, 0.0f},
     {0.0f, 0.1f, 0.73f, 0.0f},
     {0.0f, 0.1f, 0.73f, 0.0f}},
    {// DNA 1
     // v4    v5     v6     v7
     {1.0f, -0.5f, -0.57f, 1.0f},
     {1.0f, -0.5f, -0.57f, 1.0f},
     {1.0f, -0.5f, -0.57f, 1.0f}},
    // Bucket 2
    {// DNA 0
     // v8   v9    v10   v11
     {0.1f, 0.73f, 0.0f, 0.1f},
     {0.1f, 0.73f, 0.0f, 0.1f},
     {0.1f, 0.73f, 0.0f, 0.1f}},
    {// DNA 1
     // v8   v9     v10    v11
     {-0.5f, -0.57f, 1.0f, -0.5f},
     {-0.5f, -0.57f, 1.0f, -0.5f},
     {-0.5f, -0.57f, 1.0f, -0.5f}},
    // Bucket 3
    {// DNA 0
     // v12   v13   v14   v15
     {0.73f, 0.0f, 0.1f, 0.73f},
     {0.73f, 0.0f, 0.1f, 0.73f},
     {0.73f, 0.0f, 0.1f, 0.73f}},
    {// DNA 1
     // v12   v13    v14    v15
     {-0.57f, 1.0f, -0.5f, -0.57f},
     {-0.57f, 1.0f, -0.5f, -0.57f},
     {-0.57f, 1.0f, -0.5f, -0.57f}},
    // Bucket 4
    {// DNA 0
     // v16   v17   v18   v19
     {0.0f, 0.1f, 0.73f, 0.0f},
     {0.0f, 0.1f, 0.73f, 0.0f},
     {0.0f, 0.1f, 0.73f, 0.0f}},
    {// DNA 1
     // v16   v17    v18    v19
     {1.0f, -0.5f, -0.57f, 1.0f},
     {1.0f, -0.5f, -0.57f, 1.0f},
     {1.0f, -0.5f, -0.57f, 1.0f}},
    // Bucket 5
    {// DNA 0
     // v20   p   p      p
     {0.1f, 0.0f, 0.0f, 0.0f},
     {0.1f, 0.0f, 0.0f, 0.0f},
     {0.1f, 0.0f, 0.0f, 0.0f}},
    {// DNA 1
     // v20   p      p     p
     {-0.5f, 0.0f, 0.0f, 0.0f},
     {-0.5f, 0.0f, 0.0f, 0.0f},
     {-0.5f, 0.0f, 0.0f, 0.0f}},
    // BlendShape 1
    // Bucket 0
    {// DNA 0
     // v0   v1     v2        v3
     {-0.73f, 1.0f, -0.1f, -0.73f},
     {-0.73f, 1.0f, -0.1f, -0.73f},
     {-0.73f, 1.0f, -0.1f, -0.73f}},
    {// DNA 1
     // v0      v1   v2     v3
     {0.57f, 0.0f, 0.5f, 0.57f},
     {0.57f, 0.0f, 0.5f, 0.57f},
     {0.57f, 0.0f, 0.5f, 0.57f}},
    // Bucket 1
    {// DNA 0
     // v4   v5     v6     v7
     {1.0f, -0.1f, -0.73f, 1.0f},
     {1.0f, -0.1f, -0.73f, 1.0f},
     {1.0f, -0.1f, -0.73f, 1.0f}},
    {// DNA 1
     // v4    v5     v6    v7
     {0.0f, 0.5f, 0.57f, 0.0f},
     {0.0f, 0.5f, 0.57f, 0.0f},
     {0.0f, 0.5f, 0.57f, 0.0f}},
    // Bucket 2
    {// DNA 0
     // v8   v9    v10   v11
     {-0.1f, -0.73f, 1.0f, -0.1f},
     {-0.1f, -0.73f, 1.0f, -0.1f},
     {-0.1f, -0.73f, 1.0f, -0.1f}},
    {// DNA 1
     // v8   v9     v10    v11
     {0.5f, 0.57f, 0.0f, 0.5f},
     {0.5f, 0.57f, 0.0f, 0.5f},
     {0.5f, 0.57f, 0.0f, 0.5f}},
    // Bucket 3
    {// DNA 0
     // v12   v13   v14     v15
     {-0.73f, 1.0f, -0.1f, -0.73f},
     {-0.73f, 1.0f, -0.1f, -0.73f},
     {-0.73f, 1.0f, -0.1f, -0.73f}},
    {// DNA 1
     // v12   v13   v14   v15
     {0.57f, 0.0f, 0.5f, 0.57f},
     {0.57f, 0.0f, 0.5f, 0.57f},
     {0.57f, 0.0f, 0.5f, 0.57f}},
    // Bucket 4
    {// DNA 0
     // v16   v17   v18   v19
     {1.0f, -0.1f, -0.73f, 0.0f},
     {1.0f, -0.1f, -0.73f, 0.0f},
     {1.0f, -0.1f, -0.73f, 0.0f}},
    {// DNA 1
     // v16   v17    v18    v19
     {0.0f, 0.5f, 0.57f, 0.5f},
     {0.0f, 0.5f, 0.57f, 0.5f},
     {0.0f, 0.5f, 0.57f, 0.5f}},
    // Bucket 5
    {// DNA 0
     // v20   p   p      p
     {-0.1f, 0.0f, 0.0f, 0.0f},
     {-0.1f, 0.0f, 0.0f, 0.0f},
     {-0.1f, 0.0f, 0.0f, 0.0f}},
    {// DNA 1
     // v20   p      p     p
     {0.5f, 0.0f, 0.0f, 0.0f},
     {0.5f, 0.0f, 0.0f, 0.0f},
     {0.5f, 0.0f, 0.0f, 0.0f}}};

const Vector<XYZBlock<4u>> expectedBlendShapePoolArchDeltas = {
    // Mesh 0
    // BlendShape 0
    {// Bucket 0
     {0.08f, 0.5f, -0.2f, 0.08f},
     {0.08f, 0.5f, -0.2f, 0.08f},
     {0.08f, 0.5f, -0.2f, 0.08f}},
    {// Bucket 1
     {0.5f, -0.2f, 0.08f, 0.5f},
     {0.5f, -0.2f, 0.08f, 0.5f},
     {0.5f, -0.2f, 0.08f, 0.5f}},
    {// Bucket 2
     {-0.2f, 0.08f, 0.5f, -0.2f},
     {-0.2f, 0.08f, 0.5f, -0.2f},
     {-0.2f, 0.08f, 0.5f, -0.2f}},
    {// Bucket 3
     {0.08f, 0.5f, -0.2f, 0.08f},
     {0.08f, 0.5f, -0.2f, 0.08f},
     {0.08f, 0.5f, -0.2f, 0.08f}},
    {// Bucket 4
     {0.5f, -0.2f, 0.08f, 0.5f},
     {0.5f, -0.2f, 0.08f, 0.5f},
     {0.5f, -0.2f, 0.08f, 0.5f}},
    {// Bucket 5
     {-0.2f, 0.0f, 0.0f, 0.0f},
     {-0.2f, 0.0f, 0.0f, 0.0f},
     {-0.2f, 0.0f, 0.0f, 0.0f}},
    // BlendShape 1
    {// Bucket 0
     {-0.08f, 0.5f, 0.2f, -0.08f},
     {-0.08f, 0.5f, 0.2f, -0.08f},
     {-0.08f, 0.5f, 0.2f, -0.08f}},
    {// Bucket 1
     {0.5f, 0.2f, -0.08f, 0.5f},
     {0.5f, 0.2f, -0.08f, 0.5f},
     {0.5f, 0.2f, -0.08f, 0.5f}},
    {// Bucket 2
     {0.2f, -0.08f, 0.5f, 0.2f},
     {0.2f, -0.08f, 0.5f, 0.2f},
     {0.2f, -0.08f, 0.5f, 0.2f}},
    {// Bucket 3
     {-0.08f, 0.5f, 0.2f, -0.08f},
     {-0.08f, 0.5f, 0.2f, -0.08f},
     {-0.08f, 0.5f, 0.2f, -0.08f}},
    {// Bucket 4
     {0.5f, 0.2f, -0.08f, 0.0f},
     {0.5f, 0.2f, -0.08f, 0.0f},
     {0.5f, 0.2f, -0.08f, 0.0f}},
    {// Bucket 5
     {0.2f, 0.0f, 0.0f, 0.0f},
     {0.2f, 0.0f, 0.0f, 0.0f},
     {0.2f, 0.0f, 0.0f, 0.0f}},
    // Mesh 1
    // BlendShape 0
    {// Bucket 0
     {0.08f, 0.5f, -0.2f, 0.08f},
     {0.08f, 0.5f, -0.2f, 0.08f},
     {0.08f, 0.5f, -0.2f, 0.08f}},
    {// Bucket 1
     {0.5f, -0.2f, 0.08f, 0.5f},
     {0.5f, -0.2f, 0.08f, 0.5f},
     {0.5f, -0.2f, 0.08f, 0.5f}},
    {// Bucket 2
     {-0.2f, 0.08f, 0.5f, -0.2f},
     {-0.2f, 0.08f, 0.5f, -0.2f},
     {-0.2f, 0.08f, 0.5f, -0.2f}},
    {// Bucket 3
     {0.08f, 0.5f, -0.2f, 0.08f},
     {0.08f, 0.5f, -0.2f, 0.08f},
     {0.08f, 0.5f, -0.2f, 0.08f}},
    {// Bucket 4
     {0.5f, -0.2f, 0.08f, 0.5f},
     {0.5f, -0.2f, 0.08f, 0.5f},
     {0.5f, -0.2f, 0.08f, 0.5f}},
    {// Bucket 5
     {-0.2f, 0.0f, 0.0f, 0.0f},
     {-0.2f, 0.0f, 0.0f, 0.0f},
     {-0.2f, 0.0f, 0.0f, 0.0f}},
    // BlendShape 1
    {// Bucket 0
     {-0.08f, 0.5f, 0.2f, -0.08f},
     {-0.08f, 0.5f, 0.2f, -0.08f},
     {-0.08f, 0.5f, 0.2f, -0.08f}},
    {// Bucket 1
     {0.5f, 0.2f, -0.08f, 0.5f},
     {0.5f, 0.2f, -0.08f, 0.5f},
     {0.5f, 0.2f, -0.08f, 0.5f}},
    {// Bucket 2
     {0.2f, -0.08f, 0.5f, 0.2f},
     {0.2f, -0.08f, 0.5f, 0.2f},
     {0.2f, -0.08f, 0.5f, 0.2f}},
    {// Bucket 3
     {-0.08f, 0.5f, 0.2f, -0.08f},
     {-0.08f, 0.5f, 0.2f, -0.08f},
     {-0.08f, 0.5f, 0.2f, -0.08f}},
    {// Bucket 4
     {0.5f, 0.2f, -0.08f, 0.0f},
     {0.5f, 0.2f, -0.08f, 0.0f},
     {0.5f, 0.2f, -0.08f, 0.0f}},
    {// Bucket 5
     {0.2f, 0.0f, 0.0f, 0.0f},
     {0.2f, 0.0f, 0.0f, 0.0f},
     {0.2f, 0.0f, 0.0f, 0.0f}}};

const Vector<std::uint16_t> expectedBlendShapePoolDNAIndices = {
    // Mesh 0
    // BlendShape 0
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    // BlendShape 1
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    // Mesh 1
    // BlendShape 0
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    // BlendShape 1
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,
    0u,
    1u,

};

const Vector<Vector<Vector<float>>> accustomedArchetypeBlendShapeDeltas = {
    {
        // BlendShape-0
        // V0     V1     V2    V3    V4      V5     V6    V7     V8    V9     V10    V11   V12    V13    V14   V15   V16    V17
        // V18   V19    V20
        {0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,
         -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f},  // Xs
        {0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,
         -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f},  // Ys
        {0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,
         -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f, 0.08f, 0.5f,  -0.2f}  // Zs
    },
    {
        // BlendShape-1
        // V0     V1      V2    V3      V4   V5     V6     V7    V8    V9     V10    V11   V12    V13    V14  V15     V16   V17
        // V18   V19,   V20
        {-0.08f, 0.5f,   0.2f, -0.08f, 0.5f,   0.2f, -0.08f, 0.5f,   0.2f, -0.08f, 0.5f,
         0.2f,   -0.08f, 0.5f, 0.2f,   -0.08f, 0.5f, 0.2f,   -0.08f, 0.0f, 0.2f},  // Xs
        {-0.08f, 0.5f,   0.2f, -0.08f, 0.5f,   0.2f, -0.08f, 0.5f,   0.2f, -0.08f, 0.5f,
         0.2f,   -0.08f, 0.5f, 0.2f,   -0.08f, 0.5f, 0.2f,   -0.08f, 0.0f, 0.2f},  // Ys
        {-0.08f, 0.5f,   0.2f, -0.08f, 0.5f,   0.2f, -0.08f, 0.5f,   0.2f, -0.08f, 0.5f,
         0.2f,   -0.08f, 0.5f, 0.2f,   -0.08f, 0.5f, 0.2f,   -0.08f, 0.0f, 0.2f}  // Zs
    }};

const Vector<Matrix<std::uint32_t>> expectedBlendShapePoolVertexIndices = {
    {
        // Mesh0
        {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20},  // BlendShape-0
        {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20}   // BlendShape-1
    },
    {
        // Mesh1
        {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20},  // BlendShape-0
        {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20}   // BlendShape-1
    }};

const Vector<Matrix<std::uint16_t>> expectedSWPoolJointIndices = {{
                                                                      // Mesh0
                                                                      {0, 2},     // V0
                                                                      {1},        // V1
                                                                      {0, 2},     // V2
                                                                      {0, 1, 2},  // V3
                                                                      {0, 1, 2},  // V4
                                                                      {0, 1, 2},  // V5
                                                                      {0, 2},     // V6
                                                                      {0, 1},     // V7
                                                                      {1, 2},     // V8
                                                                      {0, 2},     // V9
                                                                      {0, 1},     // V10
                                                                      {1, 2},     // V11
                                                                      {0, 2},     // V12
                                                                      {0, 1},     // V13
                                                                      {1, 2},     // V14
                                                                      {0, 2},     // V15
                                                                      {0, 1},     // V16
                                                                      {1, 2},     // V17
                                                                      {0, 2},     // V18
                                                                      {0, 1},     // V19
                                                                      {1, 2}      // V20
                                                                  },
                                                                  {
                                                                      // Mesh1
                                                                      {0, 2},     // V0
                                                                      {1},        // V1
                                                                      {0, 2},     // V2
                                                                      {0, 1, 2},  // V3
                                                                      {0, 1, 2},  // V4
                                                                      {0, 1, 2},  // V5
                                                                      {0, 2},     // V6
                                                                      {0, 1},     // V7
                                                                      {1, 2},     // V8
                                                                      {0, 2},     // V9
                                                                      {0, 1},     // V10
                                                                      {1, 2},     // V11
                                                                      {0, 2},     // V12
                                                                      {0, 1},     // V13
                                                                      {1, 2},     // V14
                                                                      {0, 2},     // V15
                                                                      {0, 1},     // V16
                                                                      {1, 2},     // V17
                                                                      {0, 2},     // V18
                                                                      {0, 1},     // V19
                                                                      {1, 2}      // V20
                                                                  }};

const Matrix<Matrix<VBlock<16u>>> expectedSWPoolWeights = {
    {   // Mesh0
     {  // Pool0
      { // DNA0
       {// JntPos0
        {1.0f, 1.0f, 0.0f, 0.4f, 0.2f, 0.1f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f}},
       {// JntPos1
        {0.0f, 0.0f, 1.0f, 0.6f, 0.0f, 0.9f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f}},
       {// JntPos2
        {0.0f, 0.0f, 0.0f, 0.0f, 0.8f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}}},
      { // DNA1
       {// JntPos0
        {0.0f, 1.0f, 1.0f, 0.0f, 0.2f, 0.0f, 0.4f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f}},
       {// JntPos1
        {1.0f, 0.0f, 0.0f, 0.5f, 0.8f, 0.8f, 0.6f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f}},
       {// JntPos2
        {0.0f, 0.0f, 0.0f, 0.5f, 0.0f, 0.2f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}}}},
     {  // Pool1
      { // DNA0
       {// JntPos0
        {0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}},
       {// JntPos1
        {1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}}},
      { // DNA1
       {// JntPos0
        {1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}},
       {// JntPos1
        {0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}}}}},
    {   // Mesh1
     {  // Pool0
      { // DNA0
       {// JntPos0
        {1.0f, 1.0f, 0.0f, 0.4f, 0.2f, 0.1f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f}},
       {// JntPos1
        {0.0f, 0.0f, 1.0f, 0.6f, 0.0f, 0.9f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f}},
       {// JntPos2
        {0.0f, 0.0f, 0.0f, 0.0f, 0.8f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}}},
      { // DNA1
       {// JntPos0
        {0.0f, 1.0f, 1.0f, 0.0f, 0.2f, 0.0f, 0.4f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f}},
       {// JntPos1
        {1.0f, 0.0f, 0.0f, 0.5f, 0.8f, 0.8f, 0.6f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 1.0f}},
       {// JntPos2
        {0.0f, 0.0f, 0.0f, 0.5f, 0.0f, 0.2f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}}}},
     {  // Pool1
      { // DNA0
       {// JntPos0
        {0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}},
       {// JntPos1
        {1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}}},
      { // DNA1
       {// JntPos0
        {1.0f, 1.0f, 0.0f, 1.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}},
       {// JntPos1
        {0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}}}}}};

extern const Vector<std::uint16_t> expectedSWPoolMaxInfluences{2u};

extern const Matrix<std::uint16_t> expectedJBPoolInputIndices{
    {1, 2},  // JointGroup-0
    {4, 5}   // JointGroup-1
};

extern const Matrix<std::uint16_t> expectedJBPoolOutputIndices{
    {0, 1, 2, 3, 6, 7, 10, 11, 12, 13},  // JointGroup-0
    {19, 20, 21, 22, 23}                 // JointGroup-1
};

extern const Matrix<std::uint16_t> expectedJBPoolLODs{
    {10, 6, 6},  // Joint group-0
    {5, 5, 5}    // Joint group-1
};

extern const Vector<JBJoint> expectedJBPoolBlock{
    {
        // Joint0
        { // values
         {// OutPos0
          {
              // Block0
              {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},   // dna0
              {0.05f, 0.05f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // dna1
          }},
         {// OutPos1
          {
              // Block0
              {0.05f, 0.05f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // dna0
              {0.05f, 0.05f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}   // dna1
          }},
         {// OutPos2
          {
              // Block0
              {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // dna0
              {0.2f, 0.2f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}   // dna1
          }},
         {// OutPos3
          {
              // Block0
              {0.15f, 0.15f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},   // dna0
              {-0.15f, -0.15f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // dna1
          }},
         {
             // OutPos4

         },
         {
             // OutPos5

         },
         {// OutPos6
          {
              // Block0
              {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // dna0
              {0.6f, 0.6f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}   // dna1
          }},
         {// OutPos7
          {
              // Block0
              {0.1f, 0.1f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},   // dna0
              {-0.6f, -0.6f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // dna1
          }},
         {
             // OutPos8

         }

        },
        0u,                  // jointGroup
        {0, 1, 2, 3, 4, 5},  // outIndexTargetPos
        2u                   // vBlockRemainder
    },
    {
        // Joint1
        {// values
         {
             // OutPos0

         },
         {// OutPos1
          {
              // Block0
              {-0.5f, -0.5f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // dna0
              {-0.5f, -0.5f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}   // dna1
          }},
         {// OutPos2
          {
              // Block0
              {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // dna0
              {1.1f, 1.1f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}   // dna1
          }},
         {// OutPos3
          {
              // Block0
              {-0.1f, -0.1f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // dna0
              {1.1f, 1.1f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}     // dna1
          }},
         {// OutPos4
          {
              // Block0
              {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // dna0
              {1.3f, 1.3f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}   // dna1
          }},
         {
             // OutPos5

         },
         {
             // OutPos6

         },
         {
             // OutPos7

         },
         {
             // OutPos8

         }

        },
        0u,                  // jointGroup
        {6, 7, 8, 9, 0, 0},  // outIndexTargetPos
        2u                   // vBlockRemainder
    },
    {
        // Joint1
        {// values
         {
             // OutPos0

         },
         {// OutPos1
          {
              // Block0
              {0.05f, 0.05f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},   // dna0
              {-0.05f, -0.05f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // dna1
          }},
         {// OutPos2
          {
              // Block0
              {0.25f, 0.25f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},   // dna0
              {-0.25f, -0.25f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // dna1
          }},
         {// OutPos3
          {
              // Block0
              {0.45f, 0.45f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},   // dna0
              {-0.45f, -0.45f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}  // dna1
          }},
         {// OutPos4
          {
              // Block0
              {-0.1f, -0.1f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // dna0
              {0.1f, 0.1f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}     // dna1
          }},
         {// OutPos5
          {
              // Block0
              {-0.15f, -0.15f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},  // dna0
              {0.15f, 0.15f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}     // dna1
          }},
         {
             // OutPos6

         },
         {
             // OutPos7

         },
         {
             // OutPos8

         }},
        1u,                        // jointGroup
        {0u, 1u, 2u, 3u, 4u, 0u},  // outIndexTargetPos
        2u                         // vBlockRemainder
    }};

const Vector<float> spliceWeights = {0.6f, 0.5f, 1.0f, 0.1f};

const Vector<Vector3> expectedRawGenesNeutralJointTranslations = {{0.8f, 0.8f, 0.8f},
                                                                  {1.5495f, 1.55f, 1.5497f},
                                                                  {1.7527f, 1.7501f, 1.7454f}};

const Vector<Vector3> expectedRawGenesNeutralJointRotations = {{0.13963f, 0.13963f, 0.13963f},
                                                               {0.29134f, 0.24954f, 0.29119f},
                                                               {0.74603f, 0.46237f, 0.76215f}};

// *INDENT-ON*
}  // namespace canonical

}  // namespace gs4
