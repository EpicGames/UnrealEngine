// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "gstests/Defs.h"

#include "genesplicer/TypeDefs.h"
#include "genesplicer/types/Block.h"
#include "pma/TypeDefs.h"

#include <cstddef>
#include <cstdint>

namespace gs4 {

namespace canonical {

static constexpr std::uint16_t dnaCount = 3;
static constexpr std::uint16_t meshCount = 2;
static constexpr std::uint16_t regionCount = 2;
static constexpr std::uint16_t jointCount = 3;
static constexpr std::uint16_t psdCount = 2;
static constexpr std::uint16_t blendShapeCount = 2;

extern const Matrix<Matrix<float>> neutralMeshes;
extern const Matrix<Matrix<float>> skinWeightsValues;
extern const Matrix<Matrix<std::uint16_t>> skinWeightsJointIndices;
extern const Vector<Matrix<float>> neutralJointTranslations;
extern const Vector<Matrix<float>> neutralJointRotations;
extern const Vector<Matrix<std::uint16_t>> jointGroupLODs;
extern const Vector<Matrix<std::uint16_t>> jointGroupJointIndices;
extern const Vector<Matrix<std::uint16_t>> jointGroupOutputIndices;
extern const Matrix<std::uint16_t> jointGroupInputIndices;
extern const Vector<Matrix<float>> jointGroupValues;
extern const Vector<Matrix<std::uint32_t>> blendShapeTargetVertexIndices;
extern const Matrix<Matrix<float>> blendShapeTargetDeltas;

extern const Vector<Matrix<XYZBlock<16u>>> expectedNeutralMeshPoolValues;
extern const Matrix<XYZBlock<16u>> expectedNeutralJointPoolTranslations;
extern const Matrix<XYZBlock<16u>> expectedNeutralJointPoolRotations;
extern const Vector<Vector<std::size_t>> expectedBlendShapePoolBucketOffsets;
extern const Vector<std::uint32_t> expectedBlendShapePoolBucketVertexIndices;
extern const Vector<std::size_t> expectedBlendShapePoolBucketDNABlockOffsets;
extern const Vector<XYZBlock<4u>> expectedBlendShapePoolArchDeltas;
extern const Vector<XYZBlock<4u>> expectedBlendShapePoolDNADeltas;
extern const Vector<std::uint16_t> expectedBlendShapePoolDNAIndices;

extern const Vector<Matrix<std::uint32_t>> expectedBlendShapePoolVertexIndices;
extern const Vector<Matrix<std::uint16_t>> expectedSWPoolJointIndices;
extern const Matrix<Matrix<VBlock<16u>>> expectedSWPoolWeights;
extern const Vector<std::uint16_t> expectedSWPoolMaxInfluences;
extern const Matrix<std::uint16_t> expectedJBPoolInputIndices;
extern const Matrix<std::uint16_t> expectedJBPoolOutputIndices;
extern const Matrix<std::uint16_t> expectedJBPoolLODs;

extern const Vector<Vector3> expectedRawGenesNeutralJointTranslations;
extern const Vector<Vector3> expectedRawGenesNeutralJointRotations;

struct JBJoint {
    Vector<Matrix<VBlock<16u>>> blockValues;
    std::uint16_t jointGroup;
    std::uint16_t outIndexTargetPos[6];
    std::uint8_t vBlockRemainder;
};
extern const Vector<JBJoint> expectedJBPoolBlock;
extern const Vector<Vector<Vector<float>>> accustomedArchetypeBlendShapeDeltas;
extern const Vector<AlignedVector<float>> accustomedArchetypeJointGroupValues;
extern const Vector<float> spliceWeights;

}  // namespace canonical

}  // namespace gs4
