// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/types/Aliases.h"

#include <cstdint>

#if defined(__clang__) || defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wunused-parameter"
#endif
#if defined(_MSC_VER) && !defined(__clang__)
    #pragma warning(push)
    #pragma warning(disable : 4100)
#endif

namespace dna {

class MockedReader : public Reader {
public:
    // HeaderReader methods
    std::uint16_t getFileFormatGeneration() const override {
        return static_cast<std::uint16_t>(2);
    }

    std::uint16_t getFileFormatVersion() const override {
        return static_cast<std::uint16_t>(1);
    }

    // DescriptorReader methods
    StringView getName() const override {
        return {};
    }

    Archetype getArchetype() const override {
        return {};
    }

    Gender getGender() const override {
        return {};
    }

    std::uint16_t getAge() const override {
        return {};
    }

    std::uint32_t getMetaDataCount() const override {
        return {};
    }

    StringView getMetaDataKey(std::uint32_t index) const override {
        return {};
    }

    StringView getMetaDataValue(const char* key) const override {
        return {};
    }

    TranslationUnit getTranslationUnit() const override {
        return {};
    }

    RotationUnit getRotationUnit() const override {
        return {};
    }

    CoordinateSystem getCoordinateSystem() const override {
        return {};
    }

    RotationSequence getRotationSequence() const override {
        return {};
    }

    RotationSign getRotationSign() const override {
        return {RotationDirection::positive, RotationDirection::positive, RotationDirection::positive};
    }

    FaceWindingOrder getFaceWindingOrder() const override {
        return {};
    }

    std::uint16_t getLODCount() const override {
        return {};
    }

    std::uint16_t getDBMaxLOD() const override {
        return {};
    }

    StringView getDBComplexity() const override {
        return {};
    }

    StringView getDBName() const override {
        return {};
    }

    // DefinitionReader methods
    std::uint16_t getGUIControlCount() const override {
        return {};
    }

    StringView getGUIControlName(std::uint16_t index) const override {
        return {};
    }

    std::uint16_t getRawControlCount() const override {
        return {};
    }

    StringView getRawControlName(std::uint16_t index) const override {
        return {};
    }

    std::uint16_t getJointCount() const override {
        return static_cast<std::uint16_t>(3u);
    }

    StringView getJointName(std::uint16_t index) const override {
        return {};
    }

    ArrayView<const std::uint16_t> getJointIndicesForLOD(std::uint16_t lod) const override {
        return {};
    }

    std::uint16_t getJointParentIndex(std::uint16_t index) const override {
        if (index == 0) {
            return 0u;
        }
        return static_cast<std::uint16_t>(index - 1u);
    }

    std::uint16_t getBlendShapeChannelCount() const override {
        return {};
    }

    StringView getBlendShapeChannelName(std::uint16_t index) const override {
        return {};
    }

    ArrayView<const std::uint16_t> getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const override {
        return {};
    }

    std::uint16_t getAnimatedMapCount() const override {
        return {};
    }

    ArrayView<const std::uint16_t> getAnimatedMapIndicesForLOD(std::uint16_t lod) const override {
        return {};
    }

    StringView getAnimatedMapName(std::uint16_t index) const override {
        return {};
    }

    std::uint16_t getMeshCount() const override {
        return {};
    }

    StringView getMeshName(std::uint16_t index) const override {
        return {};
    }

    std::uint16_t getMeshBlendShapeChannelMappingCount() const override {
        return {};
    }

    MeshBlendShapeChannelMapping getMeshBlendShapeChannelMapping(std::uint16_t index) const override {
        return {};
    }

    ArrayView<const std::uint16_t> getMeshBlendShapeChannelMappingIndicesForLOD(std::uint16_t lod) const override {
        return {};
    }

    ArrayView<const std::uint16_t> getMeshIndicesForLOD(std::uint16_t lod) const override {
        return {};
    }

    Vector3 getNeutralJointTranslation(std::uint16_t index) const override {
        return {};
    }

    ConstArrayView<float> getNeutralJointTranslationXs() const override {
        return {};
    }

    ConstArrayView<float> getNeutralJointTranslationYs() const override {
        return {};
    }

    ConstArrayView<float> getNeutralJointTranslationZs() const override {
        return {};
    }

    Vector3 getNeutralJointRotation(std::uint16_t index) const override {
        return {};
    }

    ConstArrayView<float> getNeutralJointRotationXs() const override {
        return {};
    }

    ConstArrayView<float> getNeutralJointRotationYs() const override {
        return {};
    }

    ConstArrayView<float> getNeutralJointRotationZs() const override {
        return {};
    }

    // BehaviorReader methods
    ArrayView<const std::uint16_t> getGUIToRawInputIndices() const override {
        return {};
    }

    ArrayView<const std::uint16_t> getGUIToRawOutputIndices() const override {
        return {};
    }

    ConstArrayView<float> getGUIToRawFromValues() const override {
        return {};
    }

    ConstArrayView<float> getGUIToRawToValues() const override {
        return {};
    }

    ConstArrayView<float> getGUIToRawSlopeValues() const override {
        return {};
    }

    ConstArrayView<float> getGUIToRawCutValues() const override {
        return {};
    }

    std::uint16_t getPSDCount() const override {
        return {};
    }

    ArrayView<const std::uint16_t> getPSDRowIndices() const override {
        return {};
    }

    ArrayView<const std::uint16_t> getPSDColumnIndices() const override {
        return {};
    }

    ConstArrayView<float> getPSDValues() const override {
        return {};
    }

    ArrayView<const std::uint16_t> getJointVariableAttributeIndices(std::uint16_t lod) const override {
        return {};
    }

    std::uint16_t getJointRowCount() const override {
        return {};
    }

    std::uint16_t getJointColumnCount() const override {
        return {};
    }

    std::uint16_t getJointGroupCount() const override {
        return {};
    }

    ArrayView<const std::uint16_t> getJointGroupLODs(std::uint16_t jointGroupIndex) const override {
        return {};
    }

    ArrayView<const std::uint16_t> getJointGroupInputIndices(std::uint16_t jointGroupIndex) const override {
        return {};
    }

    ArrayView<const std::uint16_t> getJointGroupOutputIndices(std::uint16_t jointGroupIndex) const override {
        return {};
    }

    ConstArrayView<float> getJointGroupValues(std::uint16_t jointGroupIndex) const override {
        return {};
    }

    ArrayView<const std::uint16_t> getJointGroupJointIndices(std::uint16_t jointGroupIndex) const override {
        return {};
    }

    ArrayView<const std::uint16_t> getBlendShapeChannelLODs() const override {
        return {};
    }

    ArrayView<const std::uint16_t> getBlendShapeChannelInputIndices() const override {
        return {};
    }

    ArrayView<const std::uint16_t> getBlendShapeChannelOutputIndices() const override {
        return {};
    }

    ArrayView<const std::uint16_t> getAnimatedMapLODs() const override {
        return {};
    }

    ArrayView<const std::uint16_t> getAnimatedMapInputIndices() const override {
        return {};
    }

    ArrayView<const std::uint16_t> getAnimatedMapOutputIndices() const override {
        return {};
    }

    ConstArrayView<float> getAnimatedMapFromValues() const override {
        return {};
    }

    ConstArrayView<float> getAnimatedMapToValues() const override {
        return {};
    }

    ConstArrayView<float> getAnimatedMapSlopeValues() const override {
        return {};
    }

    ConstArrayView<float> getAnimatedMapCutValues() const override {
        return {};
    }

    // GeometryReader methods
    std::uint32_t getVertexPositionCount(std::uint16_t meshIndex) const override {
        return {};
    }

    Position getVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override {
        return {};
    }

    ConstArrayView<float> getVertexPositionXs(std::uint16_t meshIndex) const override {
        return {};
    }

    ConstArrayView<float> getVertexPositionYs(std::uint16_t meshIndex) const override {
        return {};
    }

    ConstArrayView<float> getVertexPositionZs(std::uint16_t meshIndex) const override {
        return {};
    }

    std::uint32_t getVertexTextureCoordinateCount(std::uint16_t meshIndex) const override {
        return {};
    }

    TextureCoordinate getVertexTextureCoordinate(std::uint16_t meshIndex, std::uint32_t textureCoordinateIndex) const override {
        return {};
    }

    ConstArrayView<float> getVertexTextureCoordinateUs(std::uint16_t meshIndex) const override {
        return {};
    }

    ConstArrayView<float> getVertexTextureCoordinateVs(std::uint16_t meshIndex) const override {
        return {};
    }

    std::uint32_t getVertexNormalCount(std::uint16_t meshIndex) const override {
        return {};
    }

    Normal getVertexNormal(std::uint16_t meshIndex, std::uint32_t normalIndex) const override {
        return {};
    }

    ConstArrayView<float> getVertexNormalXs(std::uint16_t meshIndex) const override {
        return {};
    }

    ConstArrayView<float> getVertexNormalYs(std::uint16_t meshIndex) const override {
        return {};
    }

    ConstArrayView<float> getVertexNormalZs(std::uint16_t meshIndex) const override {
        return {};
    }

    std::uint32_t getFaceCount(std::uint16_t meshIndex) const override {
        return {};
    }

    ArrayView<const std::uint32_t> getFaceVertexLayoutIndices(std::uint16_t meshIndex, std::uint32_t faceIndex) const override {
        return {};
    }

    std::uint32_t getVertexLayoutCount(std::uint16_t meshIndex) const override {
        return {};
    }

    VertexLayout getVertexLayout(std::uint16_t meshIndex, std::uint32_t layoutIndex) const override {
        return {};
    }

    ArrayView<const std::uint32_t> getVertexLayoutPositionIndices(std::uint16_t meshIndex) const override {
        return {};
    }

    ArrayView<const std::uint32_t> getVertexLayoutTextureCoordinateIndices(std::uint16_t meshIndex) const override {
        return {};
    }

    ArrayView<const std::uint32_t> getVertexLayoutNormalIndices(std::uint16_t meshIndex) const override {
        return {};
    }

    std::uint16_t getMaximumInfluencePerVertex(std::uint16_t meshIndex) const override {
        return {};
    }

    ConstArrayView<float> getSkinWeightsValues(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override {
        return {};
    }

    ArrayView<const std::uint16_t> getSkinWeightsJointIndices(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override {
        return {};
    }

    std::uint16_t getBlendShapeTargetCount(std::uint16_t meshIndex) const override {
        return {};
    }

    std::uint16_t getBlendShapeChannelIndex(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override {
        return {};
    }

    std::uint32_t getBlendShapeTargetDeltaCount(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const override {
        return {};
    }

    Delta getBlendShapeTargetDelta(std::uint16_t meshIndex,
                                   std::uint16_t blendShapeTargetIndex,
                                   std::uint32_t deltaIndex) const override {
        return {};
    }

    ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t meshIndex,
                                                     std::uint16_t blendShapeTargetIndex) const override {
        return {};
    }

    ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t meshIndex,
                                                     std::uint16_t blendShapeTargetIndex) const override {
        return {};
    }

    ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t meshIndex,
                                                     std::uint16_t blendShapeTargetIndex) const override {
        return {};
    }

    ArrayView<const std::uint32_t> getBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                                    std::uint16_t blendShapeTargetIndex) const override {
        return {};
    }

    std::uint16_t getJointIndexListCount() const override {
        return {};
    }

    std::uint16_t getBlendShapeChannelIndexListCount() const override {
        return {};
    }

    std::uint16_t getAnimatedMapIndexListCount() const override {
        return {};
    }

    std::uint16_t getMeshIndexListCount() const override {
        return {};
    }

    // MachineLearnedBehaviorReader methods
    std::uint16_t getMLControlCount() const override {
        return {};
    }

    StringView getMLControlName(std::uint16_t index) const override {
        return {};
    }

    std::uint16_t getNeuralNetworkCount() const override {
        return {};
    }

    std::uint16_t getNeuralNetworkIndexListCount() const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getNeuralNetworkIndicesForLOD(std::uint16_t lod) const override {
        return {};
    }

    std::uint16_t getMeshRegionCount(std::uint16_t meshIndex) const override {
        return {};
    }

    StringView getMeshRegionName(std::uint16_t meshIndex, std::uint16_t regionIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getNeuralNetworkIndicesForMeshRegion(std::uint16_t meshIndex,
                                                                       std::uint16_t regionIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getNeuralNetworkInputIndices(std::uint16_t neuralNetIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getNeuralNetworkOutputIndices(std::uint16_t neuralNetIndex) const override {
        return {};
    }

    std::uint16_t getNeuralNetworkLayerCount(std::uint16_t neuralNetIndex) const override {
        return {};
    }

    ActivationFunction getNeuralNetworkLayerActivationFunction(std::uint16_t neuralNetIndex,
                                                               std::uint16_t layerIndex) const override {
        return {};
    }

    ConstArrayView<float> getNeuralNetworkLayerActivationFunctionParameters(std::uint16_t neuralNetIndex,
                                                                            std::uint16_t layerIndex) const override {
        return {};
    }

    ConstArrayView<float> getNeuralNetworkLayerBiases(std::uint16_t neuralNetIndex, std::uint16_t layerIndex) const override {
        return {};
    }

    ConstArrayView<float> getNeuralNetworkLayerWeights(std::uint16_t neuralNetIndex, std::uint16_t layerIndex) const override {
        return {};
    }

    std::uint16_t getRBFPoseCount() const override {
        return {};
    }

    StringView getRBFPoseName(std::uint16_t poseIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getRBFPoseJointOutputIndices(std::uint16_t poseIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getRBFPoseBlendShapeChannelOutputIndices(std::uint16_t poseIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getRBFPoseAnimatedMapOutputIndices(std::uint16_t poseIndex) const override {
        return {};
    }

    ConstArrayView<float> getRBFPoseJointOutputValues(std::uint16_t poseIndex) const override {
        return {};
    }

    float getRBFPoseScale(std::uint16_t poseIndex) const override {
        return {};
    }

    std::uint16_t getRBFPoseControlCount() const override {
        return {};
    }

    StringView getRBFPoseControlName(std::uint16_t poseControlIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getRBFPoseInputControlIndices(std::uint16_t poseIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getRBFPoseOutputControlIndices(std::uint16_t poseIndex) const override {
        return {};
    }

    ConstArrayView<float> getRBFPoseOutputControlWeights(std::uint16_t poseIndex) const override {
        return {};
    }

    std::uint16_t getRBFSolverCount() const override {
        return {};
    }

    std::uint16_t getRBFSolverIndexListCount() const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getRBFSolverIndicesForLOD(std::uint16_t lod) const override {
        return {};
    }

    StringView getRBFSolverName(std::uint16_t solverIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getRBFSolverRawControlIndices(std::uint16_t solverIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getRBFSolverPoseIndices(std::uint16_t solverIndex) const override {
        return {};
    }

    ConstArrayView<float> getRBFSolverRawControlValues(std::uint16_t solverIndex) const override {
        return {};
    }

    RBFSolverType getRBFSolverType(std::uint16_t solverIndex) const override {
        return {};
    }

    float getRBFSolverRadius(std::uint16_t solverIndex) const override {
        return {};
    }

    AutomaticRadius getRBFSolverAutomaticRadius(std::uint16_t solverIndex) const override {
        return {};
    }

    float getRBFSolverWeightThreshold(std::uint16_t solverIndex) const override {
        return {};
    }

    RBFDistanceMethod getRBFSolverDistanceMethod(std::uint16_t solverIndex) const override {
        return {};
    }

    RBFNormalizeMethod getRBFSolverNormalizeMethod(std::uint16_t solverIndex) const override {
        return {};
    }

    RBFFunctionType getRBFSolverFunctionType(std::uint16_t solverIndex) const override {
        return {};
    }

    TwistAxis getRBFSolverTwistAxis(std::uint16_t solverIndex) const override {
        return {};
    }

    TranslationRepresentation getJointTranslationRepresentation(std::uint16_t jointIndex) const override {
        return {};
    }

    RotationRepresentation getJointRotationRepresentation(std::uint16_t jointIndex) const override {
        return {};
    }

    ScaleRepresentation getJointScaleRepresentation(std::uint16_t jointIndex) const override {
        return {};
    }

    std::uint16_t getTwistCount() const override {
        return {};
    }

    TwistAxis getTwistSetupTwistAxis(std::uint16_t twistIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getTwistInputControlIndices(std::uint16_t twistIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getTwistOutputJointIndices(std::uint16_t twistIndex) const override {
        return {};
    }

    ConstArrayView<float> getTwistBlendWeights(std::uint16_t twistIndex) const override {
        return {};
    }

    std::uint16_t getSwingCount() const override {
        return {};
    }

    TwistAxis getSwingSetupTwistAxis(std::uint16_t swingIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getSwingInputControlIndices(std::uint16_t swingIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getSwingOutputJointIndices(std::uint16_t swingIndex) const override {
        return {};
    }

    ConstArrayView<float> getSwingBlendWeights(std::uint16_t swingIndex) const override {
        return {};
    }

    std::uint16_t getMLTypeCount() const override {
        return {};
    }

    std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const override {
        return {};
    }

    std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const override {
        return {};
    }

    MachineLearnedBehaviorOperationType getMLOperationType(std::uint16_t mlTypeIndex,
                                                           std::uint16_t mlOperationSetIndex,
                                                           std::uint16_t mlOperationIndex) const override {
        return {};
    }

    ConstArrayView<std::uint32_t> getMLOperationParameters(std::uint16_t mlTypeIndex,
                                                           std::uint16_t mlOperationSetIndex,
                                                           std::uint16_t mlOperationIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLOperationDependencyOperationSetIndices(std::uint16_t mlTypeIndex,
                                                                              std::uint16_t mlOperationSetIndex,
                                                                              std::uint16_t mlOperationIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLOperationDependencyOperationIndices(std::uint16_t mlTypeIndex,
                                                                           std::uint16_t mlOperationSetIndex,
                                                                           std::uint16_t mlOperationIndex) const override {
        return {};
    }

    std::uint16_t getMLOperationIndexListCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLOperationIndicesForLOD(std::uint16_t mlTypeIndex,
                                                              std::uint16_t mlOperationSetIndex,
                                                              std::uint16_t lod) const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLJointsInputIndices() const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLJointsOutputIndices() const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLJointsParameterKeys() const override {
        return {};
    }

    ConstArrayView<std::uint16_t> getMLJointsParameterValues() const override {
        return {};
    }

    void unload(DataLayer dataLayer) override {
    }
};

}  // namespace dna

#if defined(_MSC_VER) && !defined(__clang__)
    #pragma warning(pop)
#endif

#if defined(__clang__) || defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
