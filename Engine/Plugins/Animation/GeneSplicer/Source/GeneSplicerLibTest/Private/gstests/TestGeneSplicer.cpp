// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Assertions.h"
#include "gstests/Defs.h"
#include "gstests/FixtureReader.h"
#include "gstests/splicedata/MockedRegionAffiliationReader.h"

#include "genesplicer/GeneSplicerDNAReaderImpl.h"
#include "genesplicer/GeneSplicerImpl.h"
#include "genesplicer/TypeDefs.h"
#include "genesplicer/splicedata/SpliceData.h"
#include "pma/ScopedPtr.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

namespace {

using FixtureReaderPtr = ScopedPtr<FixtureReader>;

// Allowed delta between floating point test results
constexpr float threshold = 0.0001f;

}  // namespace

class TestGeneSplicer : public ::testing::TestWithParam<CalculationType> {
public:
    void SetUp() override {

        for (std::uint16_t dnaIdx = 0u; dnaIdx < canonical::dnaCount - 1u; ++dnaIdx) {
            smartDNAPtrs.push_back(makeScoped<FixtureReader>(dnaIdx));
        }
        for (std::uint16_t dnaIdx = 0u; dnaIdx < canonical::dnaCount - 1u; ++dnaIdx) {
            dnaRawPtrs.push_back(smartDNAPtrs[dnaIdx].get());
        }

        archetype = makeScoped<FixtureReader>(static_cast<std::uint16_t>(FixtureReader::archetype));
        output = ScopedPtr<GeneSplicerDNAReader>(GeneSplicerDNAReader::create(archetype.get(), &memRes));
        expected = makeScoped<FixtureReader>(static_cast<std::uint16_t>(FixtureReader::expected));
        regionAffiliations.reset(new MockedRegionAffiliationReader{});
        Vector<GenePoolMask> poolMasks{GenePoolMask::NeutralMeshes,
                                       GenePoolMask::BlendShapes,
                                       GenePoolMask::SkinWeights,
                                       GenePoolMask::NeutralJoints,
                                       GenePoolMask::JointBehavior};
        spliceData = SpliceData{&memRes};
        for (auto genePoolMask : poolMasks) {
            genePools.emplace_back(makeScoped<GenePool>(archetype.get(),
                                                        dnaRawPtrs.data(),
                                                        static_cast<std::uint16_t>(dnaRawPtrs.size()),
                                                        genePoolMask,
                                                        &memRes));
            auto poolName = std::to_string(static_cast<std::uint32_t>(genePoolMask));
            spliceData.registerGenePool(poolName.c_str(), regionAffiliations.get(), genePools.back().get());
            auto poolParams = spliceData.getPoolParams(poolName.c_str());
            poolParams->setSpliceWeights(0u,
                                         canonical::spliceWeights.data(),
                                         static_cast<std::uint32_t>(canonical::spliceWeights.size()));
        }
        spliceData.setBaseArchetype(archetype.get());
    }

protected:
    AlignedMemoryResource memRes;
    Vector<FixtureReaderPtr> smartDNAPtrs;
    Vector<const Reader*> dnaRawPtrs;
    ScopedPtr<GeneSplicerDNAReader> output;
    ScopedPtr<FixtureReader> archetype;
    ScopedPtr<FixtureReader> expected;
    SpliceData spliceData;
    Vector<ScopedPtr<GenePool>> genePools;

    std::unique_ptr<MockedRegionAffiliationReader> regionAffiliations;
    GeneSplicer geneSplicer{GetParam(), &memRes};
};

TEST_P(TestGeneSplicer, NeutralMeshSplicer) {
    geneSplicer.spliceNeutralMeshes(&spliceData, output.get());
    assertNeutralMeshes(output.get(), expected.get(), threshold);
}

TEST_P(TestGeneSplicer, BlendShapeSplicer) {
    geneSplicer.spliceBlendShapes(&spliceData, output.get());
    assertBlendShapeTargets(output.get(), expected.get(), threshold);
}

TEST_P(TestGeneSplicer, JointBehaviorSplicer) {
    geneSplicer.spliceJointBehavior(&spliceData, output.get());
    assertJointBehavior(output.get(), expected.get(), threshold);
}

TEST_P(TestGeneSplicer, NeutralJointSplicer) {
    geneSplicer.spliceNeutralJoints(&spliceData, output.get());
    assertNeutralJointTranslations(output.get(), expected.get(), threshold);
    assertNeutralJointRotations(output.get(), expected.get(), threshold);
}

TEST_P(TestGeneSplicer, SkinWeightSplicer) {
    geneSplicer.spliceSkinWeights(&spliceData, output.get());
    assertSkinWeights(output.get(), expected.get(), threshold);
}

TEST_P(TestGeneSplicer, SkinWeightSplicer0Weights) {
    const Vector<float> spliceWeights = {0.0f, 0.0f, 0.0f, 0.0f};
    auto poolName = std::to_string(static_cast<std::uint32_t>(GenePoolMask::SkinWeights));
    spliceData.getPoolParams(poolName.c_str())
        ->setSpliceWeights(0u, spliceWeights.data(), static_cast<std::uint32_t>(spliceWeights.size()));
    geneSplicer.spliceSkinWeights(&spliceData, output.get());
    assertSkinWeights(output.get(), archetype.get(), threshold);
}

INSTANTIATE_TEST_SUITE_P(Integration,
                         TestGeneSplicer,
                         ::testing::Values(CalculationType::Scalar, CalculationType::SSE, CalculationType::AVX));

}  // namespace gs4
