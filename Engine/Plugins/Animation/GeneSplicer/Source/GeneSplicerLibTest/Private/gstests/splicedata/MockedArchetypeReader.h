// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "gstests/MockedReader.h"

#include "genesplicer/TypeDefs.h"

#include <cstdint>

#if defined(__clang__) || defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wunused-parameter"
#endif
#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4100)
#endif

class MockedArchetypeReader : public dna::MockedReader {
public:
    MockedArchetypeReader() {
        outputIndicesPerJointGroup = {{2u, 3u, 7u, 8u, 15u, 18u, 19u, 22u, 23u, 25u}, {38u, 39u, 40u, 43u}};
        jointIndicesPerJointGroup = {{0u, 1u, 2u}, {4u}};
        jointIndicesPerVertex = {{0u, 1u, 2u, 4u}, {0u, 1u, 2u}};
        jointWeightsPerVertex = {{0.1f, 0.2f, 0.4f, 0.5f}, {0.3f, 0.1f, 0.2f}};
        jointCount = 3u;

        dbName = "db";
        dbComplexity = "complexity";
        dbMaxLodCount = 5u;
    }

    std::uint16_t getJointCount() const override {
        return jointCount;
    }

    std::uint16_t getJointGroupCount() const override {
        return 2u;
    }

    gs4::ArrayView<const std::uint16_t> getJointGroupOutputIndices(std::uint16_t jointGroupIndex) const override {
        return gs4::ArrayView<const std::uint16_t>{outputIndicesPerJointGroup[jointGroupIndex]};
    }

    gs4::ArrayView<const std::uint16_t> getJointGroupJointIndices(std::uint16_t jointGroupIndex) const override {
        return gs4::ArrayView<const std::uint16_t>{jointIndicesPerJointGroup[jointGroupIndex]};
    }

    std::uint16_t getMeshCount() const override {
        return 1u;
    }

    std::uint32_t getVertexPositionCount(std::uint16_t meshIndex) const override {
        // Ignore mesh index since there's only one mesh
        return 21u;
    }

    gs4::ArrayView<const std::uint16_t> getSkinWeightsJointIndices(std::uint16_t meshIndex,
                                                                   std::uint32_t vertexIndex) const override {
        // Ignore mesh index since there's only one mesh
        return gs4::ArrayView<const std::uint16_t>{jointIndicesPerVertex[vertexIndex]};
    }

    gs4::ConstArrayView<float> getSkinWeightsValues(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override {
        // Ignore mesh index since there's only one mesh
        return gs4::ConstArrayView<float>{jointWeightsPerVertex[vertexIndex]};
    }

    std::uint32_t getSkinWeightsCount(std::uint16_t meshIndex) const override {
        // Ignore mesh index since there's only one mesh
        return getVertexPositionCount(meshIndex);
    }

    std::uint16_t getDBMaxLOD() const override {
        return dbMaxLodCount;
    }

    gs4::StringView getDBComplexity() const override {
        return {dbComplexity.data(), dbComplexity.length()};
    }

    gs4::StringView getDBName() const override {
        return {dbName.data(), dbName.length()};
    }

    void setDBMaxLod(std::uint16_t lodCount) {
        dbMaxLodCount = lodCount;
    }

    void setDBComplexity(gs4::String<char> complexity) {
        dbComplexity = complexity;
    }

    void setDBname(gs4::String<char> name) {
        dbName = name;
    }

    void setJointCount(std::uint16_t jointCount_) {
        jointCount = jointCount_;
    }

private:
    gs4::Matrix<std::uint16_t> outputIndicesPerJointGroup;
    gs4::Matrix<std::uint16_t> jointIndicesPerJointGroup;
    gs4::Matrix<std::uint16_t> jointIndicesPerVertex;
    gs4::Matrix<float> jointWeightsPerVertex;
    gs4::String<char> dbComplexity;
    gs4::String<char> dbName;
    std::uint16_t jointCount;
    std::uint16_t dbMaxLodCount;
};
#ifdef _MSC_VER
    #pragma warning(pop)
#endif
#if defined(__clang__) || defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
