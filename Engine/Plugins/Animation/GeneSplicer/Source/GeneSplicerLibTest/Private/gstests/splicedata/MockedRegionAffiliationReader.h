// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "genesplicer/TypeDefs.h"

#include <cstdint>

#if defined(__clang__) || defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wunused-parameter"
#endif
#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4100)
#endif

class MockedRegionAffiliationReader : public gs4::RegionAffiliationReader {
private:
    static const std::uint16_t vertexCount = 21;

public:
    MockedRegionAffiliationReader() {
        vertexRegionIndices = {

            {0u, 1u},
            {0u},
            {1u},

        };
        vertexRegionAffiliations = {

            {0.7f, 0.5f},
            {0.6f},
            {1.0f}

        };
        jointRegionIndices = {{}, {0u}, {0u, 1u}};
        jointRegionAffiliations = {{}, {1.0f}, {0.3f, 0.7f}};
    }

    std::uint16_t getMeshCount() const override {
        return 2u;
    }

    std::uint32_t getVertexCount(std::uint16_t meshIndex) const override {
        if (meshIndex < 2u) {
            return vertexCount;
        }
        return 0u;
    }

    gs4::ConstArrayView<std::uint16_t> getVertexRegionIndices(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override {
        if ((meshIndex < 2u) && (vertexIndex < vertexCount)) {
            vertexIndex = vertexIndex % 3;
            return gs4::ConstArrayView<std::uint16_t>{vertexRegionIndices[vertexIndex]};
        }
        return {};
    }

    gs4::ConstArrayView<float> getVertexRegionAffiliation(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override {
        if ((meshIndex < 2u) && (vertexIndex < vertexCount)) {
            vertexIndex = vertexIndex % 3;
            return gs4::ConstArrayView<float>{vertexRegionAffiliations[vertexIndex]};
        }

        return {};
    }

    std::uint16_t getJointCount() const override {
        return static_cast<std::uint16_t>(jointRegionIndices.size());
    }

    gs4::ConstArrayView<std::uint16_t> getJointRegionIndices(std::uint16_t jointIndex) const override {
        return gs4::ConstArrayView<std::uint16_t>{jointRegionIndices[jointIndex]};
    }

    gs4::ConstArrayView<float> getJointRegionAffiliation(std::uint16_t jointIndex) const override {
        return gs4::ConstArrayView<float>{jointRegionAffiliations[jointIndex]};
    }

    std::uint16_t getRegionCount() const override {
        return 2ul;
    }

    av::StringView getRegionName(std::uint16_t regionIndex) const override {
        return {};
    }

protected:
    gs4::Matrix<std::uint16_t> vertexRegionIndices;
    gs4::Matrix<float> vertexRegionAffiliations;
    gs4::Matrix<std::uint16_t> jointRegionIndices;
    gs4::Matrix<float> jointRegionAffiliations;
};

class MockedRegionAffiliationReaderMeshCountOther : public MockedRegionAffiliationReader {
    std::uint16_t getMeshCount() const override {
        return 0u;
    }
};

template<std::uint16_t JointCount>
class MockedRegionAffiliationReaderJointCountOther : public MockedRegionAffiliationReader {
public:
    std::uint16_t getJointCount() const override {
        return JointCount;
    }

    gs4::ConstArrayView<std::uint16_t> getJointRegionIndices(std::uint16_t jointIndex) const override {
        return gs4::ConstArrayView<std::uint16_t>{jointRegionIndices[jointIndex % jointRegionAffiliations.size()]};
    }

    gs4::ConstArrayView<float> getJointRegionAffiliation(std::uint16_t jointIndex) const override {
        return gs4::ConstArrayView<float>{jointRegionAffiliations[jointIndex % jointRegionAffiliations.size()]};
    }
};

class MockedRegionAffiliationReaderVertexCountOther : public MockedRegionAffiliationReader {
    std::uint32_t getVertexCount(std::uint16_t meshIndex) const override {
        return 0u;
    }
};
#ifdef _MSC_VER
    #pragma warning(pop)
#endif
#if defined(__clang__) || defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
