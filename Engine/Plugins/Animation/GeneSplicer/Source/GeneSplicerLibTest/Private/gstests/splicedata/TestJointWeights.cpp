// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Defs.h"
#include "gstests/splicedata/MockedRegionAffiliationReader.h"

#include "genesplicer/splicedata/JointWeights.h"
#include "genesplicer/splicedata/SpliceWeights.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/Block.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <iterator>
#include <memory>
#include <numeric>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

class TestJointWeights : public ::testing::Test {
protected:
    using RegionAfiliationReaderType = MockedRegionAffiliationReaderJointCountOther<18U>;

protected:
    void SetUp() override {
        regionAffiliations.reset(new RegionAfiliationReaderType{});

        static const std::uint16_t dnaCount = 2u;
        dnaIndices = gs4::Vector<std::uint16_t>{dnaCount, 0u, &memRes};
        std::iota(dnaIndices.begin(), dnaIndices.end(), static_cast<std::uint16_t>(0u));

        static const std::uint16_t regionCount = 2u;
        spliceWeights.reset(new gs4::SpliceWeights{dnaCount, regionCount, &memRes});

        static const float rawWeights[] = {0.2f, 0.3f};
        gs4::Vector<float> weights{dnaCount * regionCount, {}, &memRes};
        auto insertPtr = weights.data();
        for (std::uint16_t dnaIdx = 0u; dnaIdx < dnaCount; dnaIdx++) {
            std::copy_n(rawWeights, regionCount, insertPtr);
            std::advance(insertPtr, regionCount);
        }

        spliceWeights->set(0u, gs4::ConstArrayView<float>{weights});
        jointWeights.reset(new gs4::JointWeights{regionAffiliations.get(), &memRes});
        // **************************Joint-0         Joint-1         Joint-2   =||=
        // Region weights        0.00f, 0.00f    1.00f, 0.00f    0.30f, 0.70f  =||=
        // Splice weights        0.20f, 0.30f    0.20f, 0.30f    0.20f, 0.30f  =||=
        // Expected joint weight     0.00f           0.20f           0.27f     =||=
        expectedJointWeights.assign(
            {{0.0f, 0.2f, 0.27f, 0.0f, 0.2f, 0.27f, 0.0f, 0.2f, 0.27f, 0.0f, 0.2f, 0.27f, 0.0f, 0.2f, 0.27f, 0.0f},
             {0.2f, 0.27f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}});
    }

protected:
    pma::AlignedMemoryResource memRes;
    std::unique_ptr<gs4::SpliceWeights> spliceWeights;
    std::unique_ptr<gs4::JointWeights> jointWeights;
    std::unique_ptr<RegionAfiliationReaderType> regionAffiliations;
    gs4::Vector<std::uint16_t> dnaIndices;
    std::vector<gs4::VBlock<16u>> expectedJointWeights;
};

TEST_F(TestJointWeights, Empty) {
    ASSERT_TRUE(jointWeights->empty());
}

TEST_F(TestJointWeights, Clear) {
    ASSERT_TRUE(jointWeights->empty());
    jointWeights->compute(*spliceWeights, gs4::ConstArrayView<std::uint16_t>{dnaIndices});
    ASSERT_FALSE(jointWeights->empty());
    jointWeights->clear();
    ASSERT_TRUE(jointWeights->empty());
}

TEST_F(TestJointWeights, ComputeWeights) {
    std::uint16_t dnaCount = spliceWeights->getDNACount();

    jointWeights->compute(*spliceWeights, gs4::ConstArrayView<std::uint16_t>{dnaIndices});
    const auto& result = jointWeights->getData();

    std::size_t blockSize = gs4::TiledMatrix2D<16u>::value_type::size();
    std::size_t expectedBlockCount = 2u;
    ASSERT_EQ(result.rowCount(), expectedBlockCount);

    for (std::size_t blockIndex = 0u; blockIndex < expectedBlockCount; blockIndex++) {
        ASSERT_EQ(result[blockIndex].size(), dnaCount);

        for (std::uint16_t i = 0; i < blockSize; i++) {
            for (std::size_t dnaIndex = 0; dnaIndex < dnaCount; dnaIndex++) {
                ASSERT_NEAR(result[blockIndex][dnaIndex].v[i], expectedJointWeights[blockIndex][i], 0.0001f);
            }
        }
    }
}

TEST_F(TestJointWeights, ComputeWeightsDNAFilter) {
    std::uint16_t dnaCount = spliceWeights->getDNACount();
    gs4::Vector<std::uint16_t> dnaFilter{1u};

    jointWeights->compute(*spliceWeights, gs4::ConstArrayView<std::uint16_t>{dnaFilter});
    const auto& result = jointWeights->getData();

    std::size_t expectedBlockCount = 2u;
    ASSERT_EQ(result.rowCount(), expectedBlockCount);  // two blocks present

    for (std::size_t blockIndex = 0u; blockIndex < expectedBlockCount; blockIndex++) {
        ASSERT_EQ(result[blockIndex].size(), dnaCount);
        auto block = result[blockIndex];
        const auto& dna0Block = block[0u];
        const auto& dna1Block = block[1u];

        for (std::uint16_t i = 0; i < gs4::TiledMatrix2D<16u>::value_type::size(); i++) {
            ASSERT_NEAR(dna0Block.v[i], 0.0f, 0.0001f);
            ASSERT_NEAR(dna1Block.v[i], expectedJointWeights[blockIndex][i], 0.0001f);
        }
    }
}
