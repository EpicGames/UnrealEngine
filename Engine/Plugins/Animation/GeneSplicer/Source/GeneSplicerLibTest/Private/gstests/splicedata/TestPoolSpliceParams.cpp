// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Defs.h"
#include "gstests/FixtureReader.h"
#include "gstests/splicedata/MockedArchetypeReader.h"
#include "gstests/splicedata/MockedRegionAffiliationReader.h"

#include "genesplicer/CalculationType.h"
#include "genesplicer/splicedata/PoolSpliceParamsImpl.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/PImplExtractor.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 6326)
#endif

namespace gs4 {

class TestPoolSpliceParams : public ::testing::Test {
protected:
    using ReaderVector = gs4::Vector<const dna::Reader*>;

protected:
    void SetUp() override {
        arch = pma::makeScoped<gs4::FixtureReader>(static_cast<std::uint16_t>(gs4::FixtureReader::archetype));
        dna0 = pma::makeScoped<gs4::FixtureReader>(static_cast<std::uint16_t>(0u));
        dna1 = pma::makeScoped<gs4::FixtureReader>(static_cast<std::uint16_t>(1u));
        readers = ReaderVector{&memRes};
        readers.push_back(dna0.get());
        readers.push_back(dna1.get());

        regionAffiliations.reset(new MockedRegionAffiliationReader{});
        readerOther.reset(new MockedArchetypeReader{});
        readerOther->setDBname("dbOther");
        readerOther->setJointCount(5ul);
        readerOthers.push_back(readerOther.get());
    }

protected:
    pma::AlignedMemoryResource memRes;
    pma::ScopedPtr<gs4::FixtureReader> arch;
    pma::ScopedPtr<gs4::FixtureReader> dna0;
    pma::ScopedPtr<gs4::FixtureReader> dna1;
    std::unique_ptr<MockedRegionAffiliationReader> regionAffiliations;
    std::unique_ptr<MockedArchetypeReader> readerOther;

    ReaderVector readers;
    ReaderVector readerOthers;
};

TEST_F(TestPoolSpliceParams, CacheAll) {
    auto genePool = gs4::GenePool{arch.get(), readers.data(), 1u, gs4::GenePoolMask::All, &memRes};
    auto poolSpliceParams = pma::makeScoped<gs4::PoolSpliceParamsImpl>(regionAffiliations.get(), &genePool, &memRes);
    const float weights[] = {0.2f, 0.3f};
    poolSpliceParams->setSpliceWeights(0u, weights, 2u);

    poolSpliceParams->cacheAll();

    const auto& jointWeights = poolSpliceParams->getJointWeightsData();
    ASSERT_EQ(jointWeights.rowCount(), 1ul);     // One block
    ASSERT_EQ(jointWeights.columnCount(), 1ul);  // 1 dna

    const auto& vertexWeights = poolSpliceParams->getVertexWeightsData();
    ASSERT_EQ(vertexWeights.size(), 2ul);  // Two meshes
    for (std::uint16_t mi = 0u; mi < 2ul; mi++) {
        ASSERT_EQ(vertexWeights[mi].rowCount(), 2ul);     // Two blocks
        ASSERT_EQ(vertexWeights[mi].columnCount(), 1ul);  // 1 dna 24 vertices per dna
    }
}

TEST_F(TestPoolSpliceParams, GenePoolIncompatible) {
    auto genePool =
        gs4::GenePool{arch.get(), readers.data(), static_cast<std::uint16_t>(readers.size()), gs4::GenePoolMask::All, &memRes};
    auto rafOtherMeshCount = MockedRegionAffiliationReaderMeshCountOther{};
    auto poolSpliceParams = pma::makeScoped<gs4::PoolSpliceParamsImpl>(&rafOtherMeshCount, &genePool, &memRes);
    ASSERT_EQ(gs4::Status::get(), gs4::PoolSpliceParams::GenePoolIncompatible);
    ASSERT_EQ(poolSpliceParams.get(), nullptr);

    auto rafOtherJointCount = MockedRegionAffiliationReaderJointCountOther<0u>{};
    poolSpliceParams = pma::makeScoped<gs4::PoolSpliceParamsImpl>(&rafOtherJointCount, &genePool, &memRes);
    ASSERT_EQ(gs4::Status::get(), gs4::PoolSpliceParams::GenePoolIncompatible);
    ASSERT_EQ(poolSpliceParams.get(), nullptr);

    auto rafOtherVertexCount = MockedRegionAffiliationReaderVertexCountOther{};
    poolSpliceParams = pma::makeScoped<gs4::PoolSpliceParamsImpl>(&rafOtherVertexCount, &genePool, &memRes);
    ASSERT_EQ(gs4::Status::get(), gs4::PoolSpliceParams::GenePoolIncompatible);
    ASSERT_EQ(poolSpliceParams.get(), nullptr);

    poolSpliceParams = pma::makeScoped<gs4::PoolSpliceParamsImpl>(regionAffiliations.get(), &genePool, &memRes);
    ASSERT_EQ(gs4::Status::isOk(), true);
    ASSERT_NE(poolSpliceParams.get(), nullptr);
}
}  // namespace gs4

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
