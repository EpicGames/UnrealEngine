// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/splicedata/RegionAffiliation.h"
#include "genesplicer/types/Aliases.h"
#include "gstests/Defs.h"

#include <gtest/gtest.h>

#include <cstdint>

namespace gs4 {

TEST(RegionAffiliation, ConstructorMemRes) {
    AlignedMemoryResource memRes;
    RegionAffiliation<> regionAffiliation{&memRes};

    ASSERT_EQ(0u, regionAffiliation.rest.size());
    for (std::size_t i = 0u; i < regionAffiliation.firstNSize(); i++) {
        ASSERT_EQ(0u, regionAffiliation.firstN[i].index);
        ASSERT_EQ(0.0f, regionAffiliation.firstN[i].value);
    }
}

TEST(RegionAffiliation, ConstructorIndicesValues) {
    AlignedMemoryResource memRes;

    Vector<RegionAffiliation<>::IndexValue> expectedFirstN{RegionAffiliation<>::firstNSize(), {1.0f, 0u}, &memRes};
    Vector<RegionAffiliation<>::IndexValue> expectedRest{{2.0f, 1u}, {3.0f, 2u}};

    Vector<std::uint16_t> regionIndices{};
    Vector<float> regionValues{};
    for (std::size_t i = 0; i < expectedFirstN.size(); i++) {
        regionIndices.push_back(expectedFirstN[i].index);
        regionValues.push_back(expectedFirstN[i].value);
    }
    for (std::size_t i = 0; i < expectedRest.size(); i++) {
        regionIndices.push_back(expectedRest[i].index);
        regionValues.push_back(expectedRest[i].value);
    }

    RegionAffiliation<> regionAffiliation{ConstArrayView<std::uint16_t>{regionIndices},
                                          ConstArrayView<float>{regionValues},
                                          &memRes};
    ASSERT_EQ(1u, regionAffiliation.firstN.size());
    for (std::size_t i = 0u; i < regionAffiliation.firstNSize(); i++) {
        ASSERT_EQ(expectedFirstN[i].index, regionAffiliation.firstN[i].index);
        ASSERT_EQ(expectedFirstN[i].value, regionAffiliation.firstN[i].value);
    }

    ASSERT_EQ(2u, regionAffiliation.rest.size());
    for (std::size_t i = 0u; i < regionAffiliation.rest.size(); i++) {
        ASSERT_EQ(expectedRest[i].index, regionAffiliation.rest[i].index);
        ASSERT_EQ(expectedRest[i].value, regionAffiliation.rest[i].value);
    }
}

TEST(RegionAffiliation, ConstructorIndicesValuesEmpty) {
    AlignedMemoryResource memRes;
    RegionAffiliation<> regionAffiliation{ConstArrayView<std::uint16_t>{}, ConstArrayView<float>{}, &memRes};

    ASSERT_EQ(0u, regionAffiliation.rest.size());
    for (std::size_t i = 0u; i < regionAffiliation.firstNSize(); i++) {
        ASSERT_EQ(0u, regionAffiliation.firstN[i].index);
        ASSERT_EQ(0.0f, regionAffiliation.firstN[i].value);
    }
}

}  // namespace gs4
