// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Assertions.h"
#include "gstests/Defs.h"
#include "gstests/FixtureReader.h"
#include "gstests/splicedata/MockedArchetypeReader.h"
#include "gstests/splicedata/MockedRegionAffiliationReader.h"

#include "genesplicer/CalculationType.h"
#include "genesplicer/splicedata/PoolSpliceParamsImpl.h"
#include "genesplicer/splicedata/SpliceDataImpl.h"
#include "genesplicer/splicedata/genepool/GenePoolInterface.h"
#include "genesplicer/splicedata/rawgenes/RawGenes.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/PImplExtractor.h"
#include "gstests/splicedata/rawgenes/AccustomedArchetypeReader.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 6326)
#endif

namespace gs4 {

class TestSpliceData : public ::testing::Test {
protected:
    using ReaderVector = Vector<const dna::Reader*>;

protected:
    void SetUp() override {
        arch = pma::makeScoped<FixtureReader>(static_cast<std::uint16_t>(FixtureReader::archetype));
        accustomedArch = makeScoped<AccustomedArchetypeReader>();
        rawGenesArch = makeScoped<RawGeneArchetypeDNAReader>();
        dna0 = pma::makeScoped<FixtureReader>(static_cast<std::uint16_t>(0u));
        dna1 = pma::makeScoped<FixtureReader>(static_cast<std::uint16_t>(1u));
        readers = ReaderVector{&memRes};
        readers.push_back(dna0.get());
        readers.push_back(dna1.get());

        regionAffiliations.reset(new MockedRegionAffiliationReader{});
        readerOther.reset(new MockedArchetypeReader{});
        readerOther->setDBname("dbOther");
        readerOther->setJointCount(5ul);
        readerOthers.push_back(readerOther.get());
        genePool = makeScoped<GenePool>(arch.get(),
                                        readers.data(),
                                        static_cast<std::uint16_t>(readers.size()),
                                        gs4::GenePoolMask::All,
                                        &memRes);
    }

protected:
    AlignedMemoryResource memRes;
    ScopedPtr<FixtureReader> arch;
    ScopedPtr<AccustomedArchetypeReader> accustomedArch;
    ScopedPtr<RawGeneArchetypeDNAReader> rawGenesArch;
    ScopedPtr<FixtureReader> dna0;
    ScopedPtr<FixtureReader> dna1;
    ScopedPtr<MockedRegionAffiliationReader> regionAffiliations;
    ScopedPtr<MockedArchetypeReader> readerOther;
    ScopedPtr<GenePool> genePool;

    ReaderVector readers;
    ReaderVector readerOthers;
};

TEST_F(TestSpliceData, Constructor) {
    SpliceDataInterface spliceData{&memRes};
    ASSERT_EQ(spliceData.getAllPoolParams().size(), 0u);
}

TEST_F(TestSpliceData, RegisterGenePool) {
    SpliceDataInterface spliceData{&memRes};
    std::string genePoolName{"name1"};
    spliceData.registerGenePool(StringView{genePoolName}, regionAffiliations.get(), genePool.get());
    ASSERT_EQ(spliceData.getAllPoolParams().size(), 1u);

    auto poolSpliceParms = static_cast<PoolSpliceParamsImpl*>(spliceData.getPoolParams(StringView{genePoolName}));
    ASSERT_EQ(poolSpliceParms->getGenePool(), PImplExtractor<GenePool>::get(genePool.get()));
}

TEST_F(TestSpliceData, UnregisterGenePool) {
    SpliceDataInterface spliceData{&memRes};
    ASSERT_EQ(spliceData.getAllPoolParams().size(), 0u);

    std::string genePoolName{"name1"};
    spliceData.registerGenePool(StringView{genePoolName}, regionAffiliations.get(), genePool.get());
    ASSERT_EQ(spliceData.getAllPoolParams().size(), 1u);

    spliceData.unregisterGenePool(StringView{genePoolName});

    auto poolSpliceParms = spliceData.getPoolParams(StringView{genePoolName});
    ASSERT_EQ(poolSpliceParms, nullptr);
    ASSERT_EQ(spliceData.getAllPoolParams().size(), 0u);
}

TEST_F(TestSpliceData, setBaseArchetype) {
    SpliceDataInterface spliceData{&memRes};
    spliceData.setBaseArchetype(arch.get());
    auto rawGenes = spliceData.getBaseArchetype();
    assertRawGenes(rawGenes, rawGenesArch.get());
}

TEST_F(TestSpliceData, getJointBehaviorTargetOffsets0) {
    SpliceDataInterface spliceData{&memRes};
    spliceData.setBaseArchetype(arch.get());
    auto rawGenes = spliceData.getBaseArchetype();
    std::string genePoolName{"name1"};
    spliceData.registerGenePool(StringView{genePoolName}, regionAffiliations.get(), genePool.get());
    auto actualOutputIndexTargetOffsets = spliceData.getAllPoolParams()[0]->getJointBehaviorOutputIndexTargetOffsets();
    std::size_t expectedRowCount = 3u;
    std::size_t expectedColumnCount = 9u;
    Vector<std::uint8_t> firstRow{0, 1, 2, 3, 0, 0, 4, 5, 0};
    Vector<std::uint8_t> secondRow{0, 6, 7, 8, 9, 0, 0, 0, 0};
    Vector<std::uint8_t> thirdRow{0, 0, 1, 2, 3, 4, 0, 0, 0};
    ASSERT_EQ(actualOutputIndexTargetOffsets.rowCount(), expectedRowCount);
    ASSERT_EQ(actualOutputIndexTargetOffsets.columnCount(), expectedColumnCount);
    ASSERT_ELEMENTS_AND_SIZE_EQ(actualOutputIndexTargetOffsets[0], firstRow);
    ASSERT_ELEMENTS_AND_SIZE_EQ(actualOutputIndexTargetOffsets[1], secondRow);
    ASSERT_ELEMENTS_AND_SIZE_EQ(actualOutputIndexTargetOffsets[2], thirdRow);
}

TEST_F(TestSpliceData, getJointBehaviorTargetOffsets1) {
    SpliceDataInterface spliceData{&memRes};
    std::string genePoolName{"name1"};
    spliceData.registerGenePool(StringView{genePoolName}, regionAffiliations.get(), genePool.get());
    spliceData.setBaseArchetype(arch.get());
    auto actualOutputIndexTargetOffsets = spliceData.getAllPoolParams()[0]->getJointBehaviorOutputIndexTargetOffsets();
    std::size_t expectedRowCount = 3u;
    std::size_t expectedColumnCount = 9u;
    Vector<std::uint8_t> firstRow{0, 1, 2, 3, 0, 0, 4, 5, 0};
    Vector<std::uint8_t> secondRow{0, 6, 7, 8, 9, 0, 0, 0, 0};
    Vector<std::uint8_t> thirdRow{0, 0, 1, 2, 3, 4, 0, 0, 0};
    ASSERT_EQ(actualOutputIndexTargetOffsets.rowCount(), expectedRowCount);
    ASSERT_EQ(actualOutputIndexTargetOffsets.columnCount(), expectedColumnCount);
    ASSERT_ELEMENTS_AND_SIZE_EQ(actualOutputIndexTargetOffsets[0], firstRow);
    ASSERT_ELEMENTS_AND_SIZE_EQ(actualOutputIndexTargetOffsets[1], secondRow);
    ASSERT_ELEMENTS_AND_SIZE_EQ(actualOutputIndexTargetOffsets[2], thirdRow);
}

}  // namespace gs4

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
