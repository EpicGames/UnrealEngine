// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Defs.h"

#include "genesplicer/splicedata/SpliceWeights.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

class TestSpliceWeights : public ::testing::Test {

protected:
    gs4::AlignedMemoryResource memRes;
    std::unique_ptr<gs4::SpliceWeights> spliceWeights;
};

TEST_F(TestSpliceWeights, getRegionCount) {
    spliceWeights.reset(new gs4::SpliceWeights{2u, 3u, &memRes});
    ASSERT_EQ(spliceWeights->getRegionCount(), 3u);
}

TEST_F(TestSpliceWeights, getDNACount) {
    spliceWeights.reset(new gs4::SpliceWeights{2u, 3u, &memRes});
    ASSERT_EQ(spliceWeights->getDNACount(), 2u);
}

TEST_F(TestSpliceWeights, GetWeightsForDNA) {
    std::uint16_t dnaCount = 3u;
    std::uint16_t regionCount = 4u;
    spliceWeights.reset(new gs4::SpliceWeights{dnaCount, regionCount, &memRes});

    const float weights[] = {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.5f, 0.6f, 0.8f};

    spliceWeights->set(0u, {weights, static_cast<std::size_t>(regionCount * dnaCount)});
    ASSERT_EQ(spliceWeights->get(0), (gs4::ConstArrayView<float>{weights, 4ul}));
    ASSERT_EQ(spliceWeights->get(1), (gs4::ConstArrayView<float>{weights + 4ul, 4ul}));
    ASSERT_EQ(spliceWeights->get(2), (gs4::ConstArrayView<float>{weights + 8ul, 4ul}));
}

TEST_F(TestSpliceWeights, GetWeightData) {
    std::uint16_t dnaCount = 3u;
    std::uint16_t regionCount = 4u;
    spliceWeights.reset(new gs4::SpliceWeights{dnaCount, regionCount, &memRes});

    const float weights[] = {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.5f, 0.6f, 0.8f};

    spliceWeights->set(0u, {weights, static_cast<std::size_t>(regionCount * dnaCount)});
    auto data = spliceWeights->getData();

    ASSERT_EQ(data[2][0], 0.0f);
    ASSERT_EQ(data[2][1], 0.5f);
    ASSERT_EQ(data[2][2], 0.6f);
    ASSERT_EQ(data[2][3], 0.8f);
}

TEST_F(TestSpliceWeights, OffsetSetWeightData) {
    std::uint16_t dnaCount = 3u;
    std::uint16_t regionCount = 4u;
    spliceWeights.reset(new gs4::SpliceWeights{dnaCount, regionCount, &memRes});

    const float weights[] = {0.1f, 0.2f, 0.3f, 0.4f, 0.5f, 0.6f, 0.7f, 0.8f};

    spliceWeights->set(1u, {weights, static_cast<std::size_t>(regionCount * 2u)});
    auto data = spliceWeights->getData();

    ASSERT_EQ(spliceWeights->get(1), (gs4::ConstArrayView<float>{weights, 4ul}));
    ASSERT_EQ(spliceWeights->get(2), (gs4::ConstArrayView<float>{weights + 4ul, 4ul}));
}
