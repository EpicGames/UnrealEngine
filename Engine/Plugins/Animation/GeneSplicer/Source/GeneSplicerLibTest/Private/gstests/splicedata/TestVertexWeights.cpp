// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Defs.h"
#include "gstests/splicedata/MockedRegionAffiliationReader.h"

#include "genesplicer/splicedata/SpliceWeights.h"
#include "genesplicer/splicedata/VertexWeights.h"
#include "genesplicer/types/Aliases.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#include <numeric>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

template<typename T>
gs4::Vector<T> getIotaVector(T n, gs4::MemoryResource* memRes) {
    gs4::Vector<T> rets{n, {}, memRes};
    std::iota(rets.begin(), rets.end(), T{});
    return rets;
}

class TestVertexWeights : public ::testing::Test {
protected:
    void SetUp() override {

        regionAffiliations.reset(new MockedRegionAffiliationReader{});
        std::uint16_t regionCount = regionAffiliations->getRegionCount();
        std::uint16_t meshCount = regionAffiliations->getMeshCount();

        spliceWeights.reset(new gs4::SpliceWeights{dnaCount, regionCount, &memRes});
        static const float weights[] = {0.2f, 0.3f, 0.2f, 0.3f};
        spliceWeights->set(0u, {weights, 4u});
        vertexWeights.reset(new gs4::VertexWeights{regionAffiliations.get(), &memRes});

        meshIndices = getIotaVector(meshCount, &memRes);
        dnaIndices = getIotaVector(dnaCount, &memRes);
    }

protected:
    gs4::AlignedMemoryResource memRes;

    std::uint16_t dnaCount = 2u;
    std::unique_ptr<gs4::SpliceWeights> spliceWeights;
    std::unique_ptr<gs4::VertexWeights> vertexWeights;
    std::unique_ptr<MockedRegionAffiliationReader> regionAffiliations;
    gs4::Vector<std::uint16_t> meshIndices;
    gs4::Vector<std::uint16_t> dnaIndices;
};

TEST_F(TestVertexWeights, Empty) {
    ASSERT_TRUE(vertexWeights->empty());
}

TEST_F(TestVertexWeights, Clear) {
    ASSERT_TRUE(vertexWeights->empty());
    vertexWeights->compute(*spliceWeights,
                           gs4::ConstArrayView<std::uint16_t>{meshIndices},
                           gs4::ConstArrayView<std::uint16_t>{dnaIndices});
    ASSERT_FALSE(vertexWeights->empty());
    vertexWeights->clear();
    ASSERT_TRUE(vertexWeights->empty());
}

TEST_F(TestVertexWeights, ComputeWeights) {
    const auto meshCount = regionAffiliations->getMeshCount();

    vertexWeights->compute(*spliceWeights,
                           gs4::ConstArrayView<std::uint16_t>{meshIndices},
                           gs4::ConstArrayView<std::uint16_t>{dnaIndices});
    const auto& result = vertexWeights->getData();
    for (std::uint16_t dnaIndex = 0u; dnaIndex < dnaCount; dnaIndex++) {
        ASSERT_EQ(result.size(), meshCount);
        const std::size_t meshIndex = 0ul;             // Only 1 mesh
        ASSERT_EQ(result[meshIndex].rowCount(), 2ul);  // 2 block, each block contains up to 16 values
        const std::size_t blockIndex = 0u;
        // Vertex-0
        // Region weights
        // R0    R1
        // 0.7f, 0.5f
        // Splice weights
        // R0    R1
        // 0.2f, 0.3f
        // Expected vertex weight
        // R0  x  S0  +  R1  x  S1
        // 0.7f x 0.2f + 0.5f x 0.3f = 0.29f
        const std::size_t vtx0 = 0ul;
        const float vtx0Weight = 0.29f;
        ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[vtx0], vtx0Weight, 0.0001f);
        // Vertex-1
        // Region weights
        // R0    R1
        // 0.6f, 0.0f
        // Splice weights
        // R0    R1
        // 0.2f, 0.3f
        // Expected vertex weight
        // R0  x  S0  +  R1  x  S1
        // 0.6f x 0.2f + 0.0f x 0.3f = 0.12f
        const std::size_t vtx1 = 1ul;
        const float vtx1Weight = 0.12f;
        ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[vtx1], vtx1Weight, 0.0001f);
        // Vertex-2
        // Region weights
        // R0    R1
        // 0.0f, 1.0f
        // Splice weights
        // R0    R1
        // 0.2f, 0.3f
        // Expected vertex weight
        // R0  x  S0  +  R1  x  S1
        // 0.0f x 0.2f + 1.0f x 0.3f = 0.3f
        const std::size_t vtx2 = 2ul;
        const float vtx2Weight = 0.3f;
        ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[vtx2], vtx2Weight, 0.0001f);

        ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[3], vtx0Weight, 0.0001f);
        ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[4], vtx1Weight, 0.0001f);
        ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[5], vtx2Weight, 0.0001f);
    }
}

TEST_F(TestVertexWeights, ComputeWeightsFiltered) {
    const auto meshCount = regionAffiliations->getMeshCount();
    std::vector<std::uint16_t> dnaIndicesHelper{0};
    vertexWeights->compute(*spliceWeights,
                           gs4::ConstArrayView<std::uint16_t>{meshIndices},
                           gs4::ConstArrayView<std::uint16_t>{dnaIndicesHelper});
    const auto& result = vertexWeights->getData();
    ASSERT_EQ(result.size(), meshCount);
    const std::size_t meshIndex = 0ul;            // Only 1 mesh
    ASSERT_EQ(result[meshIndex].rowCount(), 2u);  // 2 block, each block contains up to 16 values
    const std::size_t blockIndex = 0u;
    std::uint16_t dnaIndex = 0u;
    // Vertex-0
    // Region weights
    // R0    R1
    // 0.7f, 0.5f
    // Splice weights
    // R0    R1
    // 0.2f, 0.3f
    // Expected vertex weight
    // R0  x  S0  +  R1  x  S1
    // 0.7f x 0.2f + 0.5f x 0.3f = 0.29f
    const std::size_t vtx0 = 0ul;
    const float vtx0Weight = 0.29f;
    ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[vtx0], vtx0Weight, 0.0001f);
    // Vertex-1
    // Region weights
    // R0    R1
    // 0.6f, 0.0f
    // Splice weights
    // R0    R1
    // 0.2f, 0.3f
    // Expected vertex weight
    // R0  x  S0  +  R1  x  S1
    // 0.6f x 0.2f + 0.0f x 0.3f = 0.12f
    const std::size_t vtx1 = 1ul;
    const float vtx1Weight = 0.12f;
    ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[vtx1], vtx1Weight, 0.0001f);
    // Vertex-2
    // Region weights
    // R0    R1
    // 0.0f, 1.0f
    // Splice weights
    // R0    R1
    // 0.2f, 0.3f
    // Expected vertex weight
    // R0  x  S0  +  R1  x  S1
    // 0.0f x 0.2f + 1.0f x 0.3f = 0.3f
    const std::size_t vtx2 = 2ul;
    const float vtx2Weight = 0.3f;
    ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[vtx2], vtx2Weight, 0.0001f);

    ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[3], vtx0Weight, 0.0001f);
    ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[4], vtx1Weight, 0.0001f);
    ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[5], vtx2Weight, 0.0001f);

    dnaIndex = 1u;
    ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[vtx0], 0.0f, 0.0001f);
    ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[vtx1], 0.0f, 0.0001f);
    ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[vtx2], 0.0f, 0.0001f);
    ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[3], 0.0f, 0.0001f);
    ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[4], 0.0f, 0.0001f);
    ASSERT_NEAR(result[meshIndex][blockIndex][dnaIndex].v[5], 0.0f, 0.0001f);
}
