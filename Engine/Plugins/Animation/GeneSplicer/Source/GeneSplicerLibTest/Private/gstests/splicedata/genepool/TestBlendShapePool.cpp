// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Assertions.h"
#include "gstests/Defs.h"
#include "gstests/Fixtures.h"
#include "gstests/splicedata/genepool/TestPool.h"

#include "genesplicer/splicedata/genepool/BlendShapePool.h"

#include <cstddef>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 6326)
#endif

namespace gs4 {

using TestBlendShapePool = TestPool;

TEST_F(TestBlendShapePool, VertexIndices) {
    auto blendShapePool = BlendShapePool{arch.get(), ConstArrayView<const dna::Reader*>{readers}, &memRes};
    const auto& indices = blendShapePool.getVertexIndices();
    const auto& expectedIndices = canonical::expectedBlendShapePoolVertexIndices;
    assertBlendShapePoolVertexIndices(indices, expectedIndices);
}

TEST_F(TestBlendShapePool, Deltas) {
    auto blendShapePool = BlendShapePool{arch.get(), ConstArrayView<const dna::Reader*>{readers}, &memRes};
    const auto deltas = blendShapePool.getBlendShapeTargetDeltas();
    ASSERT_EQ(canonical::expectedBlendShapePoolBucketOffsets.size(), deltas.bucketOffsets.rowCount());
    for (std::size_t mi = 0u; mi < deltas.bucketOffsets.rowCount(); mi++) {
        const auto& bucketOffset = deltas.bucketOffsets[mi];
        ASSERT_ELEMENTS_AND_SIZE_EQ(bucketOffset, canonical::expectedBlendShapePoolBucketOffsets[mi]);
    }
    ASSERT_ELEMENTS_AND_SIZE_EQ(deltas.bucketDNABlockOffsets, canonical::expectedBlendShapePoolBucketDNABlockOffsets);
    ASSERT_ELEMENTS_AND_SIZE_EQ(deltas.bucketVertexIndices, canonical::expectedBlendShapePoolBucketVertexIndices);
    ASSERT_ELEMENTS_AND_SIZE_EQ(deltas.dnaBlocks, canonical::expectedBlendShapePoolDNADeltas);
    ASSERT_ELEMENTS_AND_SIZE_EQ(deltas.archBlocks, canonical::expectedBlendShapePoolArchDeltas);
    ASSERT_ELEMENTS_AND_SIZE_EQ(deltas.dnaIndices, canonical::expectedBlendShapePoolDNAIndices);
}

}  // namespace gs4

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
