// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Assertions.h"
#include "gstests/splicedata/genepool/TestPool.h"

#include "genesplicer/CalculationType.h"
#include "genesplicer/splicedata/GenePool.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"
#include "genesplicer/splicedata/genepool/NullGenePoolImpl.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/PImplExtractor.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 6326)
#endif

namespace gs4 {

using TestGenePool = TestPool;

TEST_F(TestGenePool, Constructor) {
    GenePoolImpl genePool{arch.get(), {readers.data(), readers.size()}, gs4::GenePoolMask::All, &memRes};

    ASSERT_EQ(genePool.getDNACount(), readers.size());
    std::uint16_t meshCount = genePool.getMeshCount();
    ASSERT_EQ(meshCount, expectedReader->getMeshCount());
    for (std::uint16_t meshIndex = 0u; meshIndex < meshCount; meshIndex++) {
        ASSERT_EQ(genePool.getVertexCount(meshIndex), expectedReader->getVertexPositionCount(meshIndex));
        ASSERT_EQ(genePool.getBlendShapeTargetCount(meshIndex), expectedReader->getBlendShapeTargetCount(meshIndex));
        ASSERT_EQ(genePool.getSkinWeightsCount(meshIndex), expectedReader->getSkinWeightsCount(meshIndex));
        ASSERT_EQ(genePool.getMaximumInfluencesPerVertex(meshIndex), expectedReader->getMaximumInfluencePerVertex(meshIndex));
    }
    ASSERT_EQ(genePool.getJointCount(), expectedReader->getJointCount());
    ASSERT_EQ(genePool.getJointGroupCount(), expectedReader->getJointGroupCount());

    auto neutralMesh = genePool.getNeutralMeshes();
    const auto& expectedNeutralMesh = canonical::expectedNeutralMeshPoolValues;
    assertNeutralMeshPoolData(neutralMesh, expectedNeutralMesh);

    const auto& translations = genePool.getNeutralJoints(JointAttribute::Translation);
    const auto& expectedTranslations = canonical::expectedNeutralJointPoolTranslations;
    assertNeutralJointPool(translations, expectedTranslations);

    const auto& rotations = genePool.getNeutralJoints(JointAttribute::Rotation);
    const auto& expectedRotations = canonical::expectedNeutralJointPoolRotations;
    assertNeutralJointPool(rotations, expectedRotations);

    auto blendShapeIndices = genePool.getBlendShapeTargetVertexIndices();
    const auto& expectedBlendShapeIndices = canonical::expectedBlendShapePoolVertexIndices;
    assertBlendShapePoolVertexIndices(blendShapeIndices, expectedBlendShapeIndices);

    auto blendShapeDeltas = genePool.getBlendShapeTargetDeltas();
    ASSERT_EQ(canonical::expectedBlendShapePoolBucketOffsets.size(), blendShapeDeltas.bucketOffsets.rowCount());
    for (std::size_t mi = 0u; mi < blendShapeDeltas.bucketOffsets.rowCount(); mi++) {
        const auto& bucketOffset = blendShapeDeltas.bucketOffsets[mi];
        ASSERT_ELEMENTS_AND_SIZE_EQ(bucketOffset, canonical::expectedBlendShapePoolBucketOffsets[mi]);
    }
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.bucketDNABlockOffsets, canonical::expectedBlendShapePoolBucketDNABlockOffsets);
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.bucketVertexIndices, canonical::expectedBlendShapePoolBucketVertexIndices);
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.dnaBlocks, canonical::expectedBlendShapePoolDNADeltas);
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.archBlocks, canonical::expectedBlendShapePoolArchDeltas);
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.dnaIndices, canonical::expectedBlendShapePoolDNAIndices);

    auto skinWeightIndices = genePool.getSkinWeightJointIndices();
    const auto& expectedSkinWeightIndices = canonical::expectedSWPoolJointIndices;
    assertSkinWeightPoolJointIndices(skinWeightIndices, expectedSkinWeightIndices);

    const auto skinWeightPoolWeights = genePool.getSkinWeightValues();
    const auto& expectedSkinWeightPoolWeights = canonical::expectedSWPoolWeights;
    assertSkinWeightPoolValues(skinWeightPoolWeights, expectedSkinWeightPoolWeights);

    const auto& jointBehaviorInputIndices = genePool.getJointBehaviorInputIndices();
    const auto& expectedJointBehaviorInputIndices = canonical::expectedJBPoolInputIndices;
    assertJointBehaviorPoolIndices(jointBehaviorInputIndices, expectedJointBehaviorInputIndices);

    const auto& jointBehaviorOutputIndices = genePool.getJointBehaviorOutputIndices();
    const auto& expectedJointBehaviorOutputIndices = canonical::expectedJBPoolOutputIndices;
    assertJointBehaviorPoolIndices(jointBehaviorOutputIndices, expectedJointBehaviorOutputIndices);

    const auto& jointBehaviorLODs = genePool.getJointBehaviorLODs();
    const auto& expectedJointBehaviorLODs = canonical::expectedJBPoolLODs;
    assertJointBehaviorPoolIndices(jointBehaviorLODs, expectedJointBehaviorLODs);

    const auto& expectedJointBehaviorValues = canonical::expectedJBPoolBlock;
    auto actualJointBehaviorValues = genePool.getJointBehaviorValues();
    assertJointBehaviorValues(actualJointBehaviorValues, expectedJointBehaviorValues);
}

TEST_F(TestGenePool, DNAMismatchError) {
    auto otherReader = MockedArchetypeReader{};
    readers.push_back(&otherReader);
    auto genePool =
        GenePool{arch.get(), readers.data(), static_cast<std::uint16_t>(readers.size()), gs4::GenePoolMask::All, &memRes};

    ASSERT_EQ(Status::get(), GenePool::DNAMismatch);
    auto genePoolImpl = PImplExtractor<GenePool>::get(genePool);
    ASSERT_TRUE(genePoolImpl->getIsNullGenePool());

    readers.pop_back();
    genePool = GenePool{arch.get(), readers.data(), static_cast<std::uint16_t>(readers.size()), gs4::GenePoolMask::All, &memRes};
    ASSERT_EQ(Status::isOk(), true);

    genePoolImpl = PImplExtractor<GenePool>::get(genePool);
    ASSERT_FALSE(genePoolImpl->getIsNullGenePool());
}

TEST_F(TestGenePool, DNAsEmpty) {
    auto otherReader = MockedArchetypeReader{};
    readers.push_back(&otherReader);
    auto genePool = GenePool{arch.get(), nullptr, 0u, gs4::GenePoolMask::All, &memRes};

    ASSERT_EQ(Status::get(), GenePool::DNAsEmpty);
    auto genePoolImpl = PImplExtractor<GenePool>::get(genePool);
    ASSERT_TRUE(genePoolImpl->getIsNullGenePool());

    readers.pop_back();
    genePool = GenePool{arch.get(), readers.data(), static_cast<std::uint16_t>(readers.size()), gs4::GenePoolMask::All, &memRes};
    ASSERT_EQ(Status::isOk(), true);
    genePoolImpl = PImplExtractor<GenePool>::get(genePool);
    ASSERT_FALSE(genePoolImpl->getIsNullGenePool());
}

TEST_F(TestGenePool, GenePoolMaskNeutralMeshes) {
    auto mask = gs4::GenePoolMask::NeutralMeshes;
    auto genePool = GenePool{arch.get(), readers.data(), static_cast<std::uint16_t>(readers.size()), mask, &memRes};
    auto genePoolImpl = PImplExtractor<GenePool>::get(genePool);

    auto neutralMesh = genePoolImpl->getNeutralMeshes();
    const auto& expectedNeutralMesh = canonical::expectedNeutralMeshPoolValues;
    assertNeutralMeshPoolData(neutralMesh, expectedNeutralMesh);

    std::uint16_t meshCount = genePoolImpl->getMeshCount();
    ASSERT_EQ(meshCount, expectedReader->getMeshCount());
    for (std::uint16_t meshIndex = 0u; meshIndex < meshCount; meshIndex++) {
        ASSERT_EQ(genePoolImpl->getVertexCount(meshIndex), expectedReader->getVertexPositionCount(meshIndex));
        ASSERT_EQ(genePoolImpl->getBlendShapeTargetCount(meshIndex), 0u);
        ASSERT_EQ(genePoolImpl->getSkinWeightsCount(meshIndex), 0u);
        ASSERT_EQ(genePoolImpl->getMaximumInfluencesPerVertex(meshIndex), 0u);
    }
    ASSERT_EQ(genePoolImpl->getBlendShapeTargetVertexIndices().size(), 0u);
    auto blendShapeDeltas = genePoolImpl->getBlendShapeTargetDeltas();
    ASSERT_EQ(blendShapeDeltas.bucketOffsets.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.bucketVertexIndices.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.bucketDNABlockOffsets.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.archBlocks.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.dnaBlocks.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.dnaIndices.size(), 0u);
    ASSERT_EQ(genePoolImpl->getSkinWeightJointIndices().size(), 0u);
    ASSERT_EQ(genePoolImpl->getSkinWeightValues().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointCount(), expectedReader->getJointCount());
    ASSERT_EQ(genePoolImpl->getNeutralJoints(JointAttribute::Translation).size(), 0u);
    ASSERT_EQ(genePoolImpl->getNeutralJoints(JointAttribute::Rotation).size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointGroupCount(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorLODs().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorInputIndices().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorOutputIndices().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorValues().size(), 0u);
}

TEST_F(TestGenePool, GenePoolMaskBlendShapes) {
    auto mask = gs4::GenePoolMask::BlendShapes;
    auto genePool = GenePool{arch.get(), readers.data(), static_cast<std::uint16_t>(readers.size()), mask, &memRes};
    auto genePoolImpl = PImplExtractor<GenePool>::get(genePool);

    auto blendShapeIndices = genePoolImpl->getBlendShapeTargetVertexIndices();
    const auto& expectedBlendShapeIndices = canonical::expectedBlendShapePoolVertexIndices;
    assertBlendShapePoolVertexIndices(blendShapeIndices, expectedBlendShapeIndices);
    auto blendShapeDeltas = genePoolImpl->getBlendShapeTargetDeltas();
    ASSERT_EQ(canonical::expectedBlendShapePoolBucketOffsets.size(), blendShapeDeltas.bucketOffsets.rowCount());
    for (std::size_t mi = 0u; mi < blendShapeDeltas.bucketOffsets.rowCount(); mi++) {
        const auto& bucketOffset = blendShapeDeltas.bucketOffsets[mi];
        ASSERT_ELEMENTS_AND_SIZE_EQ(bucketOffset, canonical::expectedBlendShapePoolBucketOffsets[mi]);
    }
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.bucketDNABlockOffsets, canonical::expectedBlendShapePoolBucketDNABlockOffsets);
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.bucketVertexIndices, canonical::expectedBlendShapePoolBucketVertexIndices);
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.dnaBlocks, canonical::expectedBlendShapePoolDNADeltas);
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.archBlocks, canonical::expectedBlendShapePoolArchDeltas);
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.dnaIndices, canonical::expectedBlendShapePoolDNAIndices);

    std::uint16_t meshCount = genePoolImpl->getMeshCount();
    ASSERT_EQ(meshCount, expectedReader->getMeshCount());
    for (std::uint16_t meshIndex = 0u; meshIndex < meshCount; meshIndex++) {
        ASSERT_EQ(genePoolImpl->getVertexCount(meshIndex), expectedReader->getVertexPositionCount(meshIndex));
        ASSERT_EQ(genePoolImpl->getSkinWeightsCount(meshIndex), 0u);
        ASSERT_EQ(genePoolImpl->getMaximumInfluencesPerVertex(meshIndex), 0u);
    }
    ASSERT_EQ(genePoolImpl->getNeutralMeshes().size(), 0u);
    ASSERT_EQ(genePoolImpl->getSkinWeightJointIndices().size(), 0u);
    ASSERT_EQ(genePoolImpl->getSkinWeightValues().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointCount(), expectedReader->getJointCount());
    ASSERT_EQ(genePoolImpl->getNeutralJoints(JointAttribute::Translation).size(), 0u);
    ASSERT_EQ(genePoolImpl->getNeutralJoints(JointAttribute::Rotation).size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointGroupCount(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorLODs().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorInputIndices().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorOutputIndices().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorValues().size(), 0u);
}

TEST_F(TestGenePool, GenePoolMaskSkinWeights) {
    auto mask = gs4::GenePoolMask::SkinWeights;
    auto genePool = GenePool{arch.get(), readers.data(), static_cast<std::uint16_t>(readers.size()), mask, &memRes};
    auto genePoolImpl = PImplExtractor<GenePool>::get(genePool);

    auto skinWeightIndices = genePoolImpl->getSkinWeightJointIndices();
    const auto& expectedSkinWeightIndices = canonical::expectedSWPoolJointIndices;
    assertSkinWeightPoolJointIndices(skinWeightIndices, expectedSkinWeightIndices);

    const auto skinWeightPoolWeights = genePoolImpl->getSkinWeightValues();
    const auto& expectedSkinWeightPoolWeights = canonical::expectedSWPoolWeights;
    assertSkinWeightPoolValues(skinWeightPoolWeights, expectedSkinWeightPoolWeights);

    std::uint16_t meshCount = genePoolImpl->getMeshCount();
    ASSERT_EQ(meshCount, expectedReader->getMeshCount());
    for (std::uint16_t meshIndex = 0u; meshIndex < meshCount; meshIndex++) {
        ASSERT_EQ(genePoolImpl->getVertexCount(meshIndex), expectedReader->getVertexPositionCount(meshIndex));
        ASSERT_EQ(genePoolImpl->getBlendShapeTargetCount(meshIndex), 0u);
    }
    ASSERT_EQ(genePoolImpl->getNeutralMeshes().size(), 0u);
    ASSERT_EQ(genePoolImpl->getBlendShapeTargetVertexIndices().size(), 0u);
    auto blendShapeDeltas = genePoolImpl->getBlendShapeTargetDeltas();
    ASSERT_EQ(blendShapeDeltas.bucketOffsets.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.bucketVertexIndices.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.bucketDNABlockOffsets.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.archBlocks.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.dnaBlocks.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.dnaIndices.size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointCount(), expectedReader->getJointCount());
    ASSERT_EQ(genePoolImpl->getNeutralJoints(JointAttribute::Translation).size(), 0u);
    ASSERT_EQ(genePoolImpl->getNeutralJoints(JointAttribute::Rotation).size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointGroupCount(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorLODs().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorInputIndices().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorOutputIndices().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorValues().size(), 0u);
}

TEST_F(TestGenePool, GenePoolMaskNeutralJoint) {
    auto mask = gs4::GenePoolMask::NeutralJoints;
    auto genePool = GenePool{arch.get(), readers.data(), static_cast<std::uint16_t>(readers.size()), mask, &memRes};
    auto genePoolImpl = PImplExtractor<GenePool>::get(genePool);

    const auto& translations = genePoolImpl->getNeutralJoints(JointAttribute::Translation);
    const auto& expectedTranslations = canonical::expectedNeutralJointPoolTranslations;
    assertNeutralJointPool(translations, expectedTranslations);

    const auto& rotations = genePoolImpl->getNeutralJoints(JointAttribute::Rotation);
    const auto& expectedRotations = canonical::expectedNeutralJointPoolRotations;
    assertNeutralJointPool(rotations, expectedRotations);

    std::uint16_t meshCount = genePoolImpl->getMeshCount();
    ASSERT_EQ(meshCount, expectedReader->getMeshCount());
    for (std::uint16_t meshIndex = 0u; meshIndex < meshCount; meshIndex++) {
        ASSERT_EQ(genePoolImpl->getVertexCount(meshIndex), expectedReader->getVertexPositionCount(meshIndex));
        ASSERT_EQ(genePoolImpl->getBlendShapeTargetCount(meshIndex), 0u);
        ASSERT_EQ(genePoolImpl->getSkinWeightsCount(meshIndex), 0u);
        ASSERT_EQ(genePoolImpl->getMaximumInfluencesPerVertex(meshIndex), 0u);
    }
    ASSERT_EQ(genePoolImpl->getNeutralMeshes().size(), 0u);
    ASSERT_EQ(genePoolImpl->getBlendShapeTargetVertexIndices().size(), 0u);
    auto blendShapeDeltas = genePoolImpl->getBlendShapeTargetDeltas();
    ASSERT_EQ(blendShapeDeltas.bucketOffsets.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.bucketVertexIndices.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.bucketDNABlockOffsets.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.archBlocks.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.dnaBlocks.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.dnaIndices.size(), 0u);
    ASSERT_EQ(genePoolImpl->getSkinWeightJointIndices().size(), 0u);
    ASSERT_EQ(genePoolImpl->getSkinWeightValues().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointCount(), expectedReader->getJointCount());
    ASSERT_EQ(genePoolImpl->getJointGroupCount(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorLODs().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorInputIndices().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorOutputIndices().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointBehaviorValues().size(), 0u);
}

TEST_F(TestGenePool, GenePoolMaskJointBehavior) {
    auto mask = gs4::GenePoolMask::JointBehavior;
    auto genePool = GenePool{arch.get(), readers.data(), static_cast<std::uint16_t>(readers.size()), mask, &memRes};
    auto genePoolImpl = PImplExtractor<GenePool>::get(genePool);

    const auto& jointBehaviorInputIndices = genePoolImpl->getJointBehaviorInputIndices();
    const auto& expectedJointBehaviorInputIndices = canonical::expectedJBPoolInputIndices;
    assertJointBehaviorPoolIndices(jointBehaviorInputIndices, expectedJointBehaviorInputIndices);

    const auto& jointBehaviorOutputIndices = genePoolImpl->getJointBehaviorOutputIndices();
    const auto& expectedJointBehaviorOutputIndices = canonical::expectedJBPoolOutputIndices;
    assertJointBehaviorPoolIndices(jointBehaviorOutputIndices, expectedJointBehaviorOutputIndices);

    const auto& jointBehaviorLODs = genePoolImpl->getJointBehaviorLODs();
    const auto& expectedJointBehaviorLODs = canonical::expectedJBPoolLODs;
    assertJointBehaviorPoolIndices(jointBehaviorLODs, expectedJointBehaviorLODs);

    const auto& expectedJointBehaviorValues = canonical::expectedJBPoolBlock;
    auto actualJointBehaviorValues = genePoolImpl->getJointBehaviorValues();
    assertJointBehaviorValues(actualJointBehaviorValues, expectedJointBehaviorValues);

    std::uint16_t meshCount = genePoolImpl->getMeshCount();
    ASSERT_EQ(meshCount, expectedReader->getMeshCount());
    for (std::uint16_t meshIndex = 0u; meshIndex < meshCount; meshIndex++) {
        ASSERT_EQ(genePoolImpl->getVertexCount(meshIndex), expectedReader->getVertexPositionCount(meshIndex));
        ASSERT_EQ(genePoolImpl->getBlendShapeTargetCount(meshIndex), 0u);
        ASSERT_EQ(genePoolImpl->getSkinWeightsCount(meshIndex), 0u);
        ASSERT_EQ(genePoolImpl->getMaximumInfluencesPerVertex(meshIndex), 0u);
    }
    ASSERT_EQ(genePoolImpl->getNeutralMeshes().size(), 0u);
    ASSERT_EQ(genePoolImpl->getBlendShapeTargetVertexIndices().size(), 0u);
    auto blendShapeDeltas = genePoolImpl->getBlendShapeTargetDeltas();
    ASSERT_EQ(blendShapeDeltas.bucketOffsets.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.bucketVertexIndices.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.bucketDNABlockOffsets.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.archBlocks.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.dnaBlocks.size(), 0u);
    ASSERT_EQ(blendShapeDeltas.dnaIndices.size(), 0u);
    ASSERT_EQ(genePoolImpl->getSkinWeightJointIndices().size(), 0u);
    ASSERT_EQ(genePoolImpl->getSkinWeightValues().size(), 0u);
    ASSERT_EQ(genePoolImpl->getJointCount(), expectedReader->getJointCount());
    ASSERT_EQ(genePoolImpl->getNeutralJoints(JointAttribute::Translation).size(), 0u);
    ASSERT_EQ(genePoolImpl->getNeutralJoints(JointAttribute::Rotation).size(), 0u);
}

TEST_F(TestGenePool, GenePoolFromToStream) {
    auto stream = pma::makeScoped<trio::MemoryStream>();
    GenePool{arch.get(), readers.data(), static_cast<std::uint16_t>(readers.size())}.dump(stream.get());
    stream->seek(0u);
    auto genePool = GenePool{stream.get()};
    auto genePoolImpl = PImplExtractor<GenePool>::get(genePool);

    std::uint16_t meshCount = genePoolImpl->getMeshCount();
    ASSERT_EQ(meshCount, expectedReader->getMeshCount());
    auto neutralMesh = genePoolImpl->getNeutralMeshes();
    const auto& expectedNeutralMesh = canonical::expectedNeutralMeshPoolValues;
    assertNeutralMeshPoolData(neutralMesh, expectedNeutralMesh);

    auto blendShapeIndices = genePoolImpl->getBlendShapeTargetVertexIndices();
    const auto& expectedBlendShapeIndices = canonical::expectedBlendShapePoolVertexIndices;
    assertBlendShapePoolVertexIndices(blendShapeIndices, expectedBlendShapeIndices);
    auto blendShapeDeltas = genePoolImpl->getBlendShapeTargetDeltas();
    ASSERT_EQ(canonical::expectedBlendShapePoolBucketOffsets.size(), blendShapeDeltas.bucketOffsets.rowCount());
    for (std::size_t mi = 0u; mi < blendShapeDeltas.bucketOffsets.rowCount(); mi++) {
        const auto& bucketOffset = blendShapeDeltas.bucketOffsets[mi];
        ASSERT_ELEMENTS_AND_SIZE_EQ(bucketOffset, canonical::expectedBlendShapePoolBucketOffsets[mi]);
    }
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.bucketDNABlockOffsets, canonical::expectedBlendShapePoolBucketDNABlockOffsets);
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.bucketVertexIndices, canonical::expectedBlendShapePoolBucketVertexIndices);
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.dnaBlocks, canonical::expectedBlendShapePoolDNADeltas);
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.archBlocks, canonical::expectedBlendShapePoolArchDeltas);
    ASSERT_ELEMENTS_AND_SIZE_EQ(blendShapeDeltas.dnaIndices, canonical::expectedBlendShapePoolDNAIndices);

    auto skinWeightIndices = genePoolImpl->getSkinWeightJointIndices();
    const auto& expectedSkinWeightIndices = canonical::expectedSWPoolJointIndices;
    assertSkinWeightPoolJointIndices(skinWeightIndices, expectedSkinWeightIndices);

    const auto skinWeightPoolWeights = genePoolImpl->getSkinWeightValues();
    const auto& expectedSkinWeightPoolWeights = canonical::expectedSWPoolWeights;
    assertSkinWeightPoolValues(skinWeightPoolWeights, expectedSkinWeightPoolWeights);

    const auto& translations = genePoolImpl->getNeutralJoints(JointAttribute::Translation);
    const auto& expectedTranslations = canonical::expectedNeutralJointPoolTranslations;
    assertNeutralJointPool(translations, expectedTranslations);

    const auto& rotations = genePoolImpl->getNeutralJoints(JointAttribute::Rotation);
    const auto& expectedRotations = canonical::expectedNeutralJointPoolRotations;
    assertNeutralJointPool(rotations, expectedRotations);

    const auto& jointBehaviorInputIndices = genePoolImpl->getJointBehaviorInputIndices();
    const auto& expectedJointBehaviorInputIndices = canonical::expectedJBPoolInputIndices;
    assertJointBehaviorPoolIndices(jointBehaviorInputIndices, expectedJointBehaviorInputIndices);

    const auto& jointBehaviorOutputIndices = genePoolImpl->getJointBehaviorOutputIndices();
    const auto& expectedJointBehaviorOutputIndices = canonical::expectedJBPoolOutputIndices;
    assertJointBehaviorPoolIndices(jointBehaviorOutputIndices, expectedJointBehaviorOutputIndices);

    const auto& jointBehaviorLODs = genePoolImpl->getJointBehaviorLODs();
    const auto& expectedJointBehaviorLODs = canonical::expectedJBPoolLODs;
    assertJointBehaviorPoolIndices(jointBehaviorLODs, expectedJointBehaviorLODs);

    const auto& expectedJointBehaviorValues = canonical::expectedJBPoolBlock;
    auto actualJointBehaviorValues = genePoolImpl->getJointBehaviorValues();
    assertJointBehaviorValues(actualJointBehaviorValues, expectedJointBehaviorValues);
}

}  // namespace gs4

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
