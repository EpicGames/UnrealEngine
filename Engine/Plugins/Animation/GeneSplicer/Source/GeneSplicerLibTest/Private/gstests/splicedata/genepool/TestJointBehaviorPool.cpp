// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Assertions.h"
#include "gstests/splicedata/genepool/TestPool.h"

#include "genesplicer/splicedata/genepool/JointBehaviorPool.h"

namespace gs4 {

using TestJointBehaviorPool = TestPool;

TEST_F(TestJointBehaviorPool, InputIndices) {
    auto jointBehaviorPool = JointBehaviorPool{arch.get(), ConstArrayView<const dna::Reader*>{readers}, &memRes};
    const auto& expectedInputIndices = canonical::expectedJBPoolInputIndices;
    const auto& actualInputIndices = jointBehaviorPool.getInputIndices();
    assertJointBehaviorPoolIndices(actualInputIndices, expectedInputIndices);
}

TEST_F(TestJointBehaviorPool, OutputIndices) {
    auto jointBehaviorPool = JointBehaviorPool{arch.get(), ConstArrayView<const dna::Reader*>{readers}, &memRes};
    const auto& expectedOutputIndices = canonical::expectedJBPoolOutputIndices;
    const auto& actualOutputIndices = jointBehaviorPool.getOutputIndices();
    assertJointBehaviorPoolIndices(actualOutputIndices, expectedOutputIndices);
}

TEST_F(TestJointBehaviorPool, LODs) {
    auto jointBehaviorPool = JointBehaviorPool{arch.get(), ConstArrayView<const dna::Reader*>{readers}, &memRes};
    const auto& expectedLODs = canonical::expectedJBPoolLODs;
    const auto& actualLODs = jointBehaviorPool.getLODs();
    assertJointBehaviorPoolIndices(actualLODs, expectedLODs);
}

TEST_F(TestJointBehaviorPool, Blocks) {
    auto jointBehaviorPool = JointBehaviorPool{arch.get(), ConstArrayView<const dna::Reader*>{readers}, &memRes};
    const auto& expectedJointBehaviorValues = canonical::expectedJBPoolBlock;
    auto actualJointBehaviorValues = jointBehaviorPool.getJointValues();
    assertJointBehaviorValues(actualJointBehaviorValues, expectedJointBehaviorValues);
}

}  // namespace gs4
