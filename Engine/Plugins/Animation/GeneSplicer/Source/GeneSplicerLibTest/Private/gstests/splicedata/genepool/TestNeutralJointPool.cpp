// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Assertions.h"
#include "gstests/splicedata/genepool/TestPool.h"

#include "genesplicer/splicedata/genepool/NeutralJointPool.h"

namespace gs4 {

using TestNeutralJointPool = TestPool;

TEST_F(TestNeutralJointPool, Translations) {
    auto neutralJointPool = NeutralJointPool{arch.get(), ConstArrayView<const dna::Reader*>{readers}, &memRes};
    const auto& translations = neutralJointPool.getDNAData<JointAttribute::Translation>();
    const auto& expectedTranslations = canonical::expectedNeutralJointPoolTranslations;
    assertNeutralJointPool(translations, expectedTranslations);
}

TEST_F(TestNeutralJointPool, Rotations) {
    auto neutralJointPool = NeutralJointPool{arch.get(), ConstArrayView<const dna::Reader*>{readers}, &memRes};
    const auto& rotations = neutralJointPool.getDNAData<JointAttribute::Rotation>();
    const auto& expectedRotations = canonical::expectedNeutralJointPoolRotations;
    assertNeutralJointPool(rotations, expectedRotations);
}

}  // namespace gs4
