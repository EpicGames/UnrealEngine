// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Assertions.h"
#include "gstests/splicedata/genepool/TestPool.h"

#include "genesplicer/splicedata/genepool/NeutralMeshPool.h"

namespace gs4 {

using TestNeutralMeshPool = TestPool;

TEST_F(TestNeutralMeshPool, Values) {
    auto neutralMeshPool = NeutralMeshPool{arch.get(), ConstArrayView<const dna::Reader*>{readers}, &memRes};
    auto data = neutralMeshPool.getData();
    const auto& expectedData = canonical::expectedNeutralMeshPoolValues;
    assertNeutralMeshPoolData(data, expectedData);
}

}  // namespace gs4
