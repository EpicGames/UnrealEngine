// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Defs.h"
#include "gstests/splicedata/MockedRegionAffiliationReader.h"

#include "genesplicer/splicedata/genepool/OutputIndexTargetOffsets.h"
#include "genesplicer/types/Aliases.h"

#include <gtest/gtest.h>

#include <cstdint>

namespace gs4 {

TEST(OutputIndexTargetOffsets, Constructor) {
    AlignedMemoryResource memRes;
    std::uint16_t jointCount = 17u;
    OutputIndexTargetOffsets outputIndexTargetOffsets{jointCount, &memRes};

    const auto& targetOffsets = outputIndexTargetOffsets.get();
    ASSERT_EQ(jointCount, targetOffsets.rowCount());

    std::uint16_t outputIndicesPerJoint = 9u;
    ASSERT_EQ(outputIndicesPerJoint, targetOffsets.columnCount());

    for (std::size_t valueIdx = 0ul; valueIdx < targetOffsets.size(); valueIdx++) {
        ASSERT_EQ(targetOffsets.data()[valueIdx], 0u);
    }
}

TEST(OutputIndexTargetOffsets, MapJointGroup) {
    AlignedMemoryResource memRes;
    std::uint16_t jointCount = 17u;
    OutputIndexTargetOffsets outputIndexTargetOffsets{jointCount, &memRes};

    Vector<std::uint16_t> outputIndices{2u, 3u, 5u, 12u, 19u, 148u, 149u};
    Vector<std::uint16_t> targetOutputIndices{1u, 2u, 3u, 5u, 11u, 12u, 19u, 147u, 148, 149u};
    outputIndexTargetOffsets.mapJointGroup(ConstArrayView<std::uint16_t>{outputIndices},
                                           ConstArrayView<std::uint16_t>{targetOutputIndices});

    struct expected {
        std::uint16_t joint, outputPos;
        std::uint8_t value;
    };
    Vector<expected>
        expectedValues{{0u, 2u, 1u}, {0u, 3u, 2u}, {0u, 5u, 3u}, {1u, 3u, 5u}, {2u, 1u, 6u}, {16u, 4u, 8u}, {16u, 5u, 9u}};

    auto targetOffsets = outputIndexTargetOffsets.get();
    for (const auto& expected : expectedValues) {
        ASSERT_EQ(targetOffsets[expected.joint][expected.outputPos], expected.value);
    }

    auto expectNonZero = [&expectedValues](std::uint16_t jntIdx, std::uint16_t outPos) {
        for (const auto& expected : expectedValues) {
            if ((expected.joint == jntIdx) && (expected.outputPos == outPos)) {
                return true;
            }
        }
        return false;
    };

    for (std::uint16_t jntIdx = 0; jntIdx < targetOffsets.rowCount(); jntIdx++) {
        for (std::uint16_t outputPos = 0; outputPos < targetOffsets.columnCount(); outputPos++) {
            if (expectNonZero(jntIdx, outputPos)) {
                continue;
            } else {
                ASSERT_EQ(targetOffsets[jntIdx][outputPos], 0u);
            }
        }
    }
}

}  // namespace gs4
