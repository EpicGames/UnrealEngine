// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/FixtureReader.h"
#include "gstests/Fixtures.h"
#include "gstests/splicedata/MockedArchetypeReader.h"
#include "gstests/splicedata/MockedRegionAffiliationReader.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

class TestPool : public ::testing::Test {
protected:
    using ReaderVector = Vector<const dna::Reader*>;

protected:
    void SetUp() override {

        arch = pma::makeScoped<FixtureReader>(static_cast<std::uint16_t>(FixtureReader::archetype));
        expectedReader = pma::makeScoped<FixtureReader>(static_cast<std::uint16_t>(FixtureReader::expected));
        dna0 = pma::makeScoped<FixtureReader>(static_cast<std::uint16_t>(0u));
        dna1 = pma::makeScoped<FixtureReader>(static_cast<std::uint16_t>(1u));
        readers = ReaderVector{&memRes};
        readers.push_back(dna0.get());
        readers.push_back(dna1.get());
    }

protected:
    pma::AlignedMemoryResource memRes;
    pma::ScopedPtr<FixtureReader> arch;
    pma::ScopedPtr<FixtureReader> dna0;
    pma::ScopedPtr<FixtureReader> dna1;
    pma::ScopedPtr<FixtureReader> expectedReader;

    ReaderVector readers;
    ReaderVector readerOthers;
};

}  // namespace gs4
