// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Defs.h"

#include "genesplicer/splicedata/genepool/SingleJointBehavior.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/Block.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

class TestSingleJointBehavior : public ::testing::Test {

    void SetUp() override {
        dnaValues.push_back(ConstArrayView<float>{dna0Values});
        dnaValues.push_back(ConstArrayView<float>{dna1Values});
    }

protected:
    AlignedMemoryResource memRes;
    Vector<float>
        dna0Values{0.1f, 0.2f, 0.3f, 0.1f, 0.2f, 0.3f, 0.1f, 0.2f, 0.3f, 0.1f, 0.2f, 0.3f, 0.1f, 0.2f, 0.3f, 0.1f, 0.2f, 0.3f};
    Vector<float>
        dna1Values{0.3f, 0.2f, 0.1f, 0.3f, 0.2f, 0.1f, 0.3f, 0.2f, 0.1f, 0.3f, 0.2f, 0.1f, 0.3f, 0.2f, 0.1f, 0.3f, 0.2f, 0.1f};
    Vector<float>
        archValues{0.1f, 0.0f, 0.2f, 0.1f, 0.0f, 0.2f, 0.1f, 0.0f, 0.2f, 0.1f, 0.0f, 0.2f, 0.1f, 0.0f, 0.2f, 0.1f, 0.0f, 0.2f};
    Vector<VBlock<16u>> expectedValues{
        {0.0f, 0.2f, 0.1f, 0.0f, 0.2f, 0.1f, 0.0f, 0.2f, 0.1f, 0.0f, 0.2f, 0.1f, 0.0f, 0.2f, 0.1f, 0.0f},
        {0.2f, 0.2f, -0.1f, 0.2f, 0.2f, -0.1f, 0.2f, 0.2f, -0.1f, 0.2f, 0.2f, -0.1f, 0.2f, 0.2f, -0.1f, 0.2f},
        {0.2f, 0.1f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},
        {0.2f, -0.1f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}};
    std::uint16_t inputCount = 18u;
    Vector<ConstArrayView<float>> dnaValues;
};

TEST_F(TestSingleJointBehavior, constructor1) {
    SingleJointBehavior singleJointBehavior{&memRes};

    auto values = singleJointBehavior.getValues();
    ASSERT_EQ(values.size(), 9u);
    for (auto outPosValues : values) {
        ASSERT_EQ(outPosValues.size(), 0u);
    }

    ASSERT_EQ(singleJointBehavior.getTranslationCount(), 0u);

    auto actualOutputOffsets = singleJointBehavior.getOutputOffsets();
    ConstArrayView<std::uint8_t> expectedOutputOffsets{};
    ASSERT_ELEMENTS_AND_SIZE_EQ(actualOutputOffsets, expectedOutputOffsets);
}

TEST_F(TestSingleJointBehavior, constructor2) {
    SingleJointBehavior singleJointBehavior{&memRes};
    singleJointBehavior.setValues(inputCount,
                                  1u,
                                  ConstArrayView<float>{archValues},
                                  ConstArrayView<ConstArrayView<float>>{dnaValues});
    singleJointBehavior.setValues(inputCount,
                                  0u,
                                  ConstArrayView<float>{archValues},
                                  ConstArrayView<ConstArrayView<float>>{dnaValues});
    singleJointBehavior.setValues(inputCount,
                                  5u,
                                  ConstArrayView<float>{archValues},
                                  ConstArrayView<ConstArrayView<float>>{dnaValues});

    SingleJointBehavior otherSingleJointBehavior{singleJointBehavior, &memRes};

    auto originalValues = singleJointBehavior.getValues();
    auto actualValues = otherSingleJointBehavior.getValues();
    ASSERT_EQ(originalValues.size(), actualValues.size());

    for (std::size_t outPos = 0; outPos < originalValues.size(); outPos++) {
        ASSERT_EQ(originalValues[outPos].size(), actualValues[outPos].size());
        ASSERT_ELEMENTS_EQ(originalValues[outPos].data(), actualValues[outPos].data(), actualValues[outPos].size());
    }

    auto expectedOutputOffsets = singleJointBehavior.getOutputOffsets();
    auto actualOutputOffsets = otherSingleJointBehavior.getOutputOffsets();
    ASSERT_ELEMENTS_AND_SIZE_EQ(expectedOutputOffsets, actualOutputOffsets);

    ASSERT_EQ(singleJointBehavior.getTranslationCount(), singleJointBehavior.getTranslationCount());
}

TEST_F(TestSingleJointBehavior, constructor3) {
    SingleJointBehavior singleJointBehavior{&memRes};
    singleJointBehavior.setValues(inputCount,
                                  1u,
                                  ConstArrayView<float>{archValues},
                                  ConstArrayView<ConstArrayView<float>>{dnaValues});
    singleJointBehavior.setValues(inputCount,
                                  0u,
                                  ConstArrayView<float>{archValues},
                                  ConstArrayView<ConstArrayView<float>>{dnaValues});
    singleJointBehavior.setValues(inputCount,
                                  5u,
                                  ConstArrayView<float>{archValues},
                                  ConstArrayView<ConstArrayView<float>>{dnaValues});

    SingleJointBehavior tempSingleJointBehavior{singleJointBehavior, &memRes};
    SingleJointBehavior otherSingleJointBehavior{std::move(tempSingleJointBehavior), &memRes};

    auto originalValues = singleJointBehavior.getValues();
    auto actualValues = otherSingleJointBehavior.getValues();
    ASSERT_EQ(originalValues.size(), actualValues.size());

    for (std::size_t outPos = 0; outPos < originalValues.size(); outPos++) {
        ASSERT_EQ(originalValues[outPos].size(), actualValues[outPos].size());
        ASSERT_ELEMENTS_EQ(originalValues[outPos].data(), actualValues[outPos].data(), actualValues[outPos].size());
    }

    auto expectedOutputOffsets = singleJointBehavior.getOutputOffsets();
    auto actualOutputOffsets = otherSingleJointBehavior.getOutputOffsets();
    ASSERT_ELEMENTS_AND_SIZE_EQ(expectedOutputOffsets, actualOutputOffsets);

    ASSERT_EQ(singleJointBehavior.getTranslationCount(), singleJointBehavior.getTranslationCount());
}

TEST_F(TestSingleJointBehavior, setValues) {
    SingleJointBehavior singleJointBehavior{&memRes};
    singleJointBehavior.setValues(inputCount,
                                  1u,
                                  ConstArrayView<float>{archValues},
                                  ConstArrayView<ConstArrayView<float>>{dnaValues});
    singleJointBehavior.setValues(inputCount,
                                  0u,
                                  ConstArrayView<float>{archValues},
                                  ConstArrayView<ConstArrayView<float>>{dnaValues});
    singleJointBehavior.setValues(inputCount,
                                  5u,
                                  ConstArrayView<float>{archValues},
                                  ConstArrayView<ConstArrayView<float>>{dnaValues});

    auto actualValues = singleJointBehavior.getValues();
    ASSERT_EQ(actualValues[0u].size(), expectedValues.size());
    ASSERT_ELEMENTS_EQ(actualValues[0u].data(), expectedValues.data(), actualValues[0u].size());

    ASSERT_EQ(actualValues[1u].size(), expectedValues.size());
    ASSERT_ELEMENTS_EQ(actualValues[1u].data(), expectedValues.data(), actualValues[1u].size());

    ASSERT_EQ(actualValues[5u].size(), expectedValues.size());
    ASSERT_ELEMENTS_EQ(actualValues[5u].data(), expectedValues.data(), actualValues[5u].size());

    ASSERT_EQ(singleJointBehavior.getTranslationCount(), 2u);

    auto actualOutputOffsets = singleJointBehavior.getOutputOffsets();
    Vector<std::uint16_t> expectedOutputOffsets{0u, 1u, 5u};

    ASSERT_ELEMENTS_AND_SIZE_EQ(actualOutputOffsets, expectedOutputOffsets);

    singleJointBehavior.setValues(0u, 1u, ConstArrayView<float>{}, ConstArrayView<ConstArrayView<float>>{});
    actualValues = singleJointBehavior.getValues();
    ASSERT_EQ(actualValues[1u].size(), 0u);
}

}  // namespace gs4
