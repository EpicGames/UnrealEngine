// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Assertions.h"
#include "gstests/splicedata/genepool/TestPool.h"

#include "genesplicer/splicedata/genepool/SkinWeightPool.h"

namespace gs4 {

using TestSkinWeightPool = TestPool;

TEST_F(TestSkinWeightPool, JointIndices) {
    auto skinWeightsPool = SkinWeightPool{ConstArrayView<const dna::Reader*>{readers}, &memRes};
    const auto& indices = skinWeightsPool.getJointIndices();
    const auto& expectedIndices = canonical::expectedSWPoolJointIndices;
    assertSkinWeightPoolJointIndices(indices, expectedIndices);
}

TEST_F(TestSkinWeightPool, Weights) {
    auto skinWeightsPool = SkinWeightPool{ConstArrayView<const dna::Reader*>{readers}, &memRes};
    const auto& weights = skinWeightsPool.getWeights();
    const auto& expectedWeights = canonical::expectedSWPoolWeights;
    assertSkinWeightPoolValues(weights, expectedWeights);
}

}  // namespace gs4
