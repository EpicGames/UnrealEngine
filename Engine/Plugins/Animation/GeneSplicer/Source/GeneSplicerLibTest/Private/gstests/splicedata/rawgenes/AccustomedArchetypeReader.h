// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "gstests/FixtureReader.h"

#include "genesplicer/types/Aliases.h"

#include <cstdint>

namespace gs4 {

#if defined(__clang__) || defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wunused-parameter"
#endif
#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4100)
#endif

class AccustomedArchetypeReader : public FixtureReader {
public:
    explicit AccustomedArchetypeReader() :
        FixtureReader{FixtureReader::archetype} {
    }

    ConstArrayView<std::uint32_t> getBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                                   std::uint16_t blendShapeTargetIndex) const override {
        return ConstArrayView<std::uint32_t>{canonical::expectedBlendShapePoolVertexIndices[meshIndex][blendShapeTargetIndex]};
    }

    ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t,  // There is only one mesh
                                                     std::uint16_t blendShapeTargetIndex) const override {
        return ConstArrayView<float>{canonical::accustomedArchetypeBlendShapeDeltas[blendShapeTargetIndex][0ul]};
    }

    ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t,  // There is only one mesh
                                                     std::uint16_t blendShapeTargetIndex) const override {
        return ConstArrayView<float>{canonical::accustomedArchetypeBlendShapeDeltas[blendShapeTargetIndex][1ul]};
    }

    ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t,  // There is only one mesh
                                                     std::uint16_t blendShapeTargetIndex) const override {
        return ConstArrayView<float>{canonical::accustomedArchetypeBlendShapeDeltas[blendShapeTargetIndex][2ul]};
    }

    ConstArrayView<float> getJointGroupValues(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<float>{canonical::accustomedArchetypeJointGroupValues[jointGroupIndex]};
    }

    ConstArrayView<std::uint16_t> getJointGroupLODs(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{canonical::jointGroupLODs[FixtureReader::expected][jointGroupIndex]};
    }

    ConstArrayView<std::uint16_t> getJointGroupInputIndices(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{canonical::jointGroupInputIndices[jointGroupIndex]};
    }

    ConstArrayView<std::uint16_t> getJointGroupOutputIndices(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{canonical::jointGroupOutputIndices[FixtureReader::expected][jointGroupIndex]};
    }
};

class RawGeneArchetypeDNAReader : public FixtureReader {

public:
    explicit RawGeneArchetypeDNAReader() :
        FixtureReader{FixtureReader::archetype} {
    }

    ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t,  // There is only one mesh
                                                     std::uint16_t blendShapeTargetIndex) const override {
        return ConstArrayView<float>{canonical::accustomedArchetypeBlendShapeDeltas[blendShapeTargetIndex][0ul]};
    }

    ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t,  // There is only one mesh
                                                     std::uint16_t blendShapeTargetIndex) const override {
        return ConstArrayView<float>{canonical::accustomedArchetypeBlendShapeDeltas[blendShapeTargetIndex][1ul]};
    }

    ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t,  // There is only one mesh
                                                     std::uint16_t blendShapeTargetIndex) const override {
        return ConstArrayView<float>{canonical::accustomedArchetypeBlendShapeDeltas[blendShapeTargetIndex][2ul]};
    }
};

#if defined(__clang__) || defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif

#ifdef _MSC_VER
    #pragma warning(pop)
#endif

}  // namespace gs4
