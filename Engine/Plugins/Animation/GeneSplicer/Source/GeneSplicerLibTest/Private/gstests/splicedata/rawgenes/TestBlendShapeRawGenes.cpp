// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Assertions.h"
#include "gstests/FixtureReader.h"
#include "gstests/Fixtures.h"
#include "gstests/splicedata/rawgenes/AccustomedArchetypeReader.h"
#include "gstests/splicedata/rawgenes/TestRawGenes.h"

#include "genesplicer/splicedata/rawgenes/BlendShapeRawGenes.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/VariableWidthMatrix.h"

namespace gs4 {

TEST_F(TestRawGenes, BlendShapeRawGenesSet) {
    BlendShapeRawGenes blendShapeRawGenes{&memRes};
    ASSERT_EQ(blendShapeRawGenes.getBlendShapeTargets().size(), 0u);
    blendShapeRawGenes.set(arch.get());
    const auto& blendShapes = blendShapeRawGenes.getBlendShapeTargets();
    assertBlendShapeTargets(blendShapes, rawGenesArch.get());
}

TEST_F(TestRawGenes, BlendShapeRawGenesSetSet) {
    BlendShapeRawGenes blendShapeRawGenes{&memRes};
    ASSERT_EQ(blendShapeRawGenes.getBlendShapeTargets().size(), 0u);
    blendShapeRawGenes.set(dna0.get());
    ASSERT_NE(blendShapeRawGenes.getBlendShapeTargets().size(), 0u);
    blendShapeRawGenes.set(arch.get());
    const auto& blendShapes = blendShapeRawGenes.getBlendShapeTargets();
    assertBlendShapeTargets(blendShapes, rawGenesArch.get());
}

TEST_F(TestRawGenes, BlendShapeRawGenesAccustomize) {
    BlendShapeRawGenes blendShapeRawGenes{&memRes};

    Vector<VariableWidthMatrix<std::uint32_t>> blendShapeIndices{&memRes};
    blendShapeIndices.resize(arch->getMeshCount());
    for (std::uint16_t meshIdx = 0u; meshIdx < arch->getMeshCount(); meshIdx++) {
        for (std::uint16_t bsIdx = 0u; bsIdx < arch->getBlendShapeTargetCount(meshIdx); bsIdx++) {
            ConstArrayView<std::uint32_t> indices{canonical::blendShapeTargetVertexIndices[FixtureReader::expected][bsIdx]};
            blendShapeIndices[meshIdx].appendRow(indices);
        }
    }
    ConstArrayView<VariableWidthMatrix<std::uint32_t>> blendShapeIndicesAV{blendShapeIndices};
    blendShapeRawGenes.accustomize(blendShapeIndicesAV);
    ASSERT_EQ(blendShapeRawGenes.getBlendShapeTargets().size(), 0u);

    blendShapeRawGenes.set(arch.get());
    blendShapeRawGenes.accustomize(blendShapeIndicesAV);

    const auto& blendShapes = blendShapeRawGenes.getBlendShapeTargets();
    assertBlendShapeTargets(blendShapes, accustomedArch.get());
}

}  // namespace gs4
