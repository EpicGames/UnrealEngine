// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/splicedata/rawgenes/TestRawGenes.h"

#include "gstests/Assertions.h"
#include "gstests/Defs.h"
#include "gstests/FixtureReader.h"
#include "gstests/Fixtures.h"

#include "genesplicer/splicedata/rawgenes/JointBehaviorRawGenes.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#include <gtest/gtest.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#include <numeric>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif
namespace gs4 {

TEST_F(TestRawGenes, JointBehaviorRawGenesSet) {
    JointBehaviorRawGenes jointBehaviorRawGenes{&memRes};
    ASSERT_EQ(jointBehaviorRawGenes.getJointGroups().size(), 0u);
    jointBehaviorRawGenes.set(arch.get());
    auto jointGroups = jointBehaviorRawGenes.getJointGroups();
    assertJointBehavior(jointGroups, arch.get());
}

TEST_F(TestRawGenes, JointBehaviorRawGenesSetSet) {
    JointBehaviorRawGenes jointBehaviorRawGenes{&memRes};
    ASSERT_EQ(jointBehaviorRawGenes.getJointGroups().size(), 0u);
    jointBehaviorRawGenes.set(dna0.get());
    ASSERT_NE(jointBehaviorRawGenes.getJointGroups().size(), 0u);
    jointBehaviorRawGenes.set(arch.get());
    auto jointGroups = jointBehaviorRawGenes.getJointGroups();
    assertJointBehavior(jointGroups, arch.get());
}

TEST_F(TestRawGenes, AccustomizeJointGroup) {
    JointBehaviorRawGenes jointBehaviorRawGenes{&memRes};
    ASSERT_EQ(jointBehaviorRawGenes.getJointGroups().size(), 0u);
    jointBehaviorRawGenes.set(arch.get());

    std::uint16_t jointGroupIdx = 0u;
    ConstArrayView<std::uint16_t> outputIndicesOther{canonical::jointGroupOutputIndices[4u][jointGroupIdx]};
    ConstArrayView<std::uint16_t> LODsOther{canonical::jointGroupLODs[4u][jointGroupIdx]};

    jointBehaviorRawGenes.accustomizeJointGroup(outputIndicesOther, LODsOther, jointGroupIdx);
    auto jointGroups = jointBehaviorRawGenes.getJointGroups();

    const auto& actualJointGroup = jointGroups[jointGroupIdx];

    auto expectedInputIndices = accustomedArch->getJointGroupInputIndices(jointGroupIdx);
    ASSERT_ELEMENTS_AND_SIZE_EQ(expectedInputIndices, actualJointGroup.inputIndices);

    auto expectedOutputIndices = accustomedArch->getJointGroupOutputIndices(jointGroupIdx);
    ASSERT_ELEMENTS_AND_SIZE_EQ(expectedOutputIndices, actualJointGroup.outputIndices);

    auto expectedLODs = accustomedArch->getJointGroupLODs(jointGroupIdx);
    ASSERT_ELEMENTS_AND_SIZE_EQ(expectedLODs, actualJointGroup.lods);

    auto values = accustomedArch->getJointGroupValues(jointGroupIdx);
    ASSERT_ELEMENTS_AND_SIZE_EQ(values, actualJointGroup.values);
}

TEST_F(TestRawGenes, Accustomize) {
    JointBehaviorRawGenes jointBehaviorRawGenes{&memRes};

    VariableWidthMatrix<std::uint16_t> outputIndicesOther{&memRes};
    VariableWidthMatrix<std::uint16_t> LODsOther{&memRes};
    std::uint16_t expectedJointGroupCount = arch->getJointGroupCount();
    for (std::uint16_t jointGroupIdx = 0u; jointGroupIdx < expectedJointGroupCount; jointGroupIdx++) {
        outputIndicesOther.appendRow(ConstArrayView<std::uint16_t>{canonical::jointGroupOutputIndices[4u][jointGroupIdx]});
        LODsOther.appendRow(ConstArrayView<std::uint16_t>{canonical::jointGroupLODs[4u][jointGroupIdx]});
    }

    jointBehaviorRawGenes.accustomize(outputIndicesOther, LODsOther);
    ASSERT_EQ(jointBehaviorRawGenes.getJointGroups().size(), 0u);

    jointBehaviorRawGenes.set(arch.get());
    jointBehaviorRawGenes.accustomize(outputIndicesOther, LODsOther);

    auto jointGroups = jointBehaviorRawGenes.getJointGroups();
    assertJointBehavior(jointGroups, accustomedArch.get());
}

}  // namespace gs4
