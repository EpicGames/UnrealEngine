// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Assertions.h"
#include "gstests/Defs.h"
#include "gstests/FixtureReader.h"

#include "genesplicer/splicedata/rawgenes/JointGroupOutputIndicesMerger.h"

#include <cstdint>
#include <iterator>

namespace gs4 {

class TestJointGroupOutputIndicesMerger : public ::testing::Test {
protected:
    using ReaderVector = Vector<const dna::Reader*>;

protected:
    void SetUp() override {
        arch = makeScoped<FixtureReader>(static_cast<std::uint16_t>(FixtureReader::archetype));
        dna0 = makeScoped<FixtureReader>(static_cast<std::uint16_t>(0u));
        dna1 = makeScoped<FixtureReader>(static_cast<std::uint16_t>(1u));
        expectedDNA = makeScoped<FixtureReader>(static_cast<std::uint16_t>(FixtureReader::expected));
    }

protected:
    AlignedMemoryResource memRes;
    ScopedPtr<FixtureReader> expectedDNA;
    ScopedPtr<FixtureReader> arch;
    ScopedPtr<FixtureReader> dna0;
    ScopedPtr<FixtureReader> dna1;
};

TEST_F(TestJointGroupOutputIndicesMerger, merge) {
    std::uint16_t jointGroupIndex = 0u;
    auto jointIndices = arch->getJointGroupJointIndices(jointGroupIndex);
    JointGroupOutputIndicesMerger merger{jointIndices, &memRes};
    merger.add(dna0->getJointGroupOutputIndices(jointGroupIndex), dna0->getJointGroupLODs(jointGroupIndex));
    merger.add(dna1->getJointGroupOutputIndices(jointGroupIndex), dna1->getJointGroupLODs(jointGroupIndex));
    merger.add(arch->getJointGroupOutputIndices(jointGroupIndex), arch->getJointGroupLODs(jointGroupIndex));

    Vector<std::uint16_t> actualOutputIndices{&memRes};
    actualOutputIndices.resize(arch->getJointCount() * 9u);

    Vector<std::uint16_t> actualLODs{&memRes};
    actualLODs.resize(arch->getJointGroupLODs(jointGroupIndex).size());

    auto end = merger.merge(actualOutputIndices.begin(), actualLODs.end());
    actualOutputIndices.resize(static_cast<std::size_t>(std::distance(actualOutputIndices.begin(), end)));

    auto expectedOutputIndices = expectedDNA->getJointGroupOutputIndices(jointGroupIndex);
    auto expectedLODs = expectedDNA->getJointGroupLODs(jointGroupIndex);

    ASSERT_ELEMENTS_AND_SIZE_EQ(expectedOutputIndices, actualOutputIndices);
    ASSERT_ELEMENTS_AND_SIZE_EQ(expectedLODs, actualLODs);
}

}  // namespace gs4
