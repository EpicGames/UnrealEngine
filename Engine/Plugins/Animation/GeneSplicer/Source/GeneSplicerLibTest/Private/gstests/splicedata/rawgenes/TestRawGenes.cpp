// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/splicedata/rawgenes/TestRawGenes.h"

#include "gstests/Assertions.h"
#include "gstests/Defs.h"

#include "genesplicer/CalculationType.h"
#include "genesplicer/neutraljointsplicer/JointAttribute.h"
#include "genesplicer/splicedata/GenePool.h"
#include "genesplicer/splicedata/genepool/GenePoolImpl.h"
#include "genesplicer/splicedata/rawgenes/RawGenes.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365)
#endif
#include <cstdint>
#include <random>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 6326)
#endif

namespace gs4 {

TEST_F(TestRawGenes, Constructor) {
    RawGenes rawGenes{&memRes};
    std::random_device rd;
    std::mt19937 mt(rd());
    std::uniform_int_distribution<std::uint16_t> rand{};
    ASSERT_EQ(0u, rawGenes.getMeshCount());
    ASSERT_EQ(0u, rawGenes.getJointCount());
    ASSERT_EQ(0u, rawGenes.getVertexCount(rand(mt)));
    ASSERT_EQ(0u, rawGenes.getSkinWeightsCount(rand(mt)));
    ASSERT_EQ(0u, rawGenes.getNeutralMeshes().size());
    ASSERT_EQ(0u, rawGenes.getJointGroups().size());
    ASSERT_EQ(0u, rawGenes.getBlendShapeTargets().size());
    ASSERT_EQ(0u, rawGenes.getNeutralJoints(JointAttribute::Translation).size());
    ASSERT_EQ(0u, rawGenes.getNeutralJoints(JointAttribute::Rotation).size());
    ASSERT_EQ(0u, rawGenes.getSkinWeights().size());
}

TEST_F(TestRawGenes, IntegrationRawGenes) {
    RawGenes rawGenes{&memRes};
    rawGenes.set(arch.get());
    assertRawGenes(rawGenes, rawGenesArch.get());
}

TEST_F(TestRawGenes, IntegrationAccustomize) {
    RawGenes rawGenes{&memRes};
    rawGenes.set(arch.get());
    Vector<const Reader*> dnas{dna0.get(), dna1.get()};
    GenePoolImpl genePool{dna0.get(), {dnas.data(), dnas.size()}, gs4::GenePoolMask::All, &memRes};

    rawGenes.accustomize(&genePool);
    assertRawGenes(rawGenes, accustomedArch.get());
}

}  // namespace gs4

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
