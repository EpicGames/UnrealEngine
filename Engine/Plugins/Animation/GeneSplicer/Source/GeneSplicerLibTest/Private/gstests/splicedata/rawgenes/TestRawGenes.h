// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "gstests/Defs.h"
#include "gstests/FixtureReader.h"
#include "gstests/splicedata/rawgenes/AccustomedArchetypeReader.h"

#include "genesplicer/splicedata/rawgenes/JointBehaviorRawGenes.h"
#include "genesplicer/types/Aliases.h"
#include "genesplicer/types/VariableWidthMatrix.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#include <numeric>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {
class TestRawGenes : public ::testing::Test {
protected:
    using ReaderVector = Vector<const dna::Reader*>;

protected:
    void SetUp() override {
        arch = makeScoped<FixtureReader>(static_cast<std::uint16_t>(FixtureReader::archetype));
        dna0 = makeScoped<FixtureReader>(static_cast<std::uint16_t>(0u));
        dna1 = makeScoped<FixtureReader>(static_cast<std::uint16_t>(1u));
        expectedDNA = makeScoped<FixtureReader>(static_cast<std::uint16_t>(FixtureReader::expected));
        accustomedArch = makeScoped<AccustomedArchetypeReader>();
        rawGenesArch = makeScoped<RawGeneArchetypeDNAReader>();
    }

protected:
    AlignedMemoryResource memRes;
    ScopedPtr<FixtureReader> expectedDNA;
    ScopedPtr<FixtureReader> arch;
    ScopedPtr<FixtureReader> dna0;
    ScopedPtr<FixtureReader> dna1;
    ScopedPtr<AccustomedArchetypeReader> accustomedArch;
    ScopedPtr<RawGeneArchetypeDNAReader> rawGenesArch;
};

}  // namespace gs4
