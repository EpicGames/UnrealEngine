// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Assertions.h"
#include "gstests/Defs.h"
#include "gstests/Fixtures.h"
#include "gstests/splicedata/rawgenes/TestRawGenes.h"

#include "genesplicer/dna/Aliases.h"
#include "genesplicer/neutraljointsplicer/JointAttribute.h"
#include "genesplicer/splicedata/rawgenes/RawGenesUtils.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#include <numeric>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

class TestRawGenesUtils : public ::testing::Test {
protected:
    using ReaderVector = Vector<const dna::Reader*>;

protected:
    void SetUp() override {
        dna0 = makeScoped<FixtureReader>(static_cast<std::uint16_t>(0u));
    }

    RawJointGroup getJointGroupFromFixtures(std::uint16_t jointGroupIndex, std::uint16_t dnaIndex) {
        RawJointGroup jointGroup{&memRes};

        jointGroup.inputIndices.assign(canonical::jointGroupInputIndices[jointGroupIndex].begin(),
                                       canonical::jointGroupInputIndices[jointGroupIndex].end());
        jointGroup.outputIndices.assign(canonical::jointGroupOutputIndices[dnaIndex][jointGroupIndex].begin(),
                                        canonical::jointGroupOutputIndices[dnaIndex][jointGroupIndex].end());
        jointGroup.lods.assign(canonical::jointGroupLODs[dnaIndex][jointGroupIndex].begin(),
                               canonical::jointGroupLODs[dnaIndex][jointGroupIndex].end());
        jointGroup.values.assign(canonical::jointGroupValues[dnaIndex][jointGroupIndex].begin(),
                                 canonical::jointGroupValues[dnaIndex][jointGroupIndex].end());
        return jointGroup;
    }

protected:
    AlignedMemoryResource memRes;
    ScopedPtr<FixtureReader> dna0;
};

TEST_F(TestRawGenesUtils, getJointValuesForOutputIndex) {
    auto jointGroup = getJointGroupFromFixtures(0, 0);
    std::vector<std::pair<std::uint16_t, std::vector<float>>> expectedOutputValuesPair{{1u, {0.1f, 0.1f}},
                                                                                       {3u, {0.3f, 0.3f}},
                                                                                       {7u, {0.7f, 0.7f}},
                                                                                       {13u, {}}};

    for (auto expected : expectedOutputValuesPair) {
        auto actualValues = getJointValuesForOutputIndex(jointGroup, expected.first);
        ASSERT_ELEMENTS_AND_SIZE_EQ(expected.second, actualValues);
    }
}

TEST_F(TestRawGenesUtils, CopyJointGroupValues) {
    auto srcJointGroup = getJointGroupFromFixtures(1, 0);
    auto destJointGroup = getJointGroupFromFixtures(1, 1);
    std::vector<float> expectedValues{
        1.0f,
        1.0f,  // O21
        0.2f,
        0.2f,  // O22
        0.3f,
        0.3f  // O23
    };
    copyJointGroupValues(srcJointGroup, destJointGroup);
    ASSERT_ELEMENTS_AND_SIZE_EQ(expectedValues, destJointGroup.values);

    srcJointGroup = getJointGroupFromFixtures(1, 1);
    destJointGroup = getJointGroupFromFixtures(1, 0);
    expectedValues = std::vector<float>{
        0.1f,
        0.1f,  // O19
        0.5f,
        0.5f,  // O20
        0.1f,
        0.1f  // O21
    };
    copyJointGroupValues(srcJointGroup, destJointGroup);
    ASSERT_ELEMENTS_AND_SIZE_EQ(expectedValues, destJointGroup.values);
}

TEST_F(TestRawGenesUtils, getOutputIndicesIntroducedByLOD) {
    auto jointGroup = getJointGroupFromFixtures(0, 1);
    std::vector<std::pair<std::uint16_t, std::vector<uint16_t>>> expectedOutputLODPair{{0u, {11, 12, 13}},
                                                                                       {1u, {}},
                                                                                       {2u, {0, 1, 2, 6}}};

    for (auto expected : expectedOutputLODPair) {
        auto actualOutputIndices = getOutputIndicesIntroducedByLOD(ConstArrayView<std::uint16_t>{jointGroup.outputIndices},
                                                                   ConstArrayView<std::uint16_t>{jointGroup.lods},
                                                                   expected.first);
        ASSERT_ELEMENTS_AND_SIZE_EQ(expected.second, actualOutputIndices);
    }
}

TEST_F(TestRawGenesUtils, getNeutralMeshesFromDNA) {
    auto neutralMeshes = getNeutralMeshesFromDNA(dna0.get(), &memRes);
    assertNeutralMeshes({neutralMeshes.data(), neutralMeshes.size()}, dna0.get());
}

TEST_F(TestRawGenesUtils, getNeutralJointTranslationsFromDNA) {
    auto neutralTranslation = getNeutralJointsFromDNA<JointAttribute::Translation>(dna0.get(), &memRes);
    assertNeutralJointTranslation(neutralTranslation, dna0.get());
}

TEST_F(TestRawGenesUtils, getNeutralJointRotationsFromDNA) {
    auto neutralRotation = getNeutralJointsFromDNA<JointAttribute::Rotation>(dna0.get(), &memRes);
    assertNeutralJointRotation(neutralRotation, dna0.get());
}

TEST_F(TestRawGenesUtils, getSkinWeightsFromDNA) {
    auto skinWeights = getSkinWeightFromDNA(dna0.get(), &memRes);
    assertSkinWeights({skinWeights.data(), skinWeights.size()}, dna0.get());
}

}  // namespace gs4
