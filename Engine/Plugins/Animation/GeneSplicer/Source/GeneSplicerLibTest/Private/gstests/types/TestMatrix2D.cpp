// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/types/Matrix.h"
#include "gstests/Defs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

class TestMatrix2D : public ::testing::Test {
protected:
    AlignedMemoryResource memRes;
};

TEST_F(TestMatrix2D, Constructor0) {
    std::size_t expectedRowCount = 0u;
    std::size_t expectedColumnCount = 0u;
    std::size_t expectedSize = expectedRowCount * expectedColumnCount;
    Matrix2D<std::uint16_t> matrix{&memRes};

    ASSERT_EQ(matrix.rowCount(), expectedRowCount);
    ASSERT_EQ(matrix.columnCount(), expectedColumnCount);
    ASSERT_EQ(expectedSize, matrix.size());
}

TEST_F(TestMatrix2D, Constructor1) {
    std::size_t expectedRowCount = 3u;
    std::size_t expectedColumnCount = 12u;
    std::size_t expectedSize = expectedRowCount * expectedColumnCount;
    Matrix2D<std::uint16_t> matrix{expectedRowCount, expectedColumnCount, &memRes};

    ASSERT_EQ(matrix.rowCount(), expectedRowCount);
    ASSERT_EQ(matrix.columnCount(), expectedColumnCount);
    ASSERT_EQ(matrix.size(), expectedSize);

    Vector<std::uint16_t> expectedValues{expectedSize, {}, &memRes};
    ASSERT_ELEMENTS_EQ(expectedValues.data(), matrix.data(), expectedSize);
}

TEST_F(TestMatrix2D, Constructor2) {
    std::size_t expectedRowCount = 3u;
    std::size_t expectedColumnCount = 12u;
    std::size_t expectedSize = expectedRowCount * expectedColumnCount;
    std::uint16_t initialValue = 13u;
    Matrix2D<std::uint16_t> matrix{expectedRowCount, expectedColumnCount, initialValue, &memRes};

    ASSERT_EQ(matrix.rowCount(), expectedRowCount);
    ASSERT_EQ(matrix.columnCount(), expectedColumnCount);
    ASSERT_EQ(matrix.size(), expectedSize);

    Vector<std::uint16_t> expectedValues{expectedSize, initialValue, &memRes};
    ASSERT_ELEMENTS_EQ(expectedValues.data(), matrix.data(), expectedSize);
}

TEST_F(TestMatrix2D, Constructor3) {
    std::size_t expectedRowCount = 3u;
    std::size_t expectedColumnCount = 12u;
    std::size_t expectedSize = expectedRowCount * expectedColumnCount;
    std::uint16_t initialValue = 13u;
    Matrix2D<std::uint16_t> matrixTemp{expectedRowCount, expectedColumnCount, initialValue, &memRes};
    Matrix2D<std::uint16_t> matrix{std::move(matrixTemp), &memRes};

    ASSERT_EQ(matrix.rowCount(), expectedRowCount);
    ASSERT_EQ(matrix.columnCount(), expectedColumnCount);
    ASSERT_EQ(matrix.size(), expectedSize);

    Vector<std::uint16_t> expectedValues{expectedSize, initialValue, &memRes};
    ASSERT_ELEMENTS_EQ(expectedValues.data(), matrix.data(), expectedSize);
}

}  // namespace gs4
