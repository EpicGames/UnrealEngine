// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/types/Matrix.h"
#include "gstests/Defs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#include <numeric>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

class TestMatrix2DView : public ::testing::Test {
protected:
    AlignedMemoryResource memRes;
};

TEST_F(TestMatrix2DView, Constructor0) {
    std::size_t expectedRowCount = 10u;
    std::size_t expectedColumnCount = 13u;
    std::size_t expectedSize = expectedRowCount * expectedColumnCount;

    Vector<std::uint16_t> dataHolder{expectedSize, {}, &memRes};
    std::iota(dataHolder.begin(), dataHolder.end(), static_cast<std::uint16_t>(0u));
    std::uint16_t* expectedPtr = dataHolder.data();
    Matrix2DView<std::uint16_t> matrixView{dataHolder.data(), expectedRowCount, expectedColumnCount};

    ASSERT_EQ(matrixView.data(), expectedPtr);
    ASSERT_EQ(matrixView.rowCount(), expectedRowCount);
    ASSERT_EQ(matrixView.columnCount(), expectedColumnCount);
    ASSERT_EQ(matrixView.size(), expectedSize);

    for (std::uint16_t rowIndex = 0u; rowIndex < expectedRowCount; rowIndex++) {
        auto row = matrixView[rowIndex];
        ASSERT_ELEMENTS_EQ(row.data(), (dataHolder.data() + rowIndex * expectedColumnCount), expectedColumnCount);
    }
}

TEST_F(TestMatrix2DView, Constructor1) {
    std::size_t expectedRowCount = 7u;
    std::size_t expectedColumnCount = 5u;
    std::size_t expectedSize = expectedRowCount * expectedColumnCount;

    Matrix2D<std::uint16_t> matrix{expectedRowCount, expectedColumnCount, &memRes};
    std::iota(matrix.data(), matrix.data() + matrix.size(), static_cast<std::uint16_t>(0u));
    std::uint16_t* expectedPtr = matrix.data();
    Matrix2DView<std::uint16_t> matrixView{matrix};

    ASSERT_EQ(matrixView.data(), expectedPtr);
    ASSERT_EQ(matrixView.rowCount(), expectedRowCount);
    ASSERT_EQ(matrixView.columnCount(), expectedColumnCount);
    ASSERT_EQ(matrixView.size(), expectedSize);

    for (std::uint16_t rowIndex = 0u; rowIndex < expectedRowCount; rowIndex++) {
        ASSERT_EQ(matrixView[rowIndex], matrix[rowIndex]);
    }
}

TEST_F(TestMatrix2DView, Constructor2) {
    std::size_t expectedRowCount = 7u;
    std::size_t expectedColumnCount = 5u;
    std::size_t expectedSize = expectedRowCount * expectedColumnCount;

    Matrix2D<std::uint16_t> tempMatrix{expectedRowCount, expectedColumnCount, &memRes};
    std::iota(tempMatrix.data(), tempMatrix.data() + tempMatrix.size(), static_cast<std::uint16_t>(0u));
    const Matrix2D<std::uint16_t> matrix{tempMatrix};
    const std::uint16_t* expectedPtr = matrix.data();

    Matrix2DView<const std::uint16_t> matrixView{matrix};

    ASSERT_EQ(matrixView.data(), expectedPtr);
    ASSERT_EQ(matrixView.rowCount(), expectedRowCount);
    ASSERT_EQ(matrixView.columnCount(), expectedColumnCount);
    ASSERT_EQ(matrixView.size(), expectedSize);

    for (std::uint16_t rowIndex = 0u; rowIndex < expectedRowCount; rowIndex++) {
        ASSERT_EQ(matrixView[rowIndex], matrix[rowIndex]);
    }
}

TEST_F(TestMatrix2DView, Constructor3) {
    std::size_t expectedRowCount = 12u;
    std::size_t expectedColumnCount = 9u;
    std::size_t expectedSize = expectedRowCount * expectedColumnCount;

    Matrix2D<std::uint16_t> matrix{expectedRowCount, expectedColumnCount, &memRes};
    std::iota(matrix.data(), matrix.data() + matrix.size(), static_cast<std::uint16_t>(0u));
    std::uint16_t* expectedPtr = matrix.data();
    Matrix2DView<std::uint16_t> tempMatrixView{matrix};
    Matrix2DView<std::uint16_t> matrixView{tempMatrixView};

    ASSERT_EQ(matrixView.data(), expectedPtr);
    ASSERT_EQ(matrixView.rowCount(), expectedRowCount);
    ASSERT_EQ(matrixView.columnCount(), expectedColumnCount);
    ASSERT_EQ(matrixView.size(), expectedSize);

    for (std::uint16_t rowIndex = 0u; rowIndex < expectedRowCount; rowIndex++) {
        ASSERT_EQ(matrixView[rowIndex], matrix[rowIndex]);
    }
}
}  // namespace gs4
