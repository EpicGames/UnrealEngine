// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/types/Matrix.h"
#include "gstests/Defs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#include <numeric>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

class TestMatrix3D : public ::testing::Test {
protected:
    AlignedMemoryResource memRes;
};

TEST_F(TestMatrix3D, Constructor0) {
    std::size_t expectedZCount = 0u;
    std::size_t expectedYCount = 0u;
    std::size_t expectedXCount = 0u;
    std::size_t expectedSize = 0u;

    Matrix3D<std::uint16_t> matrix{&memRes};

    ASSERT_EQ(matrix.zCount(), expectedZCount);
    ASSERT_EQ(matrix.yCount(), expectedYCount);
    ASSERT_EQ(matrix.xCount(), expectedXCount);
    ASSERT_EQ(matrix.size(), expectedSize);
}

TEST_F(TestMatrix3D, Constructor1) {
    std::size_t expectedZCount = 4u;
    std::size_t expectedYCount = 7u;
    std::size_t expectedXCount = 3u;
    std::size_t expectedSize = expectedZCount * expectedYCount * expectedXCount;
    Matrix3D<std::uint16_t> matrix{expectedZCount, expectedYCount, expectedXCount, &memRes};

    ASSERT_EQ(matrix.zCount(), expectedZCount);
    ASSERT_EQ(matrix.yCount(), expectedYCount);
    ASSERT_EQ(matrix.xCount(), expectedXCount);
    ASSERT_EQ(matrix.size(), expectedSize);

    Vector<std::uint16_t> expectedValues{expectedSize, {}, &memRes};
    ASSERT_ELEMENTS_EQ(expectedValues.data(), matrix.data(), expectedSize);
}

TEST_F(TestMatrix3D, Constructor2) {
    std::size_t expectedZCount = 6u;
    std::size_t expectedYCount = 6u;
    std::size_t expectedXCount = 6u;
    std::size_t expectedSize = expectedZCount * expectedYCount * expectedXCount;
    Matrix3D<std::uint16_t> expectedMatrix{expectedZCount, expectedYCount, expectedXCount, &memRes};
    std::iota(expectedMatrix.data(), expectedMatrix.data() + expectedMatrix.size(), static_cast<std::uint16_t>(0u));
    Matrix3D<std::uint16_t> matrix{expectedMatrix, &memRes};

    ASSERT_EQ(matrix.zCount(), expectedZCount);
    ASSERT_EQ(matrix.yCount(), expectedYCount);
    ASSERT_EQ(matrix.xCount(), expectedXCount);
    ASSERT_EQ(matrix.size(), expectedSize);

    ASSERT_ELEMENTS_EQ(expectedMatrix.data(), matrix.data(), expectedSize);
}

TEST_F(TestMatrix3D, Constructor3) {
    std::size_t expectedZCount = 5u;
    std::size_t expectedYCount = 7u;
    std::size_t expectedXCount = 2u;
    std::size_t expectedSize = expectedZCount * expectedYCount * expectedXCount;
    Matrix3D<std::uint16_t> expectedMatrix{expectedZCount, expectedYCount, expectedXCount, &memRes};
    std::iota(expectedMatrix.data(), expectedMatrix.data() + expectedMatrix.size(), static_cast<std::uint16_t>(0u));
    Matrix3D<std::uint16_t> tempMatrix{expectedMatrix, &memRes};
    Matrix3D<std::uint16_t> matrix{std::move(tempMatrix), &memRes};

    ASSERT_EQ(matrix.zCount(), expectedZCount);
    ASSERT_EQ(matrix.yCount(), expectedYCount);
    ASSERT_EQ(matrix.xCount(), expectedXCount);
    ASSERT_EQ(matrix.size(), expectedSize);

    ASSERT_ELEMENTS_EQ(expectedMatrix.data(), matrix.data(), expectedSize);
}

TEST_F(TestMatrix3D, CopyAssignment) {
    std::size_t expectedZCount = 1u;
    std::size_t expectedYCount = 12u;
    std::size_t expectedXCount = 3u;
    std::size_t expectedSize = expectedZCount * expectedYCount * expectedXCount;
    Matrix3D<std::uint16_t> expectedMatrix{expectedZCount, expectedYCount, expectedXCount, &memRes};
    std::iota(expectedMatrix.data(), expectedMatrix.data() + expectedMatrix.size(), static_cast<std::uint16_t>(0u));
    Matrix3D<std::uint16_t> matrix{&memRes};
    matrix = expectedMatrix;

    ASSERT_EQ(matrix.zCount(), expectedZCount);
    ASSERT_EQ(matrix.yCount(), expectedYCount);
    ASSERT_EQ(matrix.xCount(), expectedXCount);
    ASSERT_EQ(matrix.size(), expectedSize);

    ASSERT_ELEMENTS_EQ(expectedMatrix.data(), matrix.data(), expectedSize);
}

TEST_F(TestMatrix3D, AccessOperator) {
    std::size_t expectedZCount = 4u;
    std::size_t expectedYCount = 12u;
    std::size_t expectedXCount = 3u;
    Matrix3D<std::uint16_t> matrix{expectedZCount, expectedYCount, expectedXCount, &memRes};
    std::iota(matrix.data(), matrix.data() + matrix.size(), static_cast<std::uint16_t>(0u));
    for (std::size_t x = 0; x < expectedZCount; x++) {
        std::size_t offset = x * expectedYCount * expectedXCount;
        auto expectedSlice = Matrix2DView<std::uint16_t>{matrix.data() + offset, expectedYCount, expectedXCount};
        auto actualSlice = matrix[x];
        ASSERT_EQ(actualSlice.data(), expectedSlice.data());
        ASSERT_EQ(actualSlice.size(), expectedSlice.size());
        ASSERT_EQ(actualSlice.columnCount(), expectedSlice.columnCount());
        ASSERT_EQ(actualSlice.rowCount(), expectedSlice.rowCount());
    }
}

}  // namespace gs4
