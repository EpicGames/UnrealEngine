// Copyright Epic Games, Inc. All Rights Reserved.

#include "genesplicer/types/VariableWidthMatrix.h"
#include "gstests/Defs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

class TestVariableWidthMatrix : public ::testing::Test {
public:
    using value_type = std::uint16_t;
    using const_slice_type = typename VariableWidthMatrix<value_type>::const_slice_type;
    using index_type = typename VariableWidthMatrix<value_type>::index_type;

public:
    void SetUp() {
    }

protected:
    AlignedMemoryResource memRes;
    VariableWidthMatrix<value_type> matrix{&memRes};
};

TEST_F(TestVariableWidthMatrix, Constructor0) {
    matrix = VariableWidthMatrix<value_type>{&memRes};
    ASSERT_EQ(matrix.rowCount(), 0u);
    ASSERT_EQ(matrix.size(), 0u);
}

TEST_F(TestVariableWidthMatrix, Constructor1) {
    matrix = VariableWidthMatrix<value_type>{&memRes};
    Vector<value_type> row0{1u, 2u, 3u, 4u};
    matrix.appendRow(const_slice_type{row0});
    matrix.appendRow(10u);
    matrix.appendRow(10u, value_type{});

    VariableWidthMatrix<value_type> actualMatrix{matrix, &memRes};

    ASSERT_EQ(actualMatrix.rowCount(), matrix.rowCount());
    ASSERT_EQ(actualMatrix.size(), matrix.size());

    for (index_type rowIndex = 0u; rowIndex < matrix.rowCount(); rowIndex++) {
        ASSERT_EQ(actualMatrix.columnCount(rowIndex), matrix.columnCount(rowIndex));
        auto actualRow = actualMatrix[rowIndex];
        auto expectedRow = matrix[rowIndex];
        ASSERT_EQ(actualRow.size(), expectedRow.size());
        ASSERT_ELEMENTS_EQ(actualRow, expectedRow, actualRow.size());
    }
}

TEST_F(TestVariableWidthMatrix, Constructor2) {
    matrix = VariableWidthMatrix<value_type>{&memRes};
    Vector<value_type> row0{1u, 2u, 3u, 4u};
    matrix.appendRow(const_slice_type{row0});
    matrix.appendRow(10u);
    matrix.appendRow(10u, value_type{});

    VariableWidthMatrix<value_type> expectedMatrix{matrix, &memRes};
    VariableWidthMatrix<value_type> actualMatrix{std::move(matrix), &memRes};

    ASSERT_EQ(actualMatrix.rowCount(), expectedMatrix.rowCount());
    ASSERT_EQ(actualMatrix.size(), expectedMatrix.size());

    for (index_type rowIndex = 0u; rowIndex < expectedMatrix.rowCount(); rowIndex++) {
        ASSERT_EQ(actualMatrix.columnCount(rowIndex), expectedMatrix.columnCount(rowIndex));
        auto actualRow = actualMatrix[rowIndex];
        auto expectedRow = expectedMatrix[rowIndex];
        ASSERT_EQ(actualRow.size(), expectedRow.size());
        ASSERT_ELEMENTS_EQ(actualRow, expectedRow, actualRow.size());
    }
}

TEST_F(TestVariableWidthMatrix, RowCount) {
    Vector<value_type> row0{1u, 2u, 3u, 4u};
    matrix.appendRow(const_slice_type{row0});
    matrix.appendRow(10u);
    matrix.appendRow(10u, value_type{});
    ASSERT_EQ(matrix.rowCount(), 3u);
}

TEST_F(TestVariableWidthMatrix, ColumnCount) {
    Vector<value_type> row0{1u, 2u, 3u, 4u};
    matrix.appendRow(const_slice_type{row0});
    index_type row0ColumnCount = row0.size();
    ASSERT_EQ(matrix.columnCount(0u), row0ColumnCount);

    index_type row1ColumnCount = 14u;
    matrix.appendRow(row1ColumnCount);
    ASSERT_EQ(matrix.columnCount(1u), row1ColumnCount);

    index_type row2ColumnCount = 3u;
    matrix.appendRow(row2ColumnCount, value_type{});
    ASSERT_EQ(matrix.columnCount(2u), row2ColumnCount);
}

TEST_F(TestVariableWidthMatrix, Size) {
    Vector<value_type> row0{1u, 2u, 3u, 4u};
    matrix.appendRow(const_slice_type{row0});
    index_type row0ColumnCount = row0.size();
    ASSERT_EQ(matrix.size(), row0ColumnCount);

    index_type row1ColumnCount = 14u;
    matrix.appendRow(row1ColumnCount);
    ASSERT_EQ(matrix.size(), row0ColumnCount + row1ColumnCount);

    index_type row2ColumnCount = 3u;
    matrix.appendRow(row2ColumnCount, value_type{});
    ASSERT_EQ(matrix.size(), row0ColumnCount + row1ColumnCount + row2ColumnCount);
}

TEST_F(TestVariableWidthMatrix, AppendRowConstSliceType) {
    Vector<value_type> row0{1u, 2u, 3u, 4u};
    Vector<value_type> row1{2u, 2u, 2u, 2u};
    Vector<value_type> row2{4u, 3u, 2u, 1u};
    matrix.appendRow(const_slice_type{row0});
    matrix.appendRow(const_slice_type{row1});
    matrix.appendRow(const_slice_type{row2});
    ASSERT_ELEMENTS_EQ(row0, matrix[0], row0.size());
    ASSERT_ELEMENTS_EQ(row1, matrix[1], row0.size());
    ASSERT_ELEMENTS_EQ(row2, matrix[2], row0.size());
}

TEST_F(TestVariableWidthMatrix, AppendRowIndexType) {
    Vector<value_type> zero_vector{10u, 0u, &memRes};

    index_type row0ColumnCount = 10u;
    matrix.appendRow(row0ColumnCount);
    index_type row1ColumnCount = 5u;
    matrix.appendRow(row1ColumnCount);
    index_type row2ColumnCount = 2u;
    matrix.appendRow(row2ColumnCount);

    ASSERT_ELEMENTS_EQ(zero_vector, matrix[0], row0ColumnCount);  // -V557
    ASSERT_ELEMENTS_EQ(zero_vector, matrix[1], row1ColumnCount);  // -V557
    ASSERT_ELEMENTS_EQ(zero_vector, matrix[2], row2ColumnCount);  // -V557
}

TEST_F(TestVariableWidthMatrix, Append) {
    Vector<value_type> zero_vector{10u, 0u, &memRes};

    index_type row0ColumnCount = 10u;
    matrix.appendRow(row0ColumnCount);
    index_type row1ColumnCount = 5u;
    matrix.appendRow(row1ColumnCount);
    index_type row2ColumnCount = 2u;
    matrix.appendRow(row2ColumnCount);

    index_type rowIndex = 1u;
    value_type expectedValue = 12u;
    matrix.append(rowIndex, expectedValue);

    auto actualRow = matrix[rowIndex];
    auto actualValue = actualRow[actualRow.size() - 1u];
    ASSERT_EQ(actualValue, expectedValue);
}

TEST_F(TestVariableWidthMatrix, Insert) {
    Vector<value_type> zero_vector{10u, 0u, &memRes};

    index_type row0ColumnCount = 10u;
    matrix.appendRow(row0ColumnCount);
    index_type row1ColumnCount = 5u;
    matrix.appendRow(row1ColumnCount);
    index_type row2ColumnCount = 2u;
    matrix.appendRow(row2ColumnCount);

    index_type rowIndex = 1u;
    index_type columnIndex = 1u;
    value_type expectedValue = 12u;
    matrix.insert(rowIndex, columnIndex, expectedValue);

    ASSERT_EQ(matrix[rowIndex][columnIndex], expectedValue);
}

TEST_F(TestVariableWidthMatrix, Clear) {
    index_type row0ColumnCount = 10u;
    matrix.appendRow(row0ColumnCount);
    index_type row1ColumnCount = 5u;
    matrix.appendRow(row1ColumnCount);
    index_type row2ColumnCount = 2u;
    matrix.appendRow(row2ColumnCount);
    ASSERT_EQ(matrix.rowCount(), 3u);
    matrix.clear();
    ASSERT_EQ(matrix.rowCount(), 0u);
}

}  // namespace gs4
