// Copyright Epic Games, Inc. All Rights Reserved.

#include "gstests/Defs.h"

#include "genesplicer/utils/Algorithm.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <memory>
#include <numeric>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace gs4 {

class TestAlgorithm : public ::testing::Test {

protected:
    pma::AlignedMemoryResource memRes;
};

TEST_F(TestAlgorithm, MergeIndices) {
    Vector<std::uint32_t> indicesA{1, 2, 3};
    Vector<std::uint32_t> indicesB{2, 3, 4};
    Vector<std::uint32_t> expectedIndices{1, 2, 3, 4};
    Vector<std::uint32_t> resultIndices{};
    resultIndices.resize(expectedIndices.size());
    auto onePastLastAdded = mergeIndices(ConstArrayView<std::uint32_t>{indicesA},
                                         ConstArrayView<std::uint32_t>{indicesB},
                                         4u,
                                         resultIndices.begin(),
                                         &memRes);
    ASSERT_EQ(onePastLastAdded, resultIndices.end());
    ASSERT_ELEMENTS_AND_SIZE_EQ(resultIndices, expectedIndices);
}

}  // namespace gs4
