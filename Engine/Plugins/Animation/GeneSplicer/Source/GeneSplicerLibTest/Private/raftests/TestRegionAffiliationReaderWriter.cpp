// Copyright Epic Games, Inc. All Rights Reserved.

#include "raftests/TestRegionAffiliationReaderWriter.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 6326)
#endif

using TReaderWriterPairs =
    ::testing::Types<std::tuple<raf::RegionAffiliationBinaryStreamReaderImpl, raf::RegionAffiliationBinaryStreamWriterImpl>,
                     std::tuple<raf::RegionAffiliationJSONStreamReaderImpl, raf::RegionAffiliationJSONStreamWriterImpl>>;
TYPED_TEST_SUITE(RegionAffiliationReaderWriterTest, TReaderWriterPairs, );

TYPED_TEST(RegionAffiliationReaderWriterTest, JointRegionIndices) {
    std::vector<std::uint16_t> expectedValues{0U, 1U, 2U};
    std::uint16_t jointIndex = 1U;
    this->writer->setJointRegionIndices(jointIndex, expectedValues.data(), static_cast<std::uint16_t>(expectedValues.size()));
    this->writer->write();

    this->reader->read();
    auto actualValues = this->reader->getJointRegionIndices(jointIndex);
    ASSERT_ELEMENTS_EQ(actualValues, expectedValues, expectedValues.size());
}

TYPED_TEST(RegionAffiliationReaderWriterTest, JointRegionAffiliation) {
    std::vector<float> expectedValues{0.1F, 0.5F, 0.4F};
    std::uint16_t jointIndex = 3U;

    this->writer->setJointRegionAffiliation(jointIndex, expectedValues.data(), static_cast<std::uint16_t>(expectedValues.size()));
    this->writer->write();

    this->reader->read();
    auto actualValues = this->reader->getJointRegionAffiliation(jointIndex);
    ASSERT_ELEMENTS_EQ(actualValues, expectedValues, expectedValues.size());
}

TYPED_TEST(RegionAffiliationReaderWriterTest, ClearJointRegionAffiliations) {
    std::vector<std::uint16_t> expectedRegionIndices1{1U, 3U, 4U};
    std::vector<float> expectedAffiliations1{0.3F, 0.4F, 0.3F};
    std::vector<std::uint16_t> expectedRegionIndices2{0U, 1U, 2U};
    std::vector<float> expectedAffiliations2{0.1F, 0.5F, 0.4F};
    std::uint16_t jointIndex1 = 3U;
    std::uint16_t jointIndex2 = 5U;

    this->writer->setJointRegionAffiliation(jointIndex1,
                                            expectedAffiliations1.data(),
                                            static_cast<std::uint16_t>(expectedAffiliations1.size()));
    this->writer->setJointRegionIndices(jointIndex1,
                                        expectedRegionIndices1.data(),
                                        static_cast<std::uint16_t>(expectedRegionIndices1.size()));
    this->writer->setJointRegionAffiliation(jointIndex2,
                                            expectedAffiliations2.data(),
                                            static_cast<std::uint16_t>(expectedAffiliations2.size()));
    this->writer->setJointRegionIndices(jointIndex2,
                                        expectedRegionIndices2.data(),
                                        static_cast<std::uint16_t>(expectedRegionIndices2.size()));
    this->writer->clearJointAffiliations();
    this->writer->write();
    this->reader->read();
    ASSERT_EQ(this->reader->getJointCount(), 0U);
    ASSERT_EQ(this->reader->getJointRegionAffiliation(jointIndex1).size(), 0U);
    ASSERT_EQ(this->reader->getJointRegionAffiliation(jointIndex2).size(), 0U);
    ASSERT_EQ(this->reader->getJointRegionIndices(jointIndex1).size(), 0U);
    ASSERT_EQ(this->reader->getJointRegionIndices(jointIndex2).size(), 0U);
}

TYPED_TEST(RegionAffiliationReaderWriterTest, DeleteJointRegionAffiliations) {
    std::vector<std::uint16_t> expectedRegionIndices1{1U, 3U, 4U};
    std::vector<float> expectedAffiliations1{0.3F, 0.4F, 0.3F};
    std::vector<std::uint16_t> expectedRegionIndices2{0U, 1U, 2U};
    std::vector<float> expectedAffiliations2{0.1F, 0.5F, 0.4F};
    std::uint16_t jointIndex1 = 3U;
    std::uint16_t jointIndex2 = 5U;
    std::uint16_t jointCount = 6u;

    this->writer->setJointRegionAffiliation(jointIndex1,
                                            expectedAffiliations1.data(),
                                            static_cast<std::uint16_t>(expectedAffiliations1.size()));
    this->writer->setJointRegionIndices(jointIndex1,
                                        expectedRegionIndices1.data(),
                                        static_cast<std::uint16_t>(expectedRegionIndices1.size()));
    this->writer->setJointRegionAffiliation(jointIndex2,
                                            expectedAffiliations2.data(),
                                            static_cast<std::uint16_t>(expectedAffiliations2.size()));
    this->writer->setJointRegionIndices(jointIndex2,
                                        expectedRegionIndices2.data(),
                                        static_cast<std::uint16_t>(expectedRegionIndices2.size()));
    this->writer->deleteJointAffiliation(jointIndex1);
    this->writer->write();
    this->reader->read();

    auto expectedJointCount = static_cast<std::uint16_t>(jointCount - 1U);
    ASSERT_EQ(this->reader->getJointCount(), expectedJointCount);
    if (jointIndex1 < jointIndex2) {
        jointIndex2--;
    }

    auto actualAffiliations = this->reader->getJointRegionAffiliation(jointIndex2);
    auto actualRegionIndices = this->reader->getJointRegionIndices(jointIndex2);
    ASSERT_ELEMENTS_EQ(actualAffiliations, expectedAffiliations2, expectedAffiliations2.size());
    ASSERT_ELEMENTS_EQ(actualRegionIndices, expectedRegionIndices2, expectedRegionIndices2.size());
}

TYPED_TEST(RegionAffiliationReaderWriterTest, VertexRegionIndices) {
    std::vector<std::uint16_t> expectedValues{0U, 1U, 2U};
    std::uint16_t meshIndex = 0U;
    std::uint32_t vertexIndex = 3U;
    this->writer->setVertexRegionIndices(meshIndex,
                                         vertexIndex,
                                         expectedValues.data(),
                                         static_cast<std::uint16_t>(expectedValues.size()));
    this->writer->write();

    this->reader->read();
    auto actualValues = this->reader->getVertexRegionIndices(meshIndex, vertexIndex);
    ASSERT_ELEMENTS_EQ(actualValues, expectedValues, expectedValues.size());
}

TYPED_TEST(RegionAffiliationReaderWriterTest, VertexRegionAffiliation) {
    std::vector<float> expectedValues{0.1F, 0.5F, 0.4F};
    std::uint16_t meshIndex = 0U;
    std::uint32_t vertexIndex = 3U;
    this->writer->setVertexRegionAffiliation(meshIndex,
                                             vertexIndex,
                                             expectedValues.data(),
                                             static_cast<std::uint16_t>(expectedValues.size()));
    this->writer->write();

    this->reader->read();
    auto actualValues = this->reader->getVertexRegionAffiliation(meshIndex, vertexIndex);
    ASSERT_ELEMENTS_EQ(actualValues, expectedValues, expectedValues.size());
}

TYPED_TEST(RegionAffiliationReaderWriterTest, ClearVertexRegionAffiliations) {
    std::vector<std::uint16_t> expectedRegionIndices1{1U, 3U, 4U};
    std::vector<float> expectedAffiliations1{0.3F, 0.4F, 0.3F};
    std::vector<std::uint16_t> expectedRegionIndices2{0U, 1U, 2U};
    std::vector<float> expectedAffiliations2{0.1F, 0.5F, 0.4F};
    std::uint16_t meshIndex1 = 0u;
    std::uint16_t meshIndex2 = 3u;
    std::uint32_t vertexIndex1 = 3U;
    std::uint32_t vertexIndex2 = 5U;

    this->writer->setVertexRegionAffiliation(meshIndex1,
                                             vertexIndex1,
                                             expectedAffiliations1.data(),
                                             static_cast<std::uint16_t>(expectedAffiliations1.size()));
    this->writer->setVertexRegionIndices(meshIndex1,
                                         vertexIndex1,
                                         expectedRegionIndices1.data(),
                                         static_cast<std::uint16_t>(expectedRegionIndices1.size()));
    this->writer->setVertexRegionAffiliation(meshIndex2,
                                             vertexIndex2,
                                             expectedAffiliations2.data(),
                                             static_cast<std::uint16_t>(expectedAffiliations2.size()));
    this->writer->setVertexRegionIndices(meshIndex2,
                                         vertexIndex2,
                                         expectedRegionIndices2.data(),
                                         static_cast<std::uint16_t>(expectedRegionIndices2.size()));
    this->writer->clearVertexAffiliations();
    this->writer->write();
    this->reader->read();
    ASSERT_EQ(this->reader->getMeshCount(), 0U);
    ASSERT_EQ(this->reader->getVertexRegionAffiliation(meshIndex1, vertexIndex1).size(), 0U);
    ASSERT_EQ(this->reader->getVertexRegionAffiliation(meshIndex2, vertexIndex2).size(), 0U);
    ASSERT_EQ(this->reader->getVertexRegionIndices(meshIndex2, vertexIndex2).size(), 0U);
    ASSERT_EQ(this->reader->getVertexRegionIndices(meshIndex2, vertexIndex2).size(), 0U);
}

TYPED_TEST(RegionAffiliationReaderWriterTest, ClearVertexRegionAffiliationsByMeshIndex) {
    std::vector<std::uint16_t> expectedRegionIndices1{1U, 3U, 4U};
    std::vector<float> expectedAffiliations1{0.3F, 0.4F, 0.3F};
    std::vector<std::uint16_t> expectedRegionIndices2{0U, 1U, 2U};
    std::vector<float> expectedAffiliations2{0.1F, 0.5F, 0.4F};
    std::uint16_t meshIndex1 = 0u;
    std::uint16_t meshIndex2 = 3u;
    std::uint32_t vertexIndex1 = 3U;
    std::uint32_t vertexIndex2 = 5U;

    this->writer->setVertexRegionAffiliation(meshIndex1,
                                             vertexIndex1,
                                             expectedAffiliations1.data(),
                                             static_cast<std::uint16_t>(expectedAffiliations1.size()));
    this->writer->setVertexRegionIndices(meshIndex1,
                                         vertexIndex1,
                                         expectedRegionIndices1.data(),
                                         static_cast<std::uint16_t>(expectedRegionIndices1.size()));
    this->writer->setVertexRegionAffiliation(meshIndex2,
                                             vertexIndex2,
                                             expectedAffiliations2.data(),
                                             static_cast<std::uint16_t>(expectedAffiliations2.size()));
    this->writer->setVertexRegionIndices(meshIndex2,
                                         vertexIndex2,
                                         expectedRegionIndices2.data(),
                                         static_cast<std::uint16_t>(expectedRegionIndices2.size()));
    this->writer->clearVertexAffiliations(meshIndex1);
    this->writer->write();
    this->reader->read();
    ASSERT_EQ(this->reader->getMeshCount(), 4U);

    ASSERT_EQ(this->reader->getVertexRegionAffiliation(meshIndex1, vertexIndex1).size(), 0U);
    ASSERT_EQ(this->reader->getVertexRegionIndices(meshIndex1, vertexIndex1).size(), 0U);

    ASSERT_ELEMENTS_EQ(this->reader->getVertexRegionAffiliation(meshIndex2, vertexIndex2),
                       expectedAffiliations2,
                       expectedAffiliations2.size());
    ASSERT_ELEMENTS_EQ(this->reader->getVertexRegionIndices(meshIndex2, vertexIndex2),
                       expectedRegionIndices2,
                       expectedRegionIndices2.size());
}

TYPED_TEST(RegionAffiliationReaderWriterTest, DeleteVertexRegionAffiliations) {

    std::vector<std::uint16_t> expectedRegionIndices1{1U, 3U, 4U};
    std::vector<float> expectedAffiliations1{0.3F, 0.4F, 0.3F};
    std::vector<std::uint16_t> expectedRegionIndices2{0U, 1U, 2U};
    std::vector<float> expectedAffiliations2{0.1F, 0.5F, 0.4F};
    std::uint16_t meshIndex = 3u;
    std::uint32_t vertexIndex1 = 3U;
    std::uint32_t vertexIndex2 = 5U;
    std::uint32_t vertexCount = 6U;

    this->writer->setVertexRegionAffiliation(meshIndex,
                                             vertexIndex1,
                                             expectedAffiliations1.data(),
                                             static_cast<std::uint16_t>(expectedAffiliations1.size()));
    this->writer->setVertexRegionIndices(meshIndex,
                                         vertexIndex1,
                                         expectedRegionIndices1.data(),
                                         static_cast<std::uint16_t>(expectedRegionIndices1.size()));
    this->writer->setVertexRegionAffiliation(meshIndex,
                                             vertexIndex2,
                                             expectedAffiliations2.data(),
                                             static_cast<std::uint16_t>(expectedAffiliations2.size()));
    this->writer->setVertexRegionIndices(meshIndex,
                                         vertexIndex2,
                                         expectedRegionIndices2.data(),
                                         static_cast<std::uint16_t>(expectedRegionIndices2.size()));
    this->writer->deleteVertexAffiliation(meshIndex, vertexIndex1);
    this->writer->write();
    this->reader->read();
    std::uint32_t expectedVertexCount = vertexCount - 1;
    ASSERT_EQ(this->reader->getMeshCount(), 4U);
    if (vertexIndex1 < vertexIndex2) {
        vertexIndex2--;
    }
    ASSERT_EQ(this->reader->getVertexCount(meshIndex), expectedVertexCount);

    ASSERT_ELEMENTS_EQ(this->reader->getVertexRegionAffiliation(meshIndex, vertexIndex2),
                       expectedAffiliations2,
                       expectedAffiliations2.size());
    ASSERT_ELEMENTS_EQ(this->reader->getVertexRegionIndices(meshIndex, vertexIndex2),
                       expectedRegionIndices2,
                       expectedRegionIndices2.size());
}

TYPED_TEST(RegionAffiliationReaderWriterTest, RegionNames) {
    std::vector<std::string> expectedValues{"R1", "R2", "R3"};
    this->writer->setRegionName(0, "R1");
    this->writer->setRegionName(1, "R2");
    this->writer->setRegionName(2, "R3");
    this->writer->write();

    this->reader->read();
    const auto regionCount = this->reader->getRegionCount();
    ASSERT_EQ(regionCount, expectedValues.size());

    for (std::uint16_t i = {}; i < regionCount; ++i) {
        const auto regionName = this->reader->getRegionName(i);
        ASSERT_EQ(regionName, raf::StringView{expectedValues[i]});
    }
}

TYPED_TEST(RegionAffiliationReaderWriterTest, ClearRegionNames) {
    std::vector<std::string> expectedValues{"R1", "R2", "R3"};
    this->writer->setRegionName(0, "R1");
    this->writer->setRegionName(1, "R2");
    this->writer->setRegionName(2, "R3");
    this->writer->clearRegionNames();
    this->writer->write();
    this->reader->read();
    ASSERT_EQ(this->reader->getRegionCount(), 0ul);
}

TYPED_TEST(RegionAffiliationReaderWriterTest, SetFrom) {
    std::vector<std::string> expectedRegionNames{"R1", "R2", "R3", "R3", "R4", "R5"};
    this->writer->setRegionName(0, "R1");
    this->writer->setRegionName(1, "R2");
    this->writer->setRegionName(2, "R3");
    this->writer->setRegionName(3, "R4");
    this->writer->setRegionName(4, "R5");

    std::vector<std::uint16_t> expectedRegionIndices1{1U, 3U, 4U};
    std::vector<float> expectedAffiliations1{0.3F, 0.4F, 0.3F};
    std::vector<std::uint16_t> expectedRegionIndices2{0U, 1U, 2U};
    std::vector<float> expectedAffiliations2{0.1F, 0.5F, 0.4F};
    std::uint16_t meshIndex1 = 0u;
    std::uint16_t meshIndex2 = 3u;
    std::uint32_t vertexIndex1 = 3U;
    std::uint32_t vertexIndex2 = 5U;

    this->writer->setVertexRegionAffiliation(meshIndex1,
                                             vertexIndex1,
                                             expectedAffiliations1.data(),
                                             static_cast<std::uint16_t>(expectedAffiliations1.size()));
    this->writer->setVertexRegionIndices(meshIndex1,
                                         vertexIndex1,
                                         expectedRegionIndices1.data(),
                                         static_cast<std::uint16_t>(expectedRegionIndices1.size()));
    this->writer->setVertexRegionAffiliation(meshIndex2,
                                             vertexIndex2,
                                             expectedAffiliations2.data(),
                                             static_cast<std::uint16_t>(expectedAffiliations2.size()));
    this->writer->setVertexRegionIndices(meshIndex2,
                                         vertexIndex2,
                                         expectedRegionIndices2.data(),
                                         static_cast<std::uint16_t>(expectedRegionIndices2.size()));
    std::vector<std::uint16_t> expectedJointRegionIndices1{1U, 3U, 4U};
    std::vector<float> expectedJointAffiliations1{0.3F, 0.4F, 0.3F};
    std::vector<std::uint16_t> expectedJointRegionIndices2{0U, 1U, 2U};
    std::vector<float> expectedJointAffiliations2{0.1F, 0.5F, 0.4F};
    std::uint16_t jointIndex1 = 3U;
    std::uint16_t jointIndex2 = 5U;

    this->writer->setJointRegionAffiliation(jointIndex1,
                                            expectedJointAffiliations1.data(),
                                            static_cast<std::uint16_t>(expectedJointAffiliations1.size()));
    this->writer->setJointRegionIndices(jointIndex1,
                                        expectedJointRegionIndices1.data(),
                                        static_cast<std::uint16_t>(expectedJointRegionIndices1.size()));
    this->writer->setJointRegionAffiliation(jointIndex2,
                                            expectedJointAffiliations2.data(),
                                            static_cast<std::uint16_t>(expectedJointAffiliations2.size()));
    this->writer->setJointRegionIndices(jointIndex2,
                                        expectedJointRegionIndices2.data(),
                                        static_cast<std::uint16_t>(expectedJointRegionIndices2.size()));

    this->writer->write();
    this->reader->read();

    using StreamWriterType = typename TestFixture::TStreamWriter;
    auto actualStream = pma::makeScoped<trio::MemoryStream>();
    auto actualWriter = pma::makeScoped<StreamWriterType>(actualStream.get());
    actualWriter->setFrom(this->reader.get());
    actualWriter->write();
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wuseless-cast"
#endif
    const auto expectedStreamSize = static_cast<std::size_t>(this->stream->size());
    ASSERT_EQ(static_cast<std::size_t>(actualStream->size()), expectedStreamSize);
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif

    std::vector<char> expectedValues{};
    std::vector<char> actualValues{};
    actualValues.resize(expectedStreamSize);
    expectedValues.resize(expectedStreamSize);

    actualStream->seek(0u);
    this->stream->seek(0u);

    actualStream->read(actualValues.data(), expectedStreamSize);
    this->stream->read(expectedValues.data(), expectedStreamSize);

    ASSERT_ELEMENTS_EQ(actualValues, expectedValues, expectedStreamSize);
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
