// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "raftests/Defs.h"

#include "raf/RegionAffiliationBinaryStreamReaderImpl.h"
#include "raf/RegionAffiliationBinaryStreamWriterImpl.h"
#include "raf/RegionAffiliationJSONStreamReaderImpl.h"
#include "raf/RegionAffiliationJSONStreamWriterImpl.h"

#include <pma/ScopedPtr.h>
#include <trio/streams/MemoryStream.h>

#include <tuple>
#include <type_traits>

template<typename TReaderWriterPair>
class RegionAffiliationReaderWriterTest : public ::testing::Test {
protected:
    using TStreamReaderImpl = typename std::tuple_element<0, TReaderWriterPair>::type;
    using TStreamWriterImpl = typename std::tuple_element<1, TReaderWriterPair>::type;
    using TStreamReader = typename TStreamReaderImpl::ReaderInterface;
    using TStreamWriter = typename TStreamWriterImpl::WriterInterface;

protected:
    void SetUp() override {
        stream = pma::makeScoped<trio::MemoryStream>();
        reader = pma::makeScoped<TStreamReader>(stream.get());
        writer = pma::makeScoped<TStreamWriter>(stream.get());
    }

protected:
    pma::ScopedPtr<trio::MemoryStream> stream;
    pma::ScopedPtr<TStreamReader> reader;
    pma::ScopedPtr<TStreamWriter> writer;
};
