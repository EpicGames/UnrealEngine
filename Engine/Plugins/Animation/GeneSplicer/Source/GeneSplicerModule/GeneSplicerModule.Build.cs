// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GeneSplicerModule : ModuleRules
	{
		public GeneSplicerModule(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"RigLogicModule",
					"Projects"
				} 
			);

			if (Target.Type == TargetType.Editor)
			{
				PublicDependencyModuleNames.Add("UnrealEd");
				PublicDependencyModuleNames.Add("MessageLog");
			}

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"RigLogicModule",
					"RigLogicLib",
					"GeneSplicerLib"
				}
			 );
		}
	}
}
