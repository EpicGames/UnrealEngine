// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenePool.h"

#include "DNAReader.h"
#include "FMemoryResource.h"
#include "ArchiveMemoryStream.h"


#include "genesplicer/splicedata/GenePool.h"
#include "riglogic/RigLogic.h"
#include "genesplicer/GeneSplicer.h"



static TUniquePtr<gs4::GenePool> CreateGenePool(const IDNAReader* DeltaArchetype, TArrayView<IDNAReader*> DNAs, EGenePoolMask GenePoolMask)
{
	TArray<const dna::Reader*, TInlineAllocator<512>> RawDNAs;
	RawDNAs.Reserve(DNAs.Num());
	for (IDNAReader* Reader : DNAs)
	{
		RawDNAs.Add(Reader->Unwrap());
	}
	return MakeUnique<gs4::GenePool>(DeltaArchetype->Unwrap(), RawDNAs.GetData(), static_cast<std::uint16_t>(RawDNAs.Num()), static_cast<gs4::GenePoolMask>(GenePoolMask), FMemoryResource::Instance());
}

static TUniquePtr<gs4::GenePool> CreateGenePool(const FString& Path, EGenePoolMask genePoolMask)
{
	auto GenePoolFileStream = rl4::makeScoped<rl4::FileStream>(TCHAR_TO_UTF8(*Path), rl4::FileStream::AccessMode::Read, rl4::FileStream::OpenMode::Binary, FMemoryResource::Instance());
	GenePoolFileStream->open();
	return MakeUnique<gs4::GenePool>(GenePoolFileStream.get(), static_cast<gs4::GenePoolMask>(genePoolMask), FMemoryResource::Instance());
}

static TUniquePtr<gs4::GenePool> CreateGenePool(trio::BoundedIOStream* Stream, EGenePoolMask genePoolMask)
{
	return MakeUnique<gs4::GenePool>(Stream, static_cast<gs4::GenePoolMask>(genePoolMask), FMemoryResource::Instance());
}

FGenePool::FGenePool(FGenePool&& rhs) = default;
FGenePool& FGenePool::operator=(FGenePool&& rhs) = default;

FGenePool::FGenePool(FArchive& Ar, EGenePoolMask GenePoolMask) 
{
	FArchiveMemoryStream GenePoolStream{ &Ar };
	GenePoolPtr = CreateGenePool(&GenePoolStream, GenePoolMask);
}


FGenePool::FGenePool(const IDNAReader* DeltaArchetype, TArrayView<IDNAReader*> DNAs, EGenePoolMask GenePoolMask) :
	GenePoolPtr{ CreateGenePool(DeltaArchetype, DNAs, GenePoolMask) }
{
}

FGenePool::FGenePool(const FString& Path, EGenePoolMask GenePoolMask) :
	GenePoolPtr{ CreateGenePool(Path, GenePoolMask) }
{
}

FGenePool::~FGenePool() = default;

gs4::GenePool* FGenePool::Unwrap() const
{
	return GenePoolPtr.Get();
}

uint16 FGenePool::GetDNACount() const
{
	return GenePoolPtr->getDNACount();
}

void FGenePool::WriteToFile(const FString& Path, EGenePoolMask GenePoolMask) const
{
	auto GenePoolFileStream = rl4::makeScoped<rl4::FileStream>(TCHAR_TO_UTF8(*Path), rl4::FileStream::AccessMode::Write, rl4::FileStream::OpenMode::Binary, FMemoryResource::Instance());
	GenePoolFileStream->open();
	GenePoolPtr->dump(GenePoolFileStream.get(), static_cast<gs4::GenePoolMask>(GenePoolMask));
	GenePoolFileStream->close();
}

void FGenePool::Serialize(FArchive& Ar, EGenePoolMask GenePoolMask) const
{
	FArchiveMemoryStream Stream{ &Ar };
	Stream.open();
	GenePoolPtr->dump(&Stream, static_cast<gs4::GenePoolMask>(GenePoolMask));
	Stream.close();
}

FString FGenePool::GetDNAName(uint16 DNAIndex) const
{
	return FString(ANSI_TO_TCHAR(GenePoolPtr->getDNAName(DNAIndex)));
}

EGender FGenePool::GetDNAGender(uint16 DNAIndex) const
{
	return static_cast<EGender>(GenePoolPtr->getDNAGender(DNAIndex));
}

uint16 FGenePool::GetDNAAge(uint16 DNAIndex) const
{
	return GenePoolPtr->getDNAAge(DNAIndex);
}

uint16 FGenePool::GetMeshCount() const
{
	return GenePoolPtr->getMeshCount();
}

uint32 FGenePool::GetVertexPositionCount(uint16 MeshIndex) const
{
	return GenePoolPtr->getVertexPositionCount(MeshIndex);
}

FVector FGenePool::GetDNAVertexPosition(uint16 DNAIndex, uint16 MeshIndex, uint32 VertexIndex) const
{
	const auto Position = GenePoolPtr->getDNAVertexPosition(DNAIndex, MeshIndex, VertexIndex);
	// X = X, Y = Z, Z = Y
	return FVector(Position.x, Position.z, Position.y);
}

FVector FGenePool::GetArchetypeVertexPosition(uint16 MeshIndex, uint32 VertexIndex) const
{
	const auto Position = GenePoolPtr->getArchetypeVertexPosition(MeshIndex, VertexIndex);
	// X = X, Y = Z, Z = Y
	return FVector(Position.x, Position.z, Position.y);
}

uint16 FGenePool::GetJointCount() const
{
	return GenePoolPtr->getJointCount();
}

FString FGenePool::GetJointName(uint16 JointIndex) const
{
	return FString(ANSI_TO_TCHAR(GenePoolPtr->getJointName(JointIndex)));
}

FVector FGenePool::GetDNANeutralJointWorldTranslation(uint16 DNAIndex, uint16 JointIndex) const
{
	const auto Translation = GenePoolPtr->getDNANeutralJointWorldTranslation(DNAIndex, JointIndex);
	// X = X, Y = -Y, Z = Z
	return FVector(Translation.x, -Translation.y, Translation.z);
}

FVector FGenePool::GetArchetypeNeutralJointWorldTranslation(uint16 JointIndex) const
{
	const auto Translation = GenePoolPtr->getArchetypeNeutralJointWorldTranslation(JointIndex);
	// X = X, Y = -Y, Z = Z
	return FVector(Translation.x, -Translation.y, Translation.z);
}

FVector FGenePool::GetDNANeutralJointWorldRotation(uint16 DNAIndex, uint16 JointIndex) const
{
	const auto Rotation = GenePoolPtr->getDNANeutralJointWorldRotation(DNAIndex, JointIndex);
	// X = -Y, Y = -Z, Z = X
	return FVector(-Rotation.y, -Rotation.z, Rotation.x);
}

FVector FGenePool::GetArchetypeNeutralJointWorldRotation(uint16 JointIndex) const
{
	const auto Rotation = GenePoolPtr->getArchetypeNeutralJointWorldRotation(JointIndex);
	// X = -Y, Y = -Z, Z = X
	return FVector(-Rotation.y, -Rotation.z, Rotation.x);
}
