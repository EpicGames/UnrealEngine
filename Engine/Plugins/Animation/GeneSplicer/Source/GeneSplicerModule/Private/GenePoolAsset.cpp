// Copyright Epic Games, Inc. All Rights Reserved.


#include "GenePoolAsset.h"

#include "ArchiveMemoryStream.h"
#include "FMemoryResource.h"
#include "GenePoolAssetCustomVersion.h"

#include "genesplicer/splicedata/GenePool.h"

DEFINE_LOG_CATEGORY(LogGenePoolAsset);

UGenePoolAsset::UGenePoolAsset()
{

}

const TSharedPtr<FGenePool>& UGenePoolAsset::GetGenePoolPtr() const 
{
	return GenePool;
}

int32 UGenePoolAsset::GetDNACount() const
{
	return static_cast<int32>(GenePool->GetDNACount());
}

void UGenePoolAsset::SetGenePoolPtr(TSharedPtr<FGenePool> GenePoolPtr)
{
	GenePool = GenePoolPtr;
}

void UGenePoolAsset::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FGenePoolAssetCustomVersion::GUID);

	FWriteScopeLock GenePoolScopeLock{ GenePoolUpdateLock };

	if (Ar.CustomVer(FGenePoolAssetCustomVersion::GUID) >= FGenePoolAssetCustomVersion::BeforeCustomVersionWasAdded)
	{ 
		if (Ar.IsLoading())
		{
			GenePool = MakeShared<FGenePool>(Ar);
		}

		if (Ar.IsSaving())
		{
			if (GenePool.IsValid()) 
			{
				GenePool->Serialize(Ar);
			}
		}
	}
}