// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenePoolAssetCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FGenePoolAssetCustomVersion::GUID(0xFB64E148, 0x2E284BF7, 0xA9117759, 0x1FE5C6CE);

// Register the custom version with core
FCustomVersionRegistration GRegisterGenePoolAssetCustomVersion(FGenePoolAssetCustomVersion::GUID, FGenePoolAssetCustomVersion::LatestVersion, TEXT("GenePoolAssetVer"));
