// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeneSplicer.h"
#include "GeneSplicerDNAReader.h"
#include "FMemoryResource.h"

#include "genesplicer/GeneSplicer.h"

DEFINE_LOG_CATEGORY(LogGeneSplicer);

static void GeneSplicerLogOnError()
{
	if (!gs4::Status::isOk()) 
	{
		UE_LOGF(LogGeneSplicer, Error, "%ls", ANSI_TO_TCHAR(gs4::Status::get().message));
	}
}

FGeneSplicer::FGeneSplicer(ECalculationType CalculationType):
	GeneSplicerPtr{new gs4::GeneSplicer(static_cast<gs4::CalculationType>(CalculationType), FMemoryResource::Instance())}
{
}

FGeneSplicer::~FGeneSplicer() = default;

FGeneSplicer::FGeneSplicer(FGeneSplicer&& rhs) = default;
FGeneSplicer& FGeneSplicer::operator=(FGeneSplicer&& rhs) = default;

void FGeneSplicer::Splice(const FSpliceData& MixData, FGeneSplicerDNAReader& Output)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);
	GeneSplicerPtr->splice(MixData.Unwrap(), static_cast<gs4::GeneSplicerDNAReader*>(Output.Unwrap()));
	GeneSplicerLogOnError();
}

void FGeneSplicer::SpliceNeutralMeshes(const FSpliceData& MixData, FGeneSplicerDNAReader& Output)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);
	GeneSplicerPtr->spliceNeutralMeshes(MixData.Unwrap(), static_cast<gs4::GeneSplicerDNAReader*>(Output.Unwrap()));
	GeneSplicerLogOnError();
}

void FGeneSplicer::SpliceBlendShapes(const FSpliceData& MixData, FGeneSplicerDNAReader& Output)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);
	GeneSplicerPtr->spliceBlendShapes(MixData.Unwrap(), static_cast<gs4::GeneSplicerDNAReader*>(Output.Unwrap()));
	GeneSplicerLogOnError();
}

void FGeneSplicer::SpliceNeutralJoints(const FSpliceData& MixData, FGeneSplicerDNAReader& Output)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);
	GeneSplicerPtr->spliceNeutralJoints(MixData.Unwrap(), static_cast<gs4::GeneSplicerDNAReader*>(Output.Unwrap()));
	GeneSplicerLogOnError();
}

void FGeneSplicer::SpliceJointBehavior(const FSpliceData& MixData, FGeneSplicerDNAReader& Output)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);
	GeneSplicerPtr->spliceJointBehavior(MixData.Unwrap(), static_cast<gs4::GeneSplicerDNAReader*>(Output.Unwrap()));
	GeneSplicerLogOnError();
}

void FGeneSplicer::SpliceSkinWeights(const FSpliceData& MixData, FGeneSplicerDNAReader& Output)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);
	GeneSplicerPtr->spliceSkinWeights(MixData.Unwrap(), static_cast<gs4::GeneSplicerDNAReader*>(Output.Unwrap()));
	GeneSplicerLogOnError();
}

