// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeneSplicerBP.h"

#include "DNAUtils.h"
#include "FMemoryResource.h"
#include "GeneSplicer.h"
#include "GenePool.h"
#include "SkelMeshDNAUtils.h"
#include "genesplicer/GeneSplicerDNAReader.h"

#include "HAL/FileManagerGeneric.h"
#include "Logging/LogMacros.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

bool UGeneSplicerBP::CreateGenePool(const FString& DNAFolderPath, const FString& ArchetypePath, const FString& GenePoolOutputPath) {
	IFileManager& FileManager = IFileManager::Get();
	TArray<FString> FoundFiles;
	FileManager.FindFiles(FoundFiles, *DNAFolderPath, TEXT(".dna"));
	TArray<TSharedPtr<IDNAReader>> DNAReaders{};
	TArray<IDNAReader*> DNAReadersRaw{};
	if (FoundFiles.IsEmpty()) 
	{
		return false;
	}
	for (const FString& FoundFile : FoundFiles)
	{
		FString DNAPath = FPaths::Combine(DNAFolderPath, FoundFile);
		UE_LOGF(LogTemp, Warning, "Found file: %ls", *DNAPath);
		auto DNA = ReadDNAFromFile(DNAPath, EDNADataLayer::All);
		DNAReaders.Add(DNA);
		DNAReadersRaw.Add(DNA.Get());
	}
	auto Archetype = ReadDNAFromFile(ArchetypePath);
	FGenePool GenePool{ Archetype.Get(), {DNAReadersRaw.GetData(), DNAReadersRaw.Num()} };
	GenePool.WriteToFile(GenePoolOutputPath, EGenePoolMask::All);
	return true;
}

bool UGeneSplicerBP::CreateArchetype(const FString& DNAFolderPath, URegionAffiliationAsset* RafAsset, const FString& ArchetypeOutputPath) 
{
	IFileManager& FileManager = IFileManager::Get();
	TArray<FString> FoundFiles;
	FileManager.FindFiles(FoundFiles, *DNAFolderPath, TEXT(".dna"));
	TArray<TSharedPtr<IDNAReader>> DNAReaders{};
	TArray<IDNAReader*> DNAReadersRaw{};
	for (const FString& FoundFile : FoundFiles)
	{
		FString DNAPath = FPaths::Combine(DNAFolderPath, FoundFile);
		UE_LOGF(LogTemp, Warning, "Found file: %ls", *DNAPath);
		auto DNA = ReadDNAFromFile(DNAPath, EDNADataLayer::All);
		DNAReaders.Add(DNA);
		DNAReadersRaw.Add(DNA.Get());
	}
	auto Archetype = DNAReadersRaw[0];
	FGeneSplicerDNAReader OutputDNA{ Archetype };
	auto GenePool = MakeShared<FGenePool>(Archetype, TArrayView<IDNAReader*>{DNAReadersRaw.GetData(), DNAReadersRaw.Num()}, EGenePoolMask::All);

	FSpliceData SpliceData{};
	auto RafPtr = RafAsset->getRegionAffiliationReaderPtr();
	SpliceData.RegisterGenePool("GP", *RafPtr, GenePool);
	SpliceData.SetBaseArchetype(DNAReaders[0]);

	TArray<float> Weights{};
	const int WeightNum = RafPtr->GetRegionNum() * FoundFiles.Num();
	const float Weight = 1.0f / FoundFiles.Num();
	Weights.Init(Weight, WeightNum);
	SpliceData.GetPoolParams("GP")->SetSpliceWeights(0u, TArrayView<float>{Weights});
	FGeneSplicer(FGeneSplicer::ECalculationType::SSE).Splice(SpliceData, OutputDNA);
	WriteDNAToFile(&OutputDNA, EDNADataLayer::All, ArchetypeOutputPath);
	return true;
}

void UGeneSplicerBP::Splice(USpliceData* SpliceData)
{
	FGeneSplicer GeneSplicer(FGeneSplicer::ECalculationType::SSE);
	auto OutputDNA = SpliceData->GetOutputDNA().Get();
	auto DNASkelMeshMap = SpliceData->GetDNASkelMeshMap().Get();
	auto SkelMeshComponent = SpliceData->GetSkeletalMeshComponent();
	auto SkelMesh = SkelMeshComponent->GetSkeletalMeshAsset();
	GeneSplicer.Splice(SpliceData->GetSpliceDataImpl(), *OutputDNA);
#if WITH_EDITORONLY_DATA
	USkelMeshDNAUtils::UpdateJoints(SkelMesh, OutputDNA, DNASkelMeshMap);
	USkelMeshDNAUtils::UpdateJointBehavior(SkelMeshComponent);
	USkelMeshDNAUtils::UpdateBaseMesh(SkelMesh, OutputDNA, DNASkelMeshMap, ELodUpdateOption::LOD0Only);
	// if RebuildRenderData is called, vertex positions will automatically be rebuilt in RebuildRenderData, so we skip it here
	// to avoid rebuilding twice
	USkelMeshDNAUtils::RebuildRenderData_VertexPosition(SkelMesh);
	USkelMeshDNAUtils::UpdateSkinWeights(SkelMesh, OutputDNA, DNASkelMeshMap, ELodUpdateOption::LOD0Only);
	USkelMeshDNAUtils::UpdateMorphTargets(SkelMesh, OutputDNA, DNASkelMeshMap, ELodUpdateOption::LOD0Only);
	USkelMeshDNAUtils::RebuildRenderData(SkelMesh);
#endif // WITH_EDITORONLY_DATA
}