// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeneSplicerDNAReader.h"

#include "DNAReaderAdapter.h"
#include "FMemoryResource.h"

#include "genesplicer/GeneSplicerDNAReader.h"

FGeneSplicerDNAReader::FGeneSplicerDNAReader(IDNAReader* Source) :
	ReaderPtr{new FDNAReader<gs4::GeneSplicerDNAReader>{gs4::GeneSplicerDNAReader::create(Source->Unwrap(), FMemoryResource::Instance()), Source->GetConfig()}}
{
}

dna::Reader* FGeneSplicerDNAReader::Unwrap() const
{
	return ReaderPtr->Unwrap();
}

/** HEADER READER **/
uint16 FGeneSplicerDNAReader::GetFileFormatGeneration() const
{
	return ReaderPtr->GetFileFormatGeneration();
}

uint16 FGeneSplicerDNAReader::GetFileFormatVersion() const
{
	return ReaderPtr->GetFileFormatVersion();
}

/** DESCRIPTOR READER **/

FString FGeneSplicerDNAReader::GetName() const
{
	return ReaderPtr->GetName();
}

EArchetype FGeneSplicerDNAReader::GetArchetype() const
{
	return ReaderPtr->GetArchetype();
}

EGender FGeneSplicerDNAReader::GetGender() const
{
	return ReaderPtr->GetGender();
}

uint16 FGeneSplicerDNAReader::GetAge() const
{
	return ReaderPtr->GetAge();
}

uint32 FGeneSplicerDNAReader::GetMetaDataCount() const
{
	return ReaderPtr->GetMetaDataCount();
}

FString FGeneSplicerDNAReader::GetMetaDataKey(uint32 Index) const
{
	return ReaderPtr->GetMetaDataKey(Index);
}

FString FGeneSplicerDNAReader::GetMetaDataValue(const FString& Key) const
{
	return ReaderPtr->GetMetaDataValue(Key);
}

ETranslationUnit FGeneSplicerDNAReader::GetTranslationUnit() const
{
	return ReaderPtr->GetTranslationUnit();
}

ERotationUnit FGeneSplicerDNAReader::GetRotationUnit() const
{
	return ReaderPtr->GetRotationUnit();
}

FCoordinateSystem FGeneSplicerDNAReader::GetCoordinateSystem() const
{
	return ReaderPtr->GetCoordinateSystem();
}

ERotationSequence FGeneSplicerDNAReader::GetRotationSequence() const
{
	return ReaderPtr->GetRotationSequence();
}
FRotationSign FGeneSplicerDNAReader::GetRotationSign() const
{
	return ReaderPtr->GetRotationSign();
}

EFaceWindingOrder FGeneSplicerDNAReader::GetFaceWindingOrder() const
{
	return ReaderPtr->GetFaceWindingOrder();
}

uint16 FGeneSplicerDNAReader::GetLODCount() const
{
	return ReaderPtr->GetLODCount();
}

uint16 FGeneSplicerDNAReader::GetDBMaxLOD() const
{
	return ReaderPtr->GetDBMaxLOD();
}

FString FGeneSplicerDNAReader::GetDBComplexity() const
{
	return ReaderPtr->GetDBComplexity();
}

FString FGeneSplicerDNAReader::GetDBName() const
{
	return ReaderPtr->GetDBName();
}

/** DEFINITION READER **/

uint16 FGeneSplicerDNAReader::GetGUIControlCount() const
{
	return ReaderPtr->GetGUIControlCount();
}

FString FGeneSplicerDNAReader::GetGUIControlName(uint16 Index) const
{
	return ReaderPtr->GetGUIControlName(Index);
}

uint16 FGeneSplicerDNAReader::GetRawControlCount() const
{
	return ReaderPtr->GetRawControlCount();
}

FString FGeneSplicerDNAReader::GetRawControlName(uint16 Index) const
{
	return ReaderPtr->GetRawControlName(Index);
}

uint16 FGeneSplicerDNAReader::GetJointCount() const
{
	return ReaderPtr->GetJointCount();
}

FString FGeneSplicerDNAReader::GetJointName(uint16 Index) const
{
	return ReaderPtr->GetJointName(Index);
}

uint16 FGeneSplicerDNAReader::GetJointIndexListCount() const
{
	return ReaderPtr->GetJointIndexListCount();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetJointIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetJointIndicesForLOD(LOD);
}

uint16 FGeneSplicerDNAReader::GetBlendShapeChannelCount() const
{
	return ReaderPtr->GetBlendShapeChannelCount();
}

FString FGeneSplicerDNAReader::GetBlendShapeChannelName(uint16 Index) const
{
	return ReaderPtr->GetBlendShapeChannelName(Index);
}

uint16 FGeneSplicerDNAReader::GetBlendShapeChannelIndexListCount() const
{
	return ReaderPtr->GetBlendShapeChannelIndexListCount();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetBlendShapeChannelIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetBlendShapeChannelIndicesForLOD(LOD);
}

uint16 FGeneSplicerDNAReader::GetAnimatedMapCount() const
{
	return ReaderPtr->GetAnimatedMapCount();
}

FString FGeneSplicerDNAReader::GetAnimatedMapName(uint16 Index) const
{
	return ReaderPtr->GetAnimatedMapName(Index);
}

uint16 FGeneSplicerDNAReader::GetAnimatedMapIndexListCount() const
{
	return ReaderPtr->GetAnimatedMapIndexListCount();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetAnimatedMapIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetAnimatedMapIndicesForLOD(LOD);
}

uint16 FGeneSplicerDNAReader::GetMeshCount() const
{
	return ReaderPtr->GetMeshCount();
}

FString FGeneSplicerDNAReader::GetMeshName(uint16 Index) const
{
	return ReaderPtr->GetMeshName(Index);
}

uint16 FGeneSplicerDNAReader::GetMeshIndexListCount() const
{
	return ReaderPtr->GetMeshIndexListCount();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetMeshIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetMeshIndicesForLOD(LOD);
}

uint16 FGeneSplicerDNAReader::GetMeshBlendShapeChannelMappingCount() const
{
	return ReaderPtr->GetMeshBlendShapeChannelMappingCount();
}

FMeshBlendShapeChannelMapping FGeneSplicerDNAReader::GetMeshBlendShapeChannelMapping(uint16 Index) const
{
	return ReaderPtr->GetMeshBlendShapeChannelMapping(Index);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetMeshBlendShapeChannelMappingIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetMeshBlendShapeChannelMappingIndicesForLOD(LOD);
}

FVector FGeneSplicerDNAReader::GetNeutralJointTranslation(uint16 Index) const
{
	return ReaderPtr->GetNeutralJointTranslation(Index);
}

FVector FGeneSplicerDNAReader::GetNeutralJointRotation(uint16 Index) const
{
	return ReaderPtr->GetNeutralJointRotation(Index);
}

uint16 FGeneSplicerDNAReader::GetJointParentIndex(uint16 Index) const
{
	return ReaderPtr->GetJointParentIndex(Index);
}

/** BEHAVIOR READER **/

TArrayView<const uint16> FGeneSplicerDNAReader::GetGUIToRawInputIndices() const
{
	return ReaderPtr->GetGUIToRawInputIndices();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetGUIToRawOutputIndices() const
{
	return ReaderPtr->GetGUIToRawOutputIndices();
}

TArrayView<const float> FGeneSplicerDNAReader::GetGUIToRawFromValues() const
{
	return ReaderPtr->GetGUIToRawFromValues();
}

TArrayView<const float> FGeneSplicerDNAReader::GetGUIToRawToValues() const
{
	return  ReaderPtr->GetGUIToRawToValues();
}

TArrayView<const float> FGeneSplicerDNAReader::GetGUIToRawSlopeValues() const
{
	return ReaderPtr->GetGUIToRawSlopeValues();
}

TArrayView<const float> FGeneSplicerDNAReader::GetGUIToRawCutValues() const
{
	return ReaderPtr->GetGUIToRawCutValues();
}

uint16 FGeneSplicerDNAReader::GetPSDCount() const
{
	return ReaderPtr->GetPSDCount();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetPSDRowIndices() const
{
	return ReaderPtr->GetPSDRowIndices();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetPSDColumnIndices() const
{
	return ReaderPtr->GetPSDColumnIndices();
}

TArrayView<const float> FGeneSplicerDNAReader::GetPSDValues() const
{
	return ReaderPtr->GetPSDValues();
}

uint16 FGeneSplicerDNAReader::GetJointRowCount() const
{
	return ReaderPtr->GetJointRowCount();
}

uint16 FGeneSplicerDNAReader::GetJointColumnCount() const
{
	return ReaderPtr->GetJointColumnCount();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetJointGroupJointIndices(uint16 JointGroupIndex) const
{
	return ReaderPtr->GetJointGroupJointIndices(JointGroupIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetJointVariableAttributeIndices(uint16 LOD) const
{
	return ReaderPtr->GetJointVariableAttributeIndices(LOD);
}

uint16 FGeneSplicerDNAReader::GetJointGroupCount() const
{
	return ReaderPtr->GetJointGroupCount();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetJointGroupLODs(uint16 JointGroupIndex) const
{
	return ReaderPtr->GetJointGroupLODs(JointGroupIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetJointGroupInputIndices(uint16 JointGroupIndex) const
{
	return ReaderPtr->GetJointGroupInputIndices(JointGroupIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetJointGroupOutputIndices(uint16 JointGroupIndex) const
{
	return ReaderPtr->GetJointGroupOutputIndices(JointGroupIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetJointGroupValues(uint16 JointGroupIndex) const
{
	return ReaderPtr->GetJointGroupValues(JointGroupIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetBlendShapeChannelLODs() const
{
	return ReaderPtr->GetBlendShapeChannelLODs();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetBlendShapeChannelInputIndices() const
{
	return ReaderPtr->GetBlendShapeChannelInputIndices();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetBlendShapeChannelOutputIndices() const
{
	return ReaderPtr->GetBlendShapeChannelOutputIndices();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetAnimatedMapLODs() const
{
	return ReaderPtr->GetAnimatedMapLODs();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetAnimatedMapInputIndices() const
{
	return ReaderPtr->GetAnimatedMapInputIndices();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetAnimatedMapOutputIndices() const
{
	return ReaderPtr->GetAnimatedMapOutputIndices();
}

TArrayView<const float> FGeneSplicerDNAReader::GetAnimatedMapFromValues() const
{
	return ReaderPtr->GetAnimatedMapFromValues();
}

TArrayView<const float> FGeneSplicerDNAReader::GetAnimatedMapToValues() const
{
	return ReaderPtr->GetAnimatedMapToValues();
}

TArrayView<const float> FGeneSplicerDNAReader::GetAnimatedMapSlopeValues() const
{
	return ReaderPtr->GetAnimatedMapSlopeValues();
}

TArrayView<const float> FGeneSplicerDNAReader::GetAnimatedMapCutValues() const
{
	return ReaderPtr->GetAnimatedMapCutValues();
}

/** GEOMETRY READER **/
uint32 FGeneSplicerDNAReader::GetVertexPositionCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexPositionCount(MeshIndex);
}

FVector FGeneSplicerDNAReader::GetVertexPosition(uint16 MeshIndex, uint32 PositionIndex) const
{
	return ReaderPtr->GetVertexPosition(MeshIndex, PositionIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetVertexPositionXs(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexPositionXs(MeshIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetVertexPositionYs(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexPositionYs(MeshIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetVertexPositionZs(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexPositionZs(MeshIndex);
}

uint32 FGeneSplicerDNAReader::GetVertexTextureCoordinateCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexTextureCoordinateCount(MeshIndex);
}

FTextureCoordinate FGeneSplicerDNAReader::GetVertexTextureCoordinate(uint16 MeshIndex, uint32 TextureCoordinateIndex) const
{
	return ReaderPtr->GetVertexTextureCoordinate(MeshIndex, TextureCoordinateIndex);
}

uint32 FGeneSplicerDNAReader::GetVertexNormalCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexNormalCount(MeshIndex);
}

FVector FGeneSplicerDNAReader::GetVertexNormal(uint16 MeshIndex, uint32 NormalIndex) const
{
	return ReaderPtr->GetVertexNormal(MeshIndex, NormalIndex);
}

/* not needed for gene splicer */
uint32 FGeneSplicerDNAReader::GetVertexLayoutCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetVertexLayoutCount(MeshIndex);
}

/* not needed for gene splicer */
FVertexLayout FGeneSplicerDNAReader::GetVertexLayout(uint16 MeshIndex, uint32 VertexIndex) const
{
	return ReaderPtr->GetVertexLayout(MeshIndex, VertexIndex);
}

/* not needed for gene splicer */
uint32 FGeneSplicerDNAReader::GetFaceCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetFaceCount(MeshIndex);
}

/* not needed for gene splicer */
TArrayView<const uint32> FGeneSplicerDNAReader::GetFaceVertexLayoutIndices(uint16 MeshIndex, uint32 FaceIndex) const
{
	return ReaderPtr->GetFaceVertexLayoutIndices(MeshIndex, FaceIndex);
}

uint16 FGeneSplicerDNAReader::GetMaximumInfluencePerVertex(uint16 MeshIndex) const
{
	return ReaderPtr->GetMaximumInfluencePerVertex(MeshIndex);
}

uint32 FGeneSplicerDNAReader::GetSkinWeightsCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetSkinWeightsCount(MeshIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetSkinWeightsValues(uint16 MeshIndex, uint32 VertexIndex) const
{
	return ReaderPtr->GetSkinWeightsValues(MeshIndex, VertexIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetSkinWeightsJointIndices(uint16 MeshIndex, uint32 VertexIndex) const
{
	return ReaderPtr->GetSkinWeightsJointIndices(MeshIndex, VertexIndex);
}

uint16 FGeneSplicerDNAReader::GetBlendShapeTargetCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetBlendShapeTargetCount(MeshIndex);
}

uint16 FGeneSplicerDNAReader::GetBlendShapeChannelIndex(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	return ReaderPtr->GetBlendShapeChannelIndex(MeshIndex, BlendShapeTargetIndex);
}

uint32 FGeneSplicerDNAReader::GetBlendShapeTargetDeltaCount(uint16 MeshIndex, uint16 BlendShapeIndex) const
{
	return ReaderPtr->GetBlendShapeTargetDeltaCount(MeshIndex, BlendShapeIndex);
}

FVector FGeneSplicerDNAReader::GetBlendShapeTargetDelta(uint16 MeshIndex, uint16 BlendShapeIndex, uint32 DeltaIndex) const
{
	return ReaderPtr->GetBlendShapeTargetDelta(MeshIndex, BlendShapeIndex, DeltaIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetBlendShapeTargetDeltaXs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	return ReaderPtr->GetBlendShapeTargetDeltaXs(MeshIndex, BlendShapeTargetIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetBlendShapeTargetDeltaYs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	return ReaderPtr->GetBlendShapeTargetDeltaYs(MeshIndex, BlendShapeTargetIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetBlendShapeTargetDeltaZs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	return ReaderPtr->GetBlendShapeTargetDeltaZs(MeshIndex, BlendShapeTargetIndex);
}

TArrayView<const uint32> FGeneSplicerDNAReader::GetBlendShapeTargetVertexIndices(uint16 MeshIndex, uint16 BlendShapeIndex) const
{
	return ReaderPtr->GetBlendShapeTargetVertexIndices(MeshIndex, BlendShapeIndex);
}

/** MACHINE LEARNED BEHAVIOR READER **/
uint16 FGeneSplicerDNAReader::GetMLControlCount() const
{
	return ReaderPtr->GetMLControlCount();
}

FString FGeneSplicerDNAReader::GetMLControlName(uint16 Index) const
{
	return ReaderPtr->GetMLControlName(Index);
}

uint16 FGeneSplicerDNAReader::GetNeuralNetworkCount() const
{
	return ReaderPtr->GetNeuralNetworkCount();
}

uint16 FGeneSplicerDNAReader::GetNeuralNetworkIndexListCount() const
{
	return ReaderPtr->GetNeuralNetworkIndexListCount();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetNeuralNetworkIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetNeuralNetworkIndicesForLOD(LOD);
}

uint16 FGeneSplicerDNAReader::GetMeshRegionCount(uint16 MeshIndex) const
{
	return ReaderPtr->GetMeshRegionCount(MeshIndex);
}

FString FGeneSplicerDNAReader::GetMeshRegionName(uint16 MeshIndex, uint16 RegionIndex) const
{
	return ReaderPtr->GetMeshRegionName(MeshIndex, RegionIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetNeuralNetworkIndicesForMeshRegion(uint16 MeshIndex, uint16 RegionIndex) const
{
	return ReaderPtr->GetNeuralNetworkIndicesForMeshRegion(MeshIndex, RegionIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetNeuralNetworkInputIndices(uint16 NetIndex) const
{
	return ReaderPtr->GetNeuralNetworkInputIndices(NetIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetNeuralNetworkOutputIndices(uint16 NetIndex) const
{
	return ReaderPtr->GetNeuralNetworkOutputIndices(NetIndex);
}

uint16 FGeneSplicerDNAReader::GetNeuralNetworkLayerCount(uint16 NetIndex) const
{
	return ReaderPtr->GetNeuralNetworkLayerCount(NetIndex);
}

EActivationFunction FGeneSplicerDNAReader::GetNeuralNetworkLayerActivationFunction(uint16 NetIndex, uint16 LayerIndex) const
{
	return ReaderPtr->GetNeuralNetworkLayerActivationFunction(NetIndex, LayerIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetNeuralNetworkLayerActivationFunctionParameters(uint16 NetIndex, uint16 LayerIndex) const
{
	return ReaderPtr->GetNeuralNetworkLayerActivationFunctionParameters(NetIndex, LayerIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetNeuralNetworkLayerBiases(uint16 NetIndex, uint16 LayerIndex) const
{
	return ReaderPtr->GetNeuralNetworkLayerBiases(NetIndex, LayerIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetNeuralNetworkLayerWeights(uint16 NetIndex, uint16 LayerIndex) const
{
	return ReaderPtr->GetNeuralNetworkLayerWeights(NetIndex, LayerIndex);
}

ETranslationRepresentation FGeneSplicerDNAReader::GetJointTranslationRepresentation(uint16 JointIndex) const
{
	return ReaderPtr->GetJointTranslationRepresentation(JointIndex);
}

ERotationRepresentation FGeneSplicerDNAReader::GetJointRotationRepresentation(uint16 JointIndex) const
{
	return ReaderPtr->GetJointRotationRepresentation(JointIndex);
}

EScaleRepresentation FGeneSplicerDNAReader::GetJointScaleRepresentation(uint16 JointIndex) const
{
	return ReaderPtr->GetJointScaleRepresentation(JointIndex);
}

uint16 FGeneSplicerDNAReader::GetRBFPoseCount() const
{
	return ReaderPtr->GetRBFPoseCount();
}

FString FGeneSplicerDNAReader::GetRBFPoseName(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseName(PoseIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetRBFPoseJointOutputIndices(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseJointOutputIndices(PoseIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetRBFPoseBlendShapeChannelOutputIndices(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseBlendShapeChannelOutputIndices(PoseIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetRBFPoseAnimatedMapOutputIndices(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseAnimatedMapOutputIndices(PoseIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetRBFPoseJointOutputValues(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseJointOutputValues(PoseIndex);
}

float FGeneSplicerDNAReader::GetRBFPoseScale(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseScale(PoseIndex);
}

uint16 FGeneSplicerDNAReader::GetRBFPoseControlCount() const
{
	return ReaderPtr->GetRBFPoseControlCount();
}

FString FGeneSplicerDNAReader::GetRBFPoseControlName(uint16 PoseControlIndex) const
{
	return ReaderPtr->GetRBFPoseControlName(PoseControlIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetRBFPoseInputControlIndices(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseInputControlIndices(PoseIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetRBFPoseOutputControlIndices(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseOutputControlIndices(PoseIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetRBFPoseOutputControlWeights(uint16 PoseIndex) const
{
	return ReaderPtr->GetRBFPoseOutputControlWeights(PoseIndex);
}

uint16 FGeneSplicerDNAReader::GetRBFSolverCount() const
{
	return ReaderPtr->GetRBFSolverCount();
}

uint16 FGeneSplicerDNAReader::GetRBFSolverIndexListCount() const
{
	return ReaderPtr->GetRBFSolverIndexListCount();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetRBFSolverIndicesForLOD(uint16 LOD) const
{
	return ReaderPtr->GetRBFSolverIndicesForLOD(LOD);
}

FString FGeneSplicerDNAReader::GetRBFSolverName(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverName(SolverIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetRBFSolverRawControlIndices(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverRawControlIndices(SolverIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetRBFSolverPoseIndices(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverPoseIndices(SolverIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetRBFSolverRawControlValues(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverRawControlValues(SolverIndex);
}

ERBFSolverType FGeneSplicerDNAReader::GetRBFSolverType(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverType(SolverIndex);
}

float FGeneSplicerDNAReader::GetRBFSolverRadius(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverRadius(SolverIndex);
}

EAutomaticRadius FGeneSplicerDNAReader::GetRBFSolverAutomaticRadius(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverAutomaticRadius(SolverIndex);
}

float FGeneSplicerDNAReader::GetRBFSolverWeightThreshold(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverWeightThreshold(SolverIndex);
}

ERBFDistanceMethod FGeneSplicerDNAReader::GetRBFSolverDistanceMethod(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverDistanceMethod(SolverIndex);
}

ERBFNormalizeMethod FGeneSplicerDNAReader::GetRBFSolverNormalizeMethod(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverNormalizeMethod(SolverIndex);
}

ERBFFunctionType FGeneSplicerDNAReader::GetRBFSolverFunctionType(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverFunctionType(SolverIndex);
}

ETwistAxis FGeneSplicerDNAReader::GetRBFSolverTwistAxis(uint16 SolverIndex) const
{
	return ReaderPtr->GetRBFSolverTwistAxis(SolverIndex);
}

uint16 FGeneSplicerDNAReader::GetTwistCount() const
{
	return ReaderPtr->GetTwistCount();
}

ETwistAxis FGeneSplicerDNAReader::GetTwistSetupTwistAxis(uint16 TwistIndex) const
{
	return ReaderPtr->GetTwistSetupTwistAxis(TwistIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetTwistInputControlIndices(uint16 TwistIndex) const
{
	return ReaderPtr->GetTwistInputControlIndices(TwistIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetTwistOutputJointIndices(uint16 TwistIndex) const
{
	return ReaderPtr->GetTwistOutputJointIndices(TwistIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetTwistBlendWeights(uint16 TwistIndex) const
{
	return ReaderPtr->GetTwistBlendWeights(TwistIndex);
}

uint16 FGeneSplicerDNAReader::GetSwingCount() const
{
	return ReaderPtr->GetSwingCount();
}

ETwistAxis FGeneSplicerDNAReader::GetSwingSetupTwistAxis(uint16 SwingIndex) const
{
	return ReaderPtr->GetSwingSetupTwistAxis(SwingIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetSwingInputControlIndices(uint16 SwingIndex) const
{
	return ReaderPtr->GetSwingInputControlIndices(SwingIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetSwingOutputJointIndices(uint16 SwingIndex) const
{
	return ReaderPtr->GetSwingOutputJointIndices(SwingIndex);
}

TArrayView<const float> FGeneSplicerDNAReader::GetSwingBlendWeights(uint16 SwingIndex) const
{
	return ReaderPtr->GetSwingBlendWeights(SwingIndex);
}

uint16 FGeneSplicerDNAReader::GetMLTypeCount() const
{
	return ReaderPtr->GetMLTypeCount();
}

uint16 FGeneSplicerDNAReader::GetMLOperationSetCount(uint16 MLTypeIndex) const
{
	return ReaderPtr->GetMLOperationSetCount(MLTypeIndex);
}

uint16 FGeneSplicerDNAReader::GetMLOperationCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const
{
	return ReaderPtr->GetMLOperationCount(MLTypeIndex, MLOperationSetIndex);
}

EMachineLearnedBehaviorOperationType FGeneSplicerDNAReader::GetMLOperationType(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return static_cast<EMachineLearnedBehaviorOperationType>(ReaderPtr->GetMLOperationType(MLTypeIndex, MLOperationSetIndex, MLOperationIndex));
}

TArrayView<const uint32> FGeneSplicerDNAReader::GetMLOperationParameters(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return ReaderPtr->GetMLOperationParameters(MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetMLOperationDependencyOperationSetIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return ReaderPtr->GetMLOperationDependencyOperationSetIndices(MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetMLOperationDependencyOperationIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return ReaderPtr->GetMLOperationDependencyOperationIndices(MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
}

uint16 FGeneSplicerDNAReader::GetMLOperationIndexListCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const
{
	return ReaderPtr->GetMLOperationIndexListCount(MLTypeIndex, MLOperationSetIndex);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetMLOperationIndicesForLOD(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 LOD) const
{
	return ReaderPtr->GetMLOperationIndicesForLOD(MLTypeIndex, MLOperationSetIndex, LOD);
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetMLJointsInputIndices() const
{
	return ReaderPtr->GetMLJointsInputIndices();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetMLJointsOutputIndices() const
{
	return ReaderPtr->GetMLJointsOutputIndices();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetMLJointsParameterKeys() const
{
	return ReaderPtr->GetMLJointsParameterKeys();
}

TArrayView<const uint16> FGeneSplicerDNAReader::GetMLJointsParameterValues() const
{
	return ReaderPtr->GetMLJointsParameterValues();
}

const FDNAConfig& FGeneSplicerDNAReader::GetConfig() const
{
	return ReaderPtr->GetConfig();
}

void FGeneSplicerDNAReader::Unload(EDNADataLayer Layer) {
	return ReaderPtr->Unload(Layer);
}
