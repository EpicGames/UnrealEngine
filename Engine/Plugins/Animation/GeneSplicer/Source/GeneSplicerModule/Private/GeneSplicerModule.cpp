// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeneSplicerModule.h"

#include "Interfaces/IPluginManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FGeneSplicerModule"

DEFINE_LOG_CATEGORY(LogGeneSplicerModule);

void FGeneSplicerModule::StartupModule()
{
}

void FGeneSplicerModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FGeneSplicerModule, GeneSplicerModule)
