// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoolSpliceParams.h"
#include "RegionAffiliationReader.h"

#include "DNAReader.h"
#include "FMemoryResource.h"

#include "genesplicer/splicedata/PoolSpliceParams.h"


FPoolSpliceParams::FPoolSpliceParams(gs4::PoolSpliceParams* PoolSpliceParams) :
	PoolSpliceParamsPtr{ PoolSpliceParams }
{
}

FPoolSpliceParams::~FPoolSpliceParams() = default;

gs4::PoolSpliceParams* FPoolSpliceParams::Unwrap() const
{
	return PoolSpliceParamsPtr;
};

void FPoolSpliceParams::SetDNAFilter(TArrayView<const uint16> DNAIndices)
{
	PoolSpliceParamsPtr->setDNAFilter(DNAIndices.GetData(), static_cast<uint16>(DNAIndices.Num()));
}

void FPoolSpliceParams::SetMeshFilter(TArrayView<const uint16> MeshIndices)
{
	PoolSpliceParamsPtr->setMeshFilter(MeshIndices.GetData(), static_cast<uint16>(MeshIndices.Num()));
}

void FPoolSpliceParams::SetSpliceWeights(uint16 DNAStartIndex, TArrayView<const float> Weights)
{
	PoolSpliceParamsPtr->setSpliceWeights(DNAStartIndex, Weights.GetData(), Weights.Num());
}

void FPoolSpliceParams::SetScale(float scale)
{
	PoolSpliceParamsPtr->setScale(scale);
}

uint16 FPoolSpliceParams::GetDNACount() const 
{
	return PoolSpliceParamsPtr->getDNACount();
}

uint16 FPoolSpliceParams::GetRegionCount() const
{
	return PoolSpliceParamsPtr->getRegionCount();
}