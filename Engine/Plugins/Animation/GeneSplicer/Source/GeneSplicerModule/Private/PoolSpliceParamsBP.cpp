// Copyright Epic Games, Inc. All Rights Reserved.


#include "PoolSpliceParamsBP.h"
#include "genesplicer/GeneSplicerDNAReader.h"

void UPoolSpliceParams::RegisterToSpliceData(USpliceData* SpliceData, const FString& Name, const UGenePoolAsset* GenePoolAsset, URegionAffiliationAsset* Raf)
{
	RegionNames.Reserve(Raf->GetRegionCount());
	for (int16 i = 0u; i < Raf->GetRegionCount(); i++) 
	{
		RegionNames.Add(Raf->GetRegionName(i));
	}
	SpliceData->GetSpliceDataImpl().RegisterGenePool(Name, *Raf->getRegionAffiliationReaderPtr(), GenePoolAsset->GetGenePoolPtr());
	PoolSpliceParams = SpliceData->GetSpliceDataImpl().GetPoolParams(Name);
}

int32 UPoolSpliceParams::GetDNACount() const 
{
	if (!PoolSpliceParams.IsValid()) 
	{
		return 0;
	}
	return PoolSpliceParams->GetDNACount();
}

int32 UPoolSpliceParams::GetRegionCount() const
{
	if (!PoolSpliceParams.IsValid()) 
	{
		return 0;
	}
	return PoolSpliceParams->GetRegionCount();
}

const TArray<FString>& UPoolSpliceParams::GetRegionNames() const
{
	return RegionNames;
}

void UPoolSpliceParams::SetSpliceWeights(int32 DNAStartIndex, const TArray<float>& Weights) 
{
	PoolSpliceParams->SetSpliceWeights(DNAStartIndex, Weights);
}
