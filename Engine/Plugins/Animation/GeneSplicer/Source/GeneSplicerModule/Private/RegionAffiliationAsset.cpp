// Copyright Epic Games, Inc. All Rights Reserved.


#include "RegionAffiliationAsset.h"
#include "RegionAffiliationAssetCustomVersion.h"

#include "ArchiveMemoryStream.h"
#include "FMemoryResource.h"

DEFINE_LOG_CATEGORY(LogRegionAffiliationAsset);

URegionAffiliationAsset::URegionAffiliationAsset()
{

}

int32 URegionAffiliationAsset::GetRegionCount() const 
{
	return static_cast<int32>(RegionAffiliationReader->GetRegionNum());
}

FString URegionAffiliationAsset::GetRegionName(int32 RegionIndex) const 
{
	return RegionAffiliationReader->getRegionName(RegionIndex);
}

const TSharedPtr<FRegionAffiliationReader>& URegionAffiliationAsset::getRegionAffiliationReaderPtr() {
	return RegionAffiliationReader;
}

void URegionAffiliationAsset::SetRegionAffiliationPtr(TSharedPtr<FRegionAffiliationReader> RegionAffiliationReaderPtr)
{
	FWriteScopeLock RAFScopeLock{ UpdateLock };
	RegionAffiliationReader = RegionAffiliationReaderPtr;
}

void URegionAffiliationAsset::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FRegionAffiliationAssetCustomVersion::GUID);

	FWriteScopeLock RAFScopeLock{ UpdateLock };

	if (Ar.CustomVer(FRegionAffiliationAssetCustomVersion::GUID) >= FRegionAffiliationAssetCustomVersion::BeforeCustomVersionWasAdded)
	{
		if (Ar.IsLoading())
		{
			RegionAffiliationReader = MakeShared<FRegionAffiliationReader>(Ar);
		}

		if (Ar.IsSaving())
		{
			if (RegionAffiliationReader.IsValid()) {
				RegionAffiliationReader->Serialize(Ar);
			}
		}
	}
	
}