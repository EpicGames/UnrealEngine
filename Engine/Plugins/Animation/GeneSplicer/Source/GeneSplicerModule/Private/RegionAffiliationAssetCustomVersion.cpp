// Copyright Epic Games, Inc. All Rights Reserved.

#include "RegionAffiliationAssetCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FRegionAffiliationAssetCustomVersion::GUID(0xFA0C4D22, 0x6D36415E, 0x937F159F, 0xB7814F35);

// Register the custom version with core
FCustomVersionRegistration GRegisterRegionAffiliationCustomVersion(FRegionAffiliationAssetCustomVersion::GUID, FRegionAffiliationAssetCustomVersion::LatestVersion, TEXT("RegionAffiliationAssetVer"));
