// Copyright Epic Games, Inc. All Rights Reserved.

#include "RegionAffiliationReader.h"

#include "ArchiveMemoryStream.h"
#include "FMemoryResource.h"

#include "raf/RegionAffiliationBinaryStreamReader.h"
#include "raf/RegionAffiliationBinaryStreamWriter.h"
#include "raf/types/Aliases.h"
#include "riglogic/RigLogic.h"

DEFINE_LOG_CATEGORY(LogRegionAffiliationReader);


static raf::RegionAffiliationBinaryStreamReader* createRegionAffiliationReader(trio::BoundedIOStream* Stream)
{
	auto Instance = raf::makeScoped<raf::RegionAffiliationBinaryStreamReader>(Stream, FMemoryResource::Instance());
	Instance->read();
	if (!raf::Status::isOk())
	{
		UE_LOGF(LogRegionAffiliationReader, Error, "%ls", ANSI_TO_TCHAR(raf::Status::get().message));
		return nullptr;
	}
	return Instance.release();
}

static raf::RegionAffiliationBinaryStreamReader* ReadRegionAffiliationsFromFile(const FString& Path)
{
	LLM_SCOPE(ELLMTag::SkeletalMesh);
	auto RegionsFileStream = raf::makeScoped<raf::FileStream>(TCHAR_TO_ANSI(*Path), raf::FileStream::AccessMode::Read, raf::FileStream::OpenMode::Binary, FMemoryResource::Instance());
	return createRegionAffiliationReader(RegionsFileStream.get());
}

FRegionAffiliationReader::FRegionAffiliationReader(FArchive& Ar)
{
	FArchiveMemoryStream Stream{ &Ar };
	RegionAffiliationPtr = TUniquePtr<raf::RegionAffiliationBinaryStreamReader, FRegionAffiliationReaderDeleter >{ createRegionAffiliationReader(&Stream) };

}
void FRegionAffiliationReader::Serialize(FArchive& Ar) const
{
	FArchiveMemoryStream Stream{ &Ar };
	Stream.open();
	auto RAFWriter = rl4::makeScoped<raf::RegionAffiliationBinaryStreamWriter>(&Stream, FMemoryResource::Instance());
	RAFWriter->setFrom(RegionAffiliationPtr.Get());
	RAFWriter->write();
	Stream.close();
}

void FRegionAffiliationReader::WriteToFile(const FString& Path) const
{
	auto Stream = rl4::makeScoped<rl4::FileStream>(TCHAR_TO_UTF8(*Path), rl4::FileStream::AccessMode::Write, rl4::FileStream::OpenMode::Binary, FMemoryResource::Instance());
	Stream->open();
	auto RAFWriter = rl4::makeScoped<raf::RegionAffiliationBinaryStreamWriter>(Stream.get(), FMemoryResource::Instance());
	RAFWriter->setFrom(RegionAffiliationPtr.Get());
	RAFWriter->write();
	Stream->close();
}

FRegionAffiliationReader::FRegionAffiliationReader(const FString& FilePath) :
	RegionAffiliationPtr{ ReadRegionAffiliationsFromFile(FilePath) }
{
}

uint16 FRegionAffiliationReader::GetRegionNum() const
{
	return RegionAffiliationPtr->getRegionCount();
}

FString FRegionAffiliationReader::getRegionName(uint16 regionIndex) const
{
	return FString(ANSI_TO_TCHAR(RegionAffiliationPtr->getRegionName(regionIndex).data()));
}

void FRegionAffiliationReader::GetVertexRegionAffiliation(uint16 MeshId, int32 VertexId, TArray<float>& OutRegionAffiliations)
{
	raf::ConstArrayView<float> RegionAffiliationsSparse = RegionAffiliationPtr->getVertexRegionAffiliation(MeshId, VertexId);
	raf::ConstArrayView<uint16> RegionIndices = RegionAffiliationPtr->getVertexRegionIndices(MeshId, VertexId);
	TArray<float> RegionAffiliations;
	OutRegionAffiliations.Init(0, RegionAffiliationPtr->getRegionCount());
	int32 AffiliationSize = RegionAffiliationsSparse.size();
	for (int32 AffiliationIndex = 0; AffiliationIndex < AffiliationSize; AffiliationIndex++)
	{
		OutRegionAffiliations[RegionIndices[AffiliationIndex]] = RegionAffiliationsSparse[AffiliationIndex];
	}
}

FRegionAffiliationReader::~FRegionAffiliationReader() = default;

raf::RegionAffiliationBinaryStreamReader* FRegionAffiliationReader::Unwrap() const
{
	return RegionAffiliationPtr.Get();
}

void FRegionAffiliationReader::FRegionAffiliationReaderDeleter::operator()(raf::RegionAffiliationBinaryStreamReader* Pointer)
{
	if (Pointer != nullptr) 
	{
		raf::RegionAffiliationBinaryStreamReader::destroy(Pointer);
	}
}