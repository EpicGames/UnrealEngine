// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpliceData.h"
#include "RegionAffiliationReader.h"

#include "DNAReader.h"
#include "FMemoryResource.h"

#include "genesplicer/splicedata/SpliceData.h"

FSpliceData::FSpliceData() :
	SpliceDataPtr{ new gs4::SpliceData{FMemoryResource::Instance()} }
{
}

FSpliceData::~FSpliceData() = default;
FSpliceData::FSpliceData(FSpliceData&&) = default;
FSpliceData& FSpliceData::operator=(FSpliceData&&) = default;

gs4::SpliceData* FSpliceData::Unwrap() const
{
	return SpliceDataPtr.Get();
};

void FSpliceData::RegisterGenePool(const FString& name, const FRegionAffiliationReader& RegionAffiliationReader, const TSharedPtr<FGenePool>& GenePool) {
	SpliceDataPtr->registerGenePool(TCHAR_TO_ANSI(*name), RegionAffiliationReader.Unwrap(), GenePool->Unwrap());
}

void FSpliceData::UnregisterGenePool(const FString& name) {
	SpliceDataPtr->unregisterGenePool(TCHAR_TO_ANSI(*name));
}

TSharedPtr<FPoolSpliceParams> FSpliceData::GetPoolParams(const FString& name) {
	FPoolSpliceParams* temp = new FPoolSpliceParams{ SpliceDataPtr->getPoolParams(TCHAR_TO_ANSI(*name)) };
	return MakeShareable<FPoolSpliceParams>(temp);
}

void FSpliceData::SetBaseArchetype(TSharedPtr<const IDNAReader> baseArchetype) {
	SpliceDataPtr->setBaseArchetype(baseArchetype->Unwrap());
}