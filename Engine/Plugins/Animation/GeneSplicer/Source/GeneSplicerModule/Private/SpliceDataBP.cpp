// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpliceDataBP.h"

#include "FMemoryResource.h"
#include "DNAUtils.h"
#include "SkelMeshDNAUtils.h"

#include "genesplicer/splicedata/SpliceData.h"
#include "genesplicer/GeneSplicerDNAReader.h"

DEFINE_LOG_CATEGORY(LogSpliceData);

USpliceData::USpliceData() :
	SpliceDataImpl{},
	SkelMeshComponent{},
	DNASkelMeshMap{},
	OutputDNA{}
{

}

USpliceData::~USpliceData() = default;

TSharedPtr<FPoolSpliceParams> USpliceData::InitPoolSpliceParams(const FString& Name, UGenePoolAsset* GenePoolAsset, URegionAffiliationAsset* raf)
{
	SpliceDataImpl.RegisterGenePool(Name, *raf->getRegionAffiliationReaderPtr(), GenePoolAsset->GetGenePoolPtr());
	return SpliceDataImpl.GetPoolParams(Name);
}

void USpliceData::RegisterGenePool(const FString& Name, UGenePoolAsset* GenePoolAsset, URegionAffiliationAsset* raf)
{
	SpliceDataImpl.RegisterGenePool(Name, *raf->getRegionAffiliationReaderPtr(), GenePoolAsset->GetGenePoolPtr());
}

void USpliceData::SetSpliceWeights(const FString& Name, int32 DNAStartIndex, const TArray<float>& Weights)
{
	SpliceDataImpl.GetPoolParams(Name)->SetSpliceWeights(DNAStartIndex, Weights);
}

void USpliceData::SetArchetype(const FString& path)
{
	auto BaseArchetype = ReadDNAFromFile(path);
	SpliceDataImpl.SetBaseArchetype(BaseArchetype);
	OutputDNA = MakeShared<FGeneSplicerDNAReader>(BaseArchetype.Get());
	if (SkelMeshComponent) 
	{
		GenerateDNASkelMeshMapping();
	}
}

void USpliceData::SetSkeletalMeshComponent(USkeletalMeshComponent* NewSkelMeshComponent)
{
	SkelMeshComponent = NewSkelMeshComponent;
	if (OutputDNA) 
	{
		GenerateDNASkelMeshMapping();
	}
}

void USpliceData::GenerateDNASkelMeshMapping()
{
#if WITH_EDITORONLY_DATA
	USkeletalMesh* SkeletalMesh = SkelMeshComponent->GetSkeletalMeshAsset();
	DNASkelMeshMap = MakeShareable(USkelMeshDNAUtils::CreateMapForUpdatingNeutralMesh(OutputDNA.Get(), SkeletalMesh));
	DNASkelMeshMap->MapJoints(OutputDNA.Get());
	DNASkelMeshMap->MapMorphTargets(OutputDNA.Get());
#endif // WITH_EDITORONLY_DATA
}

USkeletalMeshComponent* USpliceData::GetSkeletalMeshComponent() const
{
	return SkelMeshComponent;
}


TSharedPtr<FDNAToSkelMeshMap> USpliceData::GetDNASkelMeshMap() const
{
	return DNASkelMeshMap;
}

TSharedPtr<FGeneSplicerDNAReader> USpliceData::GetOutputDNA() const
{
	return OutputDNA;
}

FSpliceData& USpliceData::GetSpliceDataImpl()
{
	return SpliceDataImpl;
}

const FSpliceData& USpliceData::GetSpliceDataImpl() const
{
	return SpliceDataImpl;
}