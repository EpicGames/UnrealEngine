// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DNACommon.h"
#include "UObject/NoExportTypes.h"


class IDNAReader;

namespace gs4
{
	class GenePool;
} // namespace gs4

namespace trio {
	class BoundedIOStream;
} // namespace trio


enum class EGenePoolMask : uint32
{
	NeutralMeshes = 1 << 0,
	BlendShapes = 1 << 1,
	SkinWeights = 1 << 2,
	NeutralJoints = 1 << 3,
	JointBehavior = 1 << 4,
	All = (1 << 5) - 1
};
ENUM_CLASS_FLAGS(EGenePoolMask);


class GENESPLICERMODULE_API FGenePool {
public:
	enum class ECalculationType : uint8
	{
		Scalar,
		SSE
	};

public:
	FGenePool(const IDNAReader* DeltaArchetype, TArrayView<IDNAReader*> DNAs, EGenePoolMask GenePoolMask = EGenePoolMask::All);
	FGenePool(const FString& Path, EGenePoolMask GenePoolMask = EGenePoolMask::All);
	FGenePool(FArchive& Ar, EGenePoolMask GenePoolMask = EGenePoolMask::All);

	~FGenePool();

	FGenePool(const FGenePool&) = delete;
	FGenePool& operator=(const FGenePool&) = delete;

	FGenePool(FGenePool&& rhs);
	FGenePool& operator=(FGenePool&& rhs);

	uint16 GetDNACount() const;
	FString GetDNAName(uint16 DNAIndex) const;
	EGender GetDNAGender(uint16 DNAIndex) const;
	uint16 GetDNAAge(uint16 DNAIndex) const;
	uint16 GetMeshCount() const;
	uint32 GetVertexPositionCount(uint16 MeshIndex) const;
	FVector GetDNAVertexPosition(uint16 DNAIndex, uint16 MeshIndex, uint32 VertexIndex) const;
	FVector GetArchetypeVertexPosition(uint16 MeshIndex, uint32 VertexIndex) const;
	uint16 GetJointCount() const;
	FString GetJointName(uint16 JointIndex) const;
	FVector GetDNANeutralJointWorldTranslation(uint16 DNAIndex, uint16 JointIndex) const;
	FVector GetArchetypeNeutralJointWorldTranslation(uint16 JointIndex) const;
	FVector GetDNANeutralJointWorldRotation(uint16 DNAIndex, uint16 JointIndex) const;
	FVector GetArchetypeNeutralJointWorldRotation(uint16 JointIndex) const;

	void WriteToFile(const FString& Path, EGenePoolMask GenePoolMask = EGenePoolMask::All) const;
	void Serialize(FArchive& Ar, EGenePoolMask GenePoolMask = EGenePoolMask::All) const;

private:
	friend class FPoolSpliceParams;
	friend class FSpliceData;
	gs4::GenePool* Unwrap() const;

private:
	TUniquePtr<gs4::GenePool> GenePoolPtr;
};
