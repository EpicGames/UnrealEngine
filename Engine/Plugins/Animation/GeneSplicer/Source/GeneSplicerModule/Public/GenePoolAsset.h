// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenePool.h"
#include "UObject/NoExportTypes.h"

#include "GenePoolAsset.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGenePoolAsset, Log, All);

UCLASS(BlueprintType)
class GENESPLICERMODULE_API UGenePoolAsset : public UObject
{
	GENERATED_BODY()
public:
	UGenePoolAsset();

	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	int32 GetDNACount() const;

	void SetGenePoolPtr(TSharedPtr<FGenePool> GenePoolPtr);
	const TSharedPtr<FGenePool>& GetGenePoolPtr() const;

	void Serialize(FArchive& Ar) override;

private:
	// Synchronize GenePool updates
	FRWLock GenePoolUpdateLock;
	TSharedPtr<FGenePool> GenePool;
};
