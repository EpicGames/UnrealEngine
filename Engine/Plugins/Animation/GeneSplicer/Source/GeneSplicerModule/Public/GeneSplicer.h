// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SpliceData.h"

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"

class FGeneSplicerDNAReader;

DECLARE_LOG_CATEGORY_EXTERN(LogGeneSplicer, Log, All);

namespace gs4
{
class GeneSplicer;
}  // namespace gs4

/** A wrapper class for GeneSplicer **/
class GENESPLICERMODULE_API FGeneSplicer
{
	public:

	/**
		@brief Implementation type for GeneSplicer calculations.
	*/
	enum class ECalculationType: uint8
	{
		Scalar,  ///< scalar implementation
		SSE  ///< vectorized implementation
	};

	/**
		@brief Constructor
		@param calculationType
			Determines which algorithm implementation is used for splicing
	*/
	explicit FGeneSplicer(ECalculationType calculationType);
	~FGeneSplicer();

	FGeneSplicer(const FGeneSplicer&) = delete;
	FGeneSplicer& operator=(const FGeneSplicer&) = delete;

	FGeneSplicer(FGeneSplicer&& rhs);
	FGeneSplicer& operator=(FGeneSplicer&& rhs);

	/**
		@brief Run all the individual splicers.
		@param spliceData
			Contains all the necessary input data that is used during splicing.
		@param Output
			Output parameter - the DNA Reader that will contain the spliced DNA data.
	*/
	void Splice(const FSpliceData& InMixData, FGeneSplicerDNAReader& Output);
	/**
		@brief Run only the neutral mesh splicer.
		@param spliceData
			Contains all the necessary input data that is used during splicing.
		@param Output
			Output parameter - the DNA Reader that will contain the spliced DNA data.
	*/
	void SpliceNeutralMeshes(const FSpliceData& InMixData, FGeneSplicerDNAReader& Output);
	/**
		@brief Run only the blend shape splicer.
		@param spliceData
			Contains all the necessary input data that is used during splicing.
		@param Output
			Output parameter - the DNA Reader that will contain the spliced DNA data.
	*/
	void SpliceBlendShapes(const FSpliceData& InMixData, FGeneSplicerDNAReader& Output);
	/**
		@brief Run only the neutral joint splicer.
		@param spliceData
			Contains all the necessary input data that is used during splicing.
		@param Output
			Output parameter - the DNA Reader that will contain the spliced DNA data.
	*/
	void SpliceNeutralJoints(const FSpliceData& InMixData, FGeneSplicerDNAReader& Output);
	/**
		@brief Run only the joint behavior splicer.
		@param spliceData
			Contains all the necessary input data that is used during splicing.
		@param Output
			Output parameter - the DNA Reader that will contain the spliced DNA data.
	*/
	void SpliceJointBehavior(const FSpliceData& InMixData, FGeneSplicerDNAReader& Output);
	/**
		@brief Run only the skin weight splicer.
		@param spliceData
			Contains all the necessary input data that is used during splicing.
		@param Output
			Output parameter - the DNA Reader that will contain the spliced DNA data.
	*/
	void SpliceSkinWeights(const FSpliceData& InMixData, FGeneSplicerDNAReader& Output);

private:
	TUniquePtr<gs4::GeneSplicer> GeneSplicerPtr;
};
