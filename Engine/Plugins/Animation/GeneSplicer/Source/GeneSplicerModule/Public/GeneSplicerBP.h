// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "RegionAffiliationAsset.h"
#include "SpliceDataBP.h"

#include "GeneSplicerBP.generated.h"


UCLASS()
class GENESPLICERMODULE_API UGeneSplicerBP : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	static bool CreateGenePool(const FString& DNAFolderPath, const FString& ArchtypePath, const FString& GenePoolOutputPath);

	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	static bool CreateArchetype(const FString& DNAFolderPath, URegionAffiliationAsset* RafAsset, const FString& ArchetypeOutputPath);

	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	static void Splice(USpliceData* SpliceData);

};
