// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DNAReader.h"

#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

/**
	@brief A UE wrapper for a special purpose DNA Reader type which serves as the output parameter of GeneSplicer.
*/
class GENESPLICERMODULE_API FGeneSplicerDNAReader : public IDNAReader
{
public:
	explicit FGeneSplicerDNAReader(IDNAReader* Source);

	// Header
	uint16 GetFileFormatGeneration() const override;
	uint16 GetFileFormatVersion() const override;
	// Descriptor
	FString GetName() const override;
	EArchetype GetArchetype() const override;
	EGender GetGender() const override;
	uint16 GetAge() const override;
	uint32 GetMetaDataCount() const override;
	FString GetMetaDataKey(uint32 Index) const override;
	FString GetMetaDataValue(const FString& Key) const override;
	ETranslationUnit GetTranslationUnit() const override;
	ERotationUnit GetRotationUnit() const override;
	FCoordinateSystem GetCoordinateSystem() const override;
	ERotationSequence GetRotationSequence() const override;
	FRotationSign GetRotationSign() const override;
	EFaceWindingOrder GetFaceWindingOrder() const override;
	uint16 GetLODCount() const override;
	uint16 GetDBMaxLOD() const override;
	FString GetDBComplexity() const override;
	FString GetDBName() const override;
	// Definition
	uint16 GetGUIControlCount() const override;
	FString GetGUIControlName(uint16 Index) const override;
	uint16 GetRawControlCount() const override;
	FString GetRawControlName(uint16 Index) const override;
	uint16 GetJointCount() const override;
	FString GetJointName(uint16 Index) const override;
	uint16 GetJointIndexListCount() const override;
	TArrayView<const uint16> GetJointIndicesForLOD(uint16 LOD) const override;
	uint16 GetJointParentIndex(uint16 Index) const override;
	uint16 GetBlendShapeChannelCount() const override;
	FString GetBlendShapeChannelName(uint16 Index) const override;
	uint16 GetBlendShapeChannelIndexListCount() const override;
	TArrayView<const uint16> GetBlendShapeChannelIndicesForLOD(uint16 LOD) const override;
	uint16 GetAnimatedMapCount() const override;
	FString GetAnimatedMapName(uint16 Index) const override;
	uint16 GetAnimatedMapIndexListCount() const override;
	TArrayView<const uint16> GetAnimatedMapIndicesForLOD(uint16 LOD) const override;
	uint16 GetMeshCount() const override;
	FString GetMeshName(uint16 Index) const override;
	uint16 GetMeshIndexListCount() const override;
	TArrayView<const uint16> GetMeshIndicesForLOD(uint16 LOD) const override;
	uint16 GetMeshBlendShapeChannelMappingCount() const override;
	FMeshBlendShapeChannelMapping GetMeshBlendShapeChannelMapping(uint16 Index) const override;
	TArrayView<const uint16> GetMeshBlendShapeChannelMappingIndicesForLOD(uint16 LOD) const override;
	FVector GetNeutralJointTranslation(uint16 Index) const override;
	FVector GetNeutralJointRotation(uint16 Index) const override;
	// Behavior
	TArrayView<const uint16> GetGUIToRawInputIndices() const override;
	TArrayView<const uint16> GetGUIToRawOutputIndices() const override;
	TArrayView<const float> GetGUIToRawFromValues() const override;
	TArrayView<const float> GetGUIToRawToValues() const override;
	TArrayView<const float> GetGUIToRawSlopeValues() const override;
	TArrayView<const float> GetGUIToRawCutValues() const override;
	uint16 GetPSDCount() const override;
	TArrayView<const uint16> GetPSDRowIndices() const override;
	TArrayView<const uint16> GetPSDColumnIndices() const override;
	TArrayView<const float> GetPSDValues() const override;
	uint16 GetJointRowCount() const override;
	uint16 GetJointColumnCount() const override;
	TArrayView<const uint16> GetJointVariableAttributeIndices(uint16 LOD) const override;
	uint16 GetJointGroupCount() const override;
	TArrayView<const uint16> GetJointGroupLODs(uint16 JointGroupIndex) const override;
	TArrayView<const uint16> GetJointGroupInputIndices(uint16 JointGroupIndex) const override;
	TArrayView<const uint16> GetJointGroupOutputIndices(uint16 JointGroupIndex) const override;
	TArrayView<const float> GetJointGroupValues(uint16 JointGroupIndex) const override;
	TArrayView<const uint16> GetJointGroupJointIndices(uint16 JointGroupIndex) const override;
	TArrayView<const uint16> GetBlendShapeChannelLODs() const override;
	TArrayView<const uint16> GetBlendShapeChannelOutputIndices() const override;
	TArrayView<const uint16> GetBlendShapeChannelInputIndices() const override;
	TArrayView<const uint16> GetAnimatedMapLODs() const override;
	TArrayView<const uint16> GetAnimatedMapInputIndices() const override;
	TArrayView<const uint16> GetAnimatedMapOutputIndices() const override;
	TArrayView<const float> GetAnimatedMapFromValues() const override;
	TArrayView<const float> GetAnimatedMapToValues() const override;
	TArrayView<const float> GetAnimatedMapSlopeValues() const override;
	TArrayView<const float> GetAnimatedMapCutValues() const override;
	// Geometry
	uint32 GetVertexPositionCount(uint16 MeshIndex) const override;
	FVector GetVertexPosition(uint16 MeshIndex, uint32 VertexIndex) const override;
	TArrayView<const float> GetVertexPositionXs(uint16 MeshIndex) const override;
	TArrayView<const float> GetVertexPositionYs(uint16 MeshIndex) const override;
	TArrayView<const float> GetVertexPositionZs(uint16 MeshIndex) const override;
	uint32 GetVertexTextureCoordinateCount(uint16 MeshIndex) const override;
	FTextureCoordinate GetVertexTextureCoordinate(uint16 MeshIndex, uint32 TextureCoordinateIndex) const override;
	uint32 GetVertexNormalCount(uint16 MeshIndex) const override;
	FVector GetVertexNormal(uint16 MeshIndex, uint32 NormalIndex) const override;
	uint32 GetFaceCount(uint16 MeshIndex) const override;
	TArrayView<const uint32> GetFaceVertexLayoutIndices(uint16 MeshIndex, uint32 FaceIndex) const override;
	uint32 GetVertexLayoutCount(uint16 MeshIndex) const override;
	FVertexLayout GetVertexLayout(uint16 MeshIndex, uint32 LayoutIndex) const override;
	uint16 GetMaximumInfluencePerVertex(uint16 MeshIndex) const override;
	uint32 GetSkinWeightsCount(uint16 MeshIndex) const override;
	TArrayView<const float> GetSkinWeightsValues(uint16 MeshIndex, uint32 VertexIndex) const override;
	TArrayView<const uint16> GetSkinWeightsJointIndices(uint16 MeshIndex, uint32 VertexIndex) const override;
	uint16 GetBlendShapeTargetCount(uint16 MeshIndex) const override;
	uint16 GetBlendShapeChannelIndex(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	uint32 GetBlendShapeTargetDeltaCount(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	FVector GetBlendShapeTargetDelta(uint16 MeshIndex, uint16 BlendShapeTargetIndex, uint32 DeltaIndex) const override;
	TArrayView<const float> GetBlendShapeTargetDeltaXs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	TArrayView<const float> GetBlendShapeTargetDeltaYs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	TArrayView<const float> GetBlendShapeTargetDeltaZs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	TArrayView<const uint32> GetBlendShapeTargetVertexIndices(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	// Machine Learned Behavior
	uint16 GetMLControlCount() const override;
	FString GetMLControlName(uint16 Index) const override;
	uint16 GetNeuralNetworkCount() const override;
	uint16 GetNeuralNetworkIndexListCount() const override;
	TArrayView<const uint16> GetNeuralNetworkIndicesForLOD(uint16 LOD) const override;
	uint16 GetMeshRegionCount(uint16 MeshIndex) const override;
	FString GetMeshRegionName(uint16 MeshIndex, uint16 RegionIndex) const override;
	TArrayView<const uint16> GetNeuralNetworkIndicesForMeshRegion(uint16 MeshIndex, uint16 RegionIndex) const override;
	TArrayView<const uint16> GetNeuralNetworkInputIndices(uint16 NetIndex) const override;
	TArrayView<const uint16> GetNeuralNetworkOutputIndices(uint16 NetIndex) const override;
	uint16 GetNeuralNetworkLayerCount(uint16 NetIndex) const override;
	EActivationFunction GetNeuralNetworkLayerActivationFunction(uint16 NetIndex, uint16 LayerIndex) const override;
	TArrayView<const float> GetNeuralNetworkLayerActivationFunctionParameters(uint16 NetIndex, uint16 LayerIndex) const override;
	TArrayView<const float> GetNeuralNetworkLayerBiases(uint16 NetIndex, uint16 LayerIndex) const override;
	TArrayView<const float> GetNeuralNetworkLayerWeights(uint16 NetIndex, uint16 LayerIndex) const override;
	// JointBehaviorMetadataReader
	ETranslationRepresentation GetJointTranslationRepresentation(uint16 JointIndex) const override;
	ERotationRepresentation GetJointRotationRepresentation(uint16 JointIndex) const override;
	EScaleRepresentation GetJointScaleRepresentation(uint16 JointIndex) const override;
	// RBFBehavior
	uint16 GetRBFPoseCount() const override;
	FString GetRBFPoseName(uint16 PoseIndex) const override;
	TArrayView<const uint16> GetRBFPoseJointOutputIndices(uint16 PoseIndex) const override;
	TArrayView<const uint16> GetRBFPoseBlendShapeChannelOutputIndices(uint16 PoseIndex) const override;
	TArrayView<const uint16> GetRBFPoseAnimatedMapOutputIndices(uint16 PoseIndex) const override;
	TArrayView<const float> GetRBFPoseJointOutputValues(uint16 PoseIndex) const override;
	float GetRBFPoseScale(uint16 PoseIndex) const override;
	uint16 GetRBFPoseControlCount() const override;
	FString GetRBFPoseControlName(uint16 PoseControlIndex) const override;
	TArrayView<const uint16> GetRBFPoseInputControlIndices(uint16 PoseIndex) const override;
	TArrayView<const uint16> GetRBFPoseOutputControlIndices(uint16 PoseIndex) const override;
	TArrayView<const float> GetRBFPoseOutputControlWeights(uint16 PoseIndex) const override;
	uint16 GetRBFSolverCount() const override;
	uint16 GetRBFSolverIndexListCount() const override;
	TArrayView<const uint16> GetRBFSolverIndicesForLOD(uint16 LOD) const override;
	FString GetRBFSolverName(uint16 SolverIndex) const override;
	TArrayView<const uint16> GetRBFSolverRawControlIndices(uint16 SolverIndex) const override;
	TArrayView<const uint16> GetRBFSolverPoseIndices(uint16 SolverIndex) const override;
	TArrayView<const float> GetRBFSolverRawControlValues(uint16 SolverIndex) const override;
	ERBFSolverType GetRBFSolverType(uint16 SolverIndex) const override;
	float GetRBFSolverRadius(uint16 SolverIndex) const override;
	EAutomaticRadius GetRBFSolverAutomaticRadius(uint16 SolverIndex) const override;
	float GetRBFSolverWeightThreshold(uint16 SolverIndex) const override;
	ERBFDistanceMethod GetRBFSolverDistanceMethod(uint16 SolverIndex) const override;
	ERBFNormalizeMethod GetRBFSolverNormalizeMethod(uint16 SolverIndex) const override;
	ERBFFunctionType GetRBFSolverFunctionType(uint16 SolverIndex) const override;
	ETwistAxis GetRBFSolverTwistAxis(uint16 SolverIndex) const override;
	// TwistSwingBehavior
	uint16 GetTwistCount() const override;
	ETwistAxis GetTwistSetupTwistAxis(uint16 TwistIndex) const override;
	TArrayView<const uint16> GetTwistInputControlIndices(uint16 TwistIndex) const override;
	TArrayView<const uint16> GetTwistOutputJointIndices(uint16 TwistIndex) const override;
	TArrayView<const float> GetTwistBlendWeights(uint16 TwistIndex) const override;
	uint16 GetSwingCount() const override;
	ETwistAxis GetSwingSetupTwistAxis(uint16 SwingIndex) const override;
	TArrayView<const uint16> GetSwingInputControlIndices(uint16 SwingIndex) const override;
	TArrayView<const uint16> GetSwingOutputJointIndices(uint16 SwingIndex) const override;
	TArrayView<const float> GetSwingBlendWeights(uint16 SwingIndex) const override;
	// MachineLearnedBehaviorExtReader
	uint16 GetMLTypeCount() const override;
	uint16 GetMLOperationSetCount(uint16 MLTypeIndex) const override;
	uint16 GetMLOperationCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const override;
	EMachineLearnedBehaviorOperationType GetMLOperationType(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const override;
	TArrayView<const uint32> GetMLOperationParameters(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const override;
	TArrayView<const uint16> GetMLOperationDependencyOperationSetIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const override;
	TArrayView<const uint16> GetMLOperationDependencyOperationIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const override;
	uint16 GetMLOperationIndexListCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const override;
	TArrayView<const uint16> GetMLOperationIndicesForLOD(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 LOD) const override;
	TArrayView<const uint16> GetMLJointsInputIndices() const override;
	TArrayView<const uint16> GetMLJointsOutputIndices() const override;
	TArrayView<const uint16> GetMLJointsParameterKeys() const override;
	TArrayView<const uint16> GetMLJointsParameterValues() const override;

	// Reader
	void Unload(EDNADataLayer Layer) override;

	const FDNAConfig& GetConfig() const override;

private:
	friend class FGeneSplicer;
	dna::Reader* Unwrap() const;

private:
	TUniquePtr<IDNAReader> ReaderPtr;

};
