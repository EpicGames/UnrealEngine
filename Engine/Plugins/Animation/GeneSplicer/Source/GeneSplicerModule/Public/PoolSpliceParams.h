// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenePool.h"
#include "RegionAffiliationReader.h"

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"

class IDNAReader;

namespace gs4
{
class PoolSpliceParams;
} // namespace gs4


class GENESPLICERMODULE_API FPoolSpliceParams
{
public:
	~FPoolSpliceParams();

	void SetDNAFilter(TArrayView<const uint16> DNAIndices);
	void SetMeshFilter(TArrayView<const uint16> MeshIndices);
	void SetSpliceWeights(uint16 DNAStartIndex, TArrayView<const float> Weights);
	void SetScale(float scale);
	uint16 GetDNACount() const;
	uint16 GetRegionCount() const;

private:
	friend class FSpliceData;
	FPoolSpliceParams(gs4::PoolSpliceParams* PoolSpliceParams);
	gs4::PoolSpliceParams* Unwrap() const;

private:
	gs4::PoolSpliceParams* PoolSpliceParamsPtr;
};
