// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PoolSpliceParams.h"
#include "SpliceDataBP.h"
#include "UObject/NoExportTypes.h"

#include "PoolSpliceParamsBP.generated.h"

UCLASS(BlueprintType)
class GENESPLICERMODULE_API UPoolSpliceParams : public UObject
{
	GENERATED_BODY()

public:
	UPoolSpliceParams() = default;

	UFUNCTION(BlueprintCallable, Category="GeneSplicer")
	void RegisterToSpliceData(USpliceData* SpliceData, const FString& Name, const UGenePoolAsset* GenePoolAsset, URegionAffiliationAsset* Raf);

	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	int32 GetDNACount() const;

	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	int32 GetRegionCount() const;

	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	const TArray<FString>& GetRegionNames() const;

	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	void SetSpliceWeights(int32 DNAStartIndex, const TArray<float>& Weights);

private:
	TSharedPtr<FPoolSpliceParams> PoolSpliceParams;
	TArray<FString> RegionNames;
	
};
