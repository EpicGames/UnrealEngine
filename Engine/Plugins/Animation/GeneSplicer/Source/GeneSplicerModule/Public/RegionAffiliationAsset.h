// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RegionAffiliationReader.h"
#include "UObject/NoExportTypes.h"

#include "RegionAffiliationAsset.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogRegionAffiliationAsset, Log, All);

UCLASS(BlueprintType)
class GENESPLICERMODULE_API URegionAffiliationAsset : public UObject
{
	GENERATED_BODY()
public:
	URegionAffiliationAsset();

	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	int32 GetRegionCount() const;
	
	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	FString GetRegionName(int32 RegionIndex) const;

	void SetRegionAffiliationPtr(TSharedPtr<FRegionAffiliationReader> RegionAffiliationReaderPtr);
	const TSharedPtr<FRegionAffiliationReader>& getRegionAffiliationReaderPtr();

	void Serialize(FArchive& Ar) override;
private:
	// Synchronize raf updates
	FRWLock UpdateLock;
	TSharedPtr<FRegionAffiliationReader> RegionAffiliationReader;
};
