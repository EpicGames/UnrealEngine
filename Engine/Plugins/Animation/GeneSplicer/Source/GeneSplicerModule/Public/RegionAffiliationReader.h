// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"

namespace raf
{
	class RegionAffiliationBinaryStreamReader;
}  // namespace raf

namespace trio {
	class BoundedIOStream;
} // namespace trio


DECLARE_LOG_CATEGORY_EXTERN(LogRegionAffiliationReader, Log, All);

class GENESPLICERMODULE_API FRegionAffiliationReader
{
public:
	FRegionAffiliationReader(const FString& FilePath);
	FRegionAffiliationReader(FArchive& Ar);
	~FRegionAffiliationReader();

	FRegionAffiliationReader(const FRegionAffiliationReader&) = delete;
	FRegionAffiliationReader& operator=(const FRegionAffiliationReader&) = delete;

	FRegionAffiliationReader(FRegionAffiliationReader&& rhs) = default;
	FRegionAffiliationReader& operator=(FRegionAffiliationReader&& rhs) = default;

	uint16 GetRegionNum() const;
	FString getRegionName(uint16 RegionIndex) const;
	void GetVertexRegionAffiliation(uint16 MeshId, int32 VertexId, TArray<float>& OutRegionAffiliations);

	void Serialize(FArchive& Ar) const;
	void WriteToFile(const FString& Path) const;

private:
	friend class FPoolSpliceParams;
	friend class FSpliceData;
	raf::RegionAffiliationBinaryStreamReader* Unwrap() const;

private:
	struct FRegionAffiliationReaderDeleter 
	{
		void operator()(raf::RegionAffiliationBinaryStreamReader* Pointer);
	};
	TUniquePtr<raf::RegionAffiliationBinaryStreamReader, FRegionAffiliationReaderDeleter> RegionAffiliationPtr;

};