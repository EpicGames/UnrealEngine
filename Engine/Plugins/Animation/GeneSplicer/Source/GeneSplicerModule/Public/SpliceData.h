// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenePool.h"
#include "PoolSpliceParams.h"
#include "RegionAffiliationReader.h"

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"

class IDNAReader;

namespace gs4
{
class SpliceData;
} // namespace gs4


class GENESPLICERMODULE_API FSpliceData
{
public:
	explicit FSpliceData();

	~FSpliceData();

	FSpliceData(const FSpliceData&) = delete;
	FSpliceData& operator=(const FSpliceData&) = delete;

	FSpliceData(FSpliceData&&);
	FSpliceData& operator=(FSpliceData&&);

	void RegisterGenePool(const FString& name, const FRegionAffiliationReader& RegionAffiliationReader, const TSharedPtr<FGenePool>& GenePool);
	void UnregisterGenePool(const FString& name);
	TSharedPtr<FPoolSpliceParams> GetPoolParams(const FString& name);
	void SetBaseArchetype(TSharedPtr<const IDNAReader> baseArchetype);
private:
	friend class FGeneSplicer;
	gs4::SpliceData* Unwrap() const;

private:
	TUniquePtr<gs4::SpliceData> SpliceDataPtr;
};
