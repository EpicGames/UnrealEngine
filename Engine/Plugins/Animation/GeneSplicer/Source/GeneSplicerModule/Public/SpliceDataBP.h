// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SkeletalMeshComponent.h"
#include "DNAToSkelMeshMap.h"
#include "GenePoolAsset.h"
#include "GeneSplicerDNAReader.h"
#include "RegionAffiliationAsset.h"
#include "SpliceData.h"
#include "UObject/NoExportTypes.h"

#include "SpliceDataBP.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSpliceData, Log, All);

class UGeneSplicer;


UCLASS(BlueprintType)
class GENESPLICERMODULE_API USpliceData : public UObject
{
	GENERATED_BODY()

public:
	USpliceData();
	~USpliceData() override;

	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	void RegisterGenePool(const FString& Name, UGenePoolAsset* GenePoolAsset, URegionAffiliationAsset* raf);

	TSharedPtr<FPoolSpliceParams> InitPoolSpliceParams(const FString& Name, UGenePoolAsset* GenePoolAsset, URegionAffiliationAsset* raf);

	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	void SetSpliceWeights(const FString& Name, int32 DNAStartIndex, const TArray<float>& Weights);

	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	void SetArchetype(const FString& path);

	UFUNCTION(BlueprintCallable, Category = "GeneSplicer")
	void SetSkeletalMeshComponent(USkeletalMeshComponent* NewSkelMeshComponent);

	UFUNCTION(BlueprintPure, Category = "GeneSplicer")
	USkeletalMeshComponent* GetSkeletalMeshComponent() const;

	TSharedPtr<FDNAToSkelMeshMap> GetDNASkelMeshMap() const;

	TSharedPtr<FGeneSplicerDNAReader> GetOutputDNA() const;
	
	FSpliceData& GetSpliceDataImpl();

	const FSpliceData& GetSpliceDataImpl() const;

private:
	void GenerateDNASkelMeshMapping();


private:
	FSpliceData SpliceDataImpl;
	USkeletalMeshComponent* SkelMeshComponent;
	TSharedPtr<FDNAToSkelMeshMap> DNASkelMeshMap;
	TSharedPtr<FGeneSplicerDNAReader> OutputDNA;

};
